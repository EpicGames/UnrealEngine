{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Sequence Navigator",
	"Description": "An advanced sequence bread-crumbing tool with editing capabilities.",
	"Category": "Animation",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"EnabledByDefault": true,
	"Modules": [
		{
			"Name": "SequenceNavigator",
			"Type": "Editor",
			"LoadingPhase": "PreDefault"
		}
	]
}
