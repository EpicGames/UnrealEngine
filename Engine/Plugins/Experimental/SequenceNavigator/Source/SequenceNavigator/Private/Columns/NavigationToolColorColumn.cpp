// Copyright Epic Games, Inc. All Rights Reserved.

#include "Columns/NavigationToolColorColumn.h"
#include "Extensions/IColorExtension.h"
#include "INavigationToolView.h"
#include "Items/NavigationToolSequence.h"
#include "Widgets/Columns/SNavigationToolColor.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SNavigationToolTreeRow.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Views/SHeaderRow.h"

#define LOCTEXT_NAMESPACE "NavigationToolColorColumn"

namespace UE::SequenceNavigator
{

UE_SEQUENCER_DEFINE_CASTABLE(FNavigationToolColorColumn)

FText FNavigationToolColorColumn::GetColumnDisplayNameText() const
{
	return LOCTEXT("ColorColumn", "Color");
}

SHeaderRow::FColumn::FArguments FNavigationToolColorColumn::ConstructHeaderRowColumn(const TSharedRef<INavigationToolView>& InView
	, const float InFillSize)
{
	const FName ColumnId = GetColumnId();

	return SHeaderRow::Column(ColumnId)
	    .FixedWidth(12.f)
		.HAlignHeader(HAlign_Center)
		.VAlignHeader(VAlign_Center)
		.HAlignCell(HAlign_Center)
		.VAlignCell(VAlign_Center)
		.DefaultLabel(GetColumnDisplayNameText())
	    .DefaultTooltip(GetColumnDisplayNameText())
		[
			// Display nothing in the column header instead of the default label
			SNew(SBox)
		];
}

TSharedRef<SWidget> FNavigationToolColorColumn::ConstructRowWidget(const FNavigationToolViewModelPtr& InItem
	, const TSharedRef<INavigationToolView>& InView
	, const TSharedRef<SNavigationToolTreeRow>& InRow)
{
	if (!InItem.AsModel()->IsA<IColorExtension>())
	{
		return SNullWidget::NullWidget;
	}

	return SNew(SNavigationToolColor, InItem, InView, InRow);
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
