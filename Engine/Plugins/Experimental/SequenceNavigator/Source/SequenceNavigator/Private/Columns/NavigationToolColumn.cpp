// Copyright Epic Games, Inc. All Rights Reserved.

#include "Columns/NavigationToolColumn.h"

namespace UE::SequenceNavigator
{

UE_SEQUENCER_DEFINE_CASTABLE(FNavigationToolColumn)

}
