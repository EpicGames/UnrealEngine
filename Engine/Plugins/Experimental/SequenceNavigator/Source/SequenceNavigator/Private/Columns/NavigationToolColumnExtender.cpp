// Copyright Epic Games, Inc. All Rights Reserved.

#include "Columns/NavigationToolColumnExtender.h"
#include "Columns/NavigationToolColumn.h"

namespace UE::SequenceNavigator
{

void FNavigationToolColumnExtender::AddColumn(const TSharedPtr<FNavigationToolColumn>& InColumn
	, const ENavigationToolExtensionPosition InExtensionPosition
	, const FName InReferenceColumnId)
{
	// By default set the Placement Index to be the next index after the last.
	int32 PlacementIndex = Columns.Num();

	if (InReferenceColumnId != NAME_None)
	{
		const int32 ReferenceColumnIndex = FindColumnIndex(InReferenceColumnId);
		if (ReferenceColumnIndex != INDEX_NONE)
		{
			PlacementIndex = InExtensionPosition == ENavigationToolExtensionPosition::Before
				? ReferenceColumnIndex
				: ReferenceColumnIndex + 1;
		}
	}

	Columns.EmplaceAt(PlacementIndex, InColumn);
}

int32 FNavigationToolColumnExtender::FindColumnIndex(const FName InColumnId) const
{
	const TSharedPtr<FNavigationToolColumn>* const FoundColumn = Columns.FindByPredicate([InColumnId](const TSharedPtr<FNavigationToolColumn>& InColumn)
		{
			return InColumn.IsValid() && InColumn->GetColumnId() == InColumnId;
		});
	if (FoundColumn)
	{
		return static_cast<int32>(FoundColumn - Columns.GetData());
	}
	return INDEX_NONE;
}

} // namespace UE::SequenceNavigator
