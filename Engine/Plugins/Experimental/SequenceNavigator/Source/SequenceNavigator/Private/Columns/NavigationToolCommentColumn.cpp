// Copyright Epic Games, Inc. All Rights Reserved.

#include "Columns/NavigationToolCommentColumn.h"
#include "INavigationToolView.h"
#include "Widgets/Columns/SNavigationToolComment.h"
#include "Widgets/SNavigationToolTreeRow.h"
#include "Widgets/Views/SHeaderRow.h"

#define LOCTEXT_NAMESPACE "NavigationToolCommentColumn"

namespace UE::SequenceNavigator
{

using namespace Sequencer;

UE_SEQUENCER_DEFINE_CASTABLE(FNavigationToolCommentColumn)

FText FNavigationToolCommentColumn::GetColumnDisplayNameText() const
{
	return LOCTEXT("CommentColumn", "Comment");
}

const FSlateBrush* FNavigationToolCommentColumn::GetIconBrush() const
{
	return FAppStyle::GetBrush(TEXT("FoliageEditMode.BubbleBorder"));
}

SHeaderRow::FColumn::FArguments FNavigationToolCommentColumn::ConstructHeaderRowColumn(const TSharedRef<INavigationToolView>& InView
	, const float InFillSize)
{
	const FName ColumnId = GetColumnId();

	return SHeaderRow::Column(ColumnId)
		.FillWidth(InFillSize)
		.HAlignHeader(HAlign_Center)
		.VAlignHeader(VAlign_Center)
		.HAlignCell(HAlign_Fill)
		.VAlignCell(VAlign_Center)
		.DefaultLabel(GetColumnDisplayNameText())
		.OnGetMenuContent(InView, &INavigationToolView::GetColumnMenuContent, ColumnId);
}

TSharedRef<SWidget> FNavigationToolCommentColumn::ConstructRowWidget(const FNavigationToolViewModelPtr& InItem
	, const TSharedRef<INavigationToolView>& InView
	, const TSharedRef<SNavigationToolTreeRow>& InRow)
{
	return SNew(SNavigationToolComment, InItem.ImplicitCast(), InView, InRow);
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
