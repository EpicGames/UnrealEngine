// Copyright Epic Games, Inc. All Rights Reserved.

#include "Columns/NavigationToolDeactiveStateColumn.h"
#include "INavigationToolView.h"
#include "Items/NavigationToolItem.h"
#include "MVVM/Extensions/IDeactivatableExtension.h"
#include "Widgets/Columns/SNavigationToolDeactiveState.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SNavigationToolTreeRow.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/SNullWidget.h"

#define LOCTEXT_NAMESPACE "NavigationToolDeactiveStateColumn"

namespace UE::SequenceNavigator
{

UE_SEQUENCER_DEFINE_CASTABLE(FNavigationToolDeactiveStateColumn)

FText FNavigationToolDeactiveStateColumn::GetColumnDisplayNameText() const
{
	return LOCTEXT("NavigationToolDeactiveStateColumn", "Sequence Evaluation");
}

const FSlateBrush* FNavigationToolDeactiveStateColumn::GetIconBrush() const
{
	return FAppStyle::GetBrush(TEXT("Sequencer.Column.Mute"));
}

SHeaderRow::FColumn::FArguments FNavigationToolDeactiveStateColumn::ConstructHeaderRowColumn(const TSharedRef<INavigationToolView>& InView
	, const float InFillSize)
{
	const FName ColumnId = GetColumnId();

	return SHeaderRow::Column(ColumnId)
		.FixedWidth(24.f)
		.HAlignHeader(HAlign_Left)
		.VAlignHeader(VAlign_Center)
		.HAlignCell(HAlign_Center)
		.VAlignCell(VAlign_Center)
		.DefaultLabel(GetColumnDisplayNameText())
		.DefaultTooltip(GetColumnDisplayNameText())
		.OnGetMenuContent(InView, &INavigationToolView::GetColumnMenuContent, ColumnId)
		[
			SNew(SImage)
			.ColorAndOpacity(FSlateColor::UseForeground())
			.Image(this, &FNavigationToolDeactiveStateColumn::GetIconBrush)
		];
}

TSharedRef<SWidget> FNavigationToolDeactiveStateColumn::ConstructRowWidget(const FNavigationToolViewModelPtr& InItem
	, const TSharedRef<INavigationToolView>& InView
	, const TSharedRef<SNavigationToolTreeRow>& InRow)
{
	if (!InItem.AsModel()->IsA<Sequencer::IDeactivatableExtension>())
	{
		return SNullWidget::NullWidget;
	}

	return SNew(SNavigationToolDeactiveState, SharedThis(this), InItem.ImplicitCast(), InView, InRow);
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
