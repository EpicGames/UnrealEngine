// Copyright Epic Games, Inc. All Rights Reserved.

#include "Columns/NavigationToolHBiasColumn.h"
#include "INavigationToolView.h"
#include "Widgets/Columns/SNavigationToolHBias.h"
#include "Widgets/SNavigationToolTreeRow.h"
#include "Widgets/Views/SHeaderRow.h"

#define LOCTEXT_NAMESPACE "NavigationToolHBiasColumn"

namespace UE::SequenceNavigator
{

UE_SEQUENCER_DEFINE_CASTABLE(FNavigationToolHBiasColumn)

FText FNavigationToolHBiasColumn::GetColumnDisplayNameText() const
{
	return LOCTEXT("HBiasColumn", "HBias");
}

const FSlateBrush* FNavigationToolHBiasColumn::GetIconBrush() const
{
	return FAppStyle::GetBrush(TEXT("ClassIcon.TimelineComponent"));
}

SHeaderRow::FColumn::FArguments FNavigationToolHBiasColumn::ConstructHeaderRowColumn(const TSharedRef<INavigationToolView>& InView
	, const float InFillSize)
{
	const FName ColumnId = GetColumnId();

	return SHeaderRow::Column(ColumnId)
		.FillWidth(InFillSize)
		.HAlignHeader(HAlign_Center)
		.VAlignHeader(VAlign_Center)
		.HAlignCell(HAlign_Fill)
		.VAlignCell(VAlign_Center)
		.DefaultLabel(GetColumnDisplayNameText())
		.OnGetMenuContent(InView, &INavigationToolView::GetColumnMenuContent, ColumnId);
}

TSharedRef<SWidget> FNavigationToolHBiasColumn::ConstructRowWidget(const FNavigationToolViewModelPtr& InItem
	, const TSharedRef<INavigationToolView>& InView
	, const TSharedRef<SNavigationToolTreeRow>& InRow)
{
	return SNew(SNavigationToolHBias, InItem.ImplicitCast(), InView, InRow);
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
