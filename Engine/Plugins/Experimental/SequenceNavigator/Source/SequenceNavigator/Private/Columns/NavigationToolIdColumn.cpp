// Copyright Epic Games, Inc. All Rights Reserved.

#include "Columns/NavigationToolIdColumn.h"
#include "Extensions/IIdExtension.h"
#include "Items/NavigationToolSequence.h"
#include "INavigationToolView.h"
#include "Widgets/Columns/SNavigationToolId.h"
#include "Widgets/SNavigationToolTreeRow.h"
#include "Widgets/Views/SHeaderRow.h"

#define LOCTEXT_NAMESPACE "NavigationToolIdColumn"

namespace UE::SequenceNavigator
{

UE_SEQUENCER_DEFINE_CASTABLE(FNavigationToolIdColumn)

FText FNavigationToolIdColumn::GetColumnDisplayNameText() const
{
	return LOCTEXT("IdColumn", "Id");
}

const FSlateBrush* FNavigationToolIdColumn::GetIconBrush() const
{
	return FAppStyle::GetBrush(TEXT("ClassIcon.LevelSequence"));
}

SHeaderRow::FColumn::FArguments FNavigationToolIdColumn::ConstructHeaderRowColumn(const TSharedRef<INavigationToolView>& InView
	, const float InFillSize)
{
	const FName ColumnId = GetColumnId();

	return SHeaderRow::Column(ColumnId)
		.FillWidth(InFillSize)
		.HAlignHeader(HAlign_Center)
		.VAlignHeader(VAlign_Center)
		.HAlignCell(HAlign_Fill)
		.VAlignCell(VAlign_Center)
		.DefaultLabel(GetColumnDisplayNameText())
		.OnGetMenuContent(InView, &INavigationToolView::GetColumnMenuContent, ColumnId);
}

TSharedRef<SWidget> FNavigationToolIdColumn::ConstructRowWidget(const FNavigationToolViewModelPtr& InItem
	, const TSharedRef<INavigationToolView>& InView
	, const TSharedRef<SNavigationToolTreeRow>& InRow)
{
	if (!InItem.AsModel()->IsA<IIdExtension>())
	{
		return SNullWidget::NullWidget;
	}

	return SNew(SNavigationToolId, InItem.ImplicitCast(), InView, InRow);
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
