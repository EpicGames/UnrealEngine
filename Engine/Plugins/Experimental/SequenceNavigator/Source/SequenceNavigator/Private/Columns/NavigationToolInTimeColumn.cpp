// Copyright Epic Games, Inc. All Rights Reserved.

#include "Columns/NavigationToolInTimeColumn.h"
#include "Extensions/IInTimeExtension.h"
#include "INavigationToolView.h"
#include "Items/NavigationToolItem.h"
#include "Widgets/Columns/SNavigationToolInTime.h"
#include "Widgets/SNavigationToolTreeRow.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Views/SHeaderRow.h"

#define LOCTEXT_NAMESPACE "NavigationToolInTimeColumn"

namespace UE::SequenceNavigator
{

UE_SEQUENCER_DEFINE_CASTABLE(FNavigationToolInTimeColumn)

FText FNavigationToolInTimeColumn::GetColumnDisplayNameText() const
{
	return LOCTEXT("InTimeColumn", "In");
}

const FSlateBrush* FNavigationToolInTimeColumn::GetIconBrush() const
{
	return FAppStyle::GetBrush(TEXT("Icons.Alignment.Left"));
}

SHeaderRow::FColumn::FArguments FNavigationToolInTimeColumn::ConstructHeaderRowColumn(const TSharedRef<INavigationToolView>& InView
	, const float InFillSize)
{
	const FName ColumnId = GetColumnId();

	return SHeaderRow::Column(ColumnId)
		.FillWidth(InFillSize)
		.HAlignHeader(HAlign_Center)
		.VAlignHeader(VAlign_Center)
		.HAlignCell(HAlign_Fill)
		.VAlignCell(VAlign_Center)
		.DefaultLabel(GetColumnDisplayNameText())
		.OnGetMenuContent(InView, &INavigationToolView::GetColumnMenuContent, ColumnId);
}

TSharedRef<SWidget> FNavigationToolInTimeColumn::ConstructRowWidget(const FNavigationToolViewModelPtr& InItem
	, const TSharedRef<INavigationToolView>& InView
	, const TSharedRef<SNavigationToolTreeRow>& InRow)
{
	if (!InItem.AsModel()->IsA<IInTimeExtension>())
	{
		return SNullWidget::NullWidget;
	}

	return SNew(SNavigationToolInTime, InItem.ImplicitCast(), InView, InRow);
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
