// Copyright Epic Games, Inc. All Rights Reserved.

#include "Columns/NavigationToolItemsColumn.h"
#include "INavigationToolView.h"
#include "Widgets/Columns/SNavigationToolItemList.h"
#include "Widgets/SNavigationToolTreeRow.h"
#include "Widgets/Views/SHeaderRow.h"

#define LOCTEXT_NAMESPACE "NavigationToolItemsColumn"

namespace UE::SequenceNavigator
{

UE_SEQUENCER_DEFINE_CASTABLE(FNavigationToolItemsColumn)

FText FNavigationToolItemsColumn::GetColumnDisplayNameText() const
{
	return LOCTEXT("DisplayName", "Items");
}

const FSlateBrush* FNavigationToolItemsColumn::GetIconBrush() const
{
	return FAppStyle::GetBrush(TEXT("ClassIcon.AbilitySystemComponent"));
}

SHeaderRow::FColumn::FArguments FNavigationToolItemsColumn::ConstructHeaderRowColumn(const TSharedRef<INavigationToolView>& InView
	, const float InFillSize)
{
	const FName ColumnId = GetColumnId();

	return SHeaderRow::Column(ColumnId)
		.FillWidth(InFillSize)
		.HAlignHeader(HAlign_Left)
		.VAlignHeader(VAlign_Center)
		.HAlignCell(HAlign_Fill)
		.VAlignCell(VAlign_Center)
		.DefaultLabel(GetColumnDisplayNameText())
		.OnGetMenuContent(InView, &INavigationToolView::GetColumnMenuContent, ColumnId);
}

TSharedRef<SWidget> FNavigationToolItemsColumn::ConstructRowWidget(const FNavigationToolViewModelPtr& InItem
	, const TSharedRef<INavigationToolView>& InView
	, const TSharedRef<SNavigationToolTreeRow>& InRow)
{
	return SNew(SNavigationToolItemList, InItem.ImplicitCast(), InView, InRow);
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
