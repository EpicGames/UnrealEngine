// Copyright Epic Games, Inc. All Rights Reserved.

#include "Columns/NavigationToolLabelColumn.h"
#include "INavigationToolView.h"
#include "Items/NavigationToolItem.h"
#include "Widgets/SNavigationToolTreeRow.h"
#include "Widgets/Views/SHeaderRow.h"

#define LOCTEXT_NAMESPACE "NavigationToolLabelColumn"

namespace UE::SequenceNavigator
{

UE_SEQUENCER_DEFINE_CASTABLE(FNavigationToolLabelColumn)

FText FNavigationToolLabelColumn::GetColumnDisplayNameText() const
{
	return LOCTEXT("LabelColumn", "Label");
}

const FSlateBrush* FNavigationToolLabelColumn::GetIconBrush() const
{
	return FAppStyle::GetBrush(TEXT("ClassIcon.FontFace"));
}

SHeaderRow::FColumn::FArguments FNavigationToolLabelColumn::ConstructHeaderRowColumn(const TSharedRef<INavigationToolView>& InToolView
	, const float InFillSize)
{
	const FName ColumnId = GetColumnId();

	return SHeaderRow::Column(ColumnId)
		.FillWidth(InFillSize)
		.DefaultLabel(GetColumnDisplayNameText())
		.ShouldGenerateWidget(true)
		.OnGetMenuContent(InToolView, &INavigationToolView::GetColumnMenuContent, ColumnId);
}

TSharedRef<SWidget> FNavigationToolLabelColumn::ConstructRowWidget(const FNavigationToolViewModelPtr& InItem
	, const TSharedRef<INavigationToolView>& InView
	, const TSharedRef<SNavigationToolTreeRow>& InRow)
{
	return InItem->GenerateLabelWidget(InRow);
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
