// Copyright Epic Games, Inc. All Rights Reserved.

#include "Columns/NavigationToolLockColumn.h"
#include "INavigationToolView.h"
#include "Items/NavigationToolItem.h"
#include "MVVM/Extensions/ILockableExtension.h"
#include "Styling/StyleColors.h"
#include "Widgets/Columns/SNavigationToolLock.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SNavigationToolTreeRow.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Views/SHeaderRow.h"

#define LOCTEXT_NAMESPACE "NavigationToolLockColumn"

namespace UE::SequenceNavigator
{

UE_SEQUENCER_DEFINE_CASTABLE(FNavigationToolLockColumn)

FText FNavigationToolLockColumn::GetColumnDisplayNameText() const
{
	return LOCTEXT("LockColumn", "Lock");
}

const FSlateBrush* FNavigationToolLockColumn::GetIconBrush() const
{
	return FAppStyle::GetBrush(TEXT("Icons.Lock"));
}

SHeaderRow::FColumn::FArguments FNavigationToolLockColumn::ConstructHeaderRowColumn(const TSharedRef<INavigationToolView>& InView
	, const float InFillSize)
{
	const FName ColumnId = GetColumnId();

	return SHeaderRow::Column(ColumnId)
		.FixedWidth(24.f)
		.HAlignHeader(HAlign_Center)
		.VAlignHeader(VAlign_Center)
		.HAlignCell(HAlign_Center)
		.VAlignCell(VAlign_Center)
		.DefaultLabel(GetColumnDisplayNameText())
		.DefaultTooltip(GetColumnDisplayNameText())
		[
			SNew(SImage)
			.ColorAndOpacity(FStyleColors::Foreground)
			.Image(FAppStyle::GetBrush(TEXT("Icons.Lock")))
		];
}

TSharedRef<SWidget> FNavigationToolLockColumn::ConstructRowWidget(const FNavigationToolViewModelPtr& InItem
	, const TSharedRef<INavigationToolView>& InView
	, const TSharedRef<SNavigationToolTreeRow>& InRow)
{
	if (!InItem.AsModel()->IsA<Sequencer::ILockableExtension>())
	{
		return SNullWidget::NullWidget;
	}

	return SNew(SNavigationToolLock, InItem.ImplicitCast(), InView, InRow);
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
