// Copyright Epic Games, Inc. All Rights Reserved.

#include "Columns/NavigationToolMarkerVisibilityColumn.h"
#include "Extensions/IMarkerVisibilityExtension.h"
#include "INavigationToolView.h"
#include "Items/NavigationToolItem.h"
#include "Widgets/Columns/SNavigationToolMarkerVisibility.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SNavigationToolTreeRow.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Views/SHeaderRow.h"

#define LOCTEXT_NAMESPACE "NavigationToolMarkerVisibilityColumn"

namespace UE::SequenceNavigator
{

UE_SEQUENCER_DEFINE_CASTABLE(FNavigationToolMarkerVisibilityColumn)

FText FNavigationToolMarkerVisibilityColumn::GetColumnDisplayNameText() const
{
	return LOCTEXT("NavigationToolMarkerVisibilityColumn", "Marker Visibility");
}

const FSlateBrush* FNavigationToolMarkerVisibilityColumn::GetIconBrush() const
{
	return FAppStyle::GetBrush(TEXT("AnimTimeline.SectionMarker"));
}

SHeaderRow::FColumn::FArguments FNavigationToolMarkerVisibilityColumn::ConstructHeaderRowColumn(const TSharedRef<INavigationToolView>& InView
	, const float InFillSize)
{
	const FName ColumnId = GetColumnId();

	return SHeaderRow::Column(ColumnId)
		.FixedWidth(24.f)
		.HAlignHeader(HAlign_Center)
		.VAlignHeader(VAlign_Center)
		.HAlignCell(HAlign_Center)
		.VAlignCell(VAlign_Center)
		.DefaultLabel(GetColumnDisplayNameText())
		.DefaultTooltip(GetColumnDisplayNameText())
		[
			SNew(SImage)
			.ColorAndOpacity(FSlateColor::UseForeground())
			.Image(FAppStyle::GetBrush(TEXT("AnimTimeline.SectionMarker")))
		];
}

TSharedRef<SWidget> FNavigationToolMarkerVisibilityColumn::ConstructRowWidget(const FNavigationToolViewModelPtr& InItem
	, const TSharedRef<INavigationToolView>& InView
	, const TSharedRef<SNavigationToolTreeRow>& InRow)
{
	if (!InItem.AsModel()->IsA<IMarkerVisibilityExtension>())
	{
		return SNullWidget::NullWidget;
	}

	return SNew(SNavigationToolMarkerVisibility, SharedThis(this), InItem.ImplicitCast(), InView, InRow);
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
