// Copyright Epic Games, Inc. All Rights Reserved.

#include "Columns/NavigationToolOutTimeColumn.h"
#include "Extensions/IOutTimeExtension.h"
#include "INavigationToolView.h"
#include "Items/NavigationToolItem.h"
#include "Widgets/Columns/SNavigationToolOutTime.h"
#include "Widgets/SNavigationToolTreeRow.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Views/SHeaderRow.h"

#define LOCTEXT_NAMESPACE "NavigationToolOutTimeColumn"

namespace UE::SequenceNavigator
{

UE_SEQUENCER_DEFINE_CASTABLE(FNavigationToolOutTimeColumn)

FText FNavigationToolOutTimeColumn::GetColumnDisplayNameText() const
{
	return LOCTEXT("OutTimeColumn", "Out");
}

const FSlateBrush* FNavigationToolOutTimeColumn::GetIconBrush() const
{
	return FAppStyle::GetBrush(TEXT("Icons.Alignment.Right"));
}

SHeaderRow::FColumn::FArguments FNavigationToolOutTimeColumn::ConstructHeaderRowColumn(const TSharedRef<INavigationToolView>& InView
	, const float InFillSize)
{
	const FName ColumnId = GetColumnId();

	return SHeaderRow::Column(ColumnId)
		.FillWidth(InFillSize)
		.HAlignHeader(HAlign_Center)
		.VAlignHeader(VAlign_Center)
		.HAlignCell(HAlign_Fill)
		.VAlignCell(VAlign_Center)
		.DefaultLabel(GetColumnDisplayNameText())
		.OnGetMenuContent(InView, &INavigationToolView::GetColumnMenuContent, ColumnId);
}

TSharedRef<SWidget> FNavigationToolOutTimeColumn::ConstructRowWidget(const FNavigationToolViewModelPtr& InItem
	, const TSharedRef<INavigationToolView>& InView
	, const TSharedRef<SNavigationToolTreeRow>& InRow)
{
	if (!InItem.AsModel()->IsA<IOutTimeExtension>())
	{
		return SNullWidget::NullWidget;
	}

	return SNew(SNavigationToolOutTime, InItem.ImplicitCast(), InView, InRow);
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
