// Copyright Epic Games, Inc. All Rights Reserved.

#include "Columns/NavigationToolPlayheadColumn.h"
#include "Extensions/IPlayheadExtension.h"
#include "INavigationToolView.h"
#include "Items/NavigationToolItem.h"
#include "Widgets/Columns/SNavigationToolPlayhead.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SNavigationToolTreeRow.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Views/SHeaderRow.h"

#define LOCTEXT_NAMESPACE "NavigationToolPlayheadColumn"

namespace UE::SequenceNavigator
{

UE_SEQUENCER_DEFINE_CASTABLE(FNavigationToolPlayheadColumn)

FText FNavigationToolPlayheadColumn::GetColumnDisplayNameText() const
{
	return LOCTEXT("NavigationToolPlayheadColumn", "Playhead");
}

const FSlateBrush* FNavigationToolPlayheadColumn::GetIconBrush() const
{
	return FAppStyle::GetBrush(TEXT("GenericPlay"));
}

SHeaderRow::FColumn::FArguments FNavigationToolPlayheadColumn::ConstructHeaderRowColumn(const TSharedRef<INavigationToolView>& InView
	, const float InFillSize)
{
	const FName ColumnId = GetColumnId();

	return SHeaderRow::Column(ColumnId)
		.FixedWidth(24.f)
		.HAlignHeader(HAlign_Center)
		.VAlignHeader(VAlign_Center)
		.HAlignCell(HAlign_Center)
		.VAlignCell(VAlign_Center)
		.DefaultLabel(GetColumnDisplayNameText())
		.DefaultTooltip(GetColumnDisplayNameText())
		[
			SNew(SImage)
			.ColorAndOpacity(FSlateColor::UseForeground())
			.Image(FAppStyle::GetBrush(TEXT("GenericPlay")))
		];
}

TSharedRef<SWidget> FNavigationToolPlayheadColumn::ConstructRowWidget(const FNavigationToolViewModelPtr& InItem
	, const TSharedRef<INavigationToolView>& InView
	, const TSharedRef<SNavigationToolTreeRow>& InRow)
{
	if (!InItem.AsModel()->IsA<IPlayheadExtension>())
	{
		return SNullWidget::NullWidget;
	}

	return SNew(SNavigationToolPlayhead, SharedThis(this), InItem.ImplicitCast(), InView, InRow);
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
