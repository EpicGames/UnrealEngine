// Copyright Epic Games, Inc. All Rights Reserved.

#include "Columns/NavigationToolRevisionControlColumn.h"
#include "Extensions/IRevisionControlExtension.h"
#include "INavigationToolView.h"
#include "RevisionControlStyle/RevisionControlStyle.h"
#include "Widgets/Columns/SNavigationToolRevisionControl.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SNavigationToolTreeRow.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Views/SHeaderRow.h"

#define LOCTEXT_NAMESPACE "NavigationToolRevisionControlColumn"

namespace UE::SequenceNavigator
{

UE_SEQUENCER_DEFINE_CASTABLE(FNavigationToolRevisionControlColumn)

FText FNavigationToolRevisionControlColumn::GetColumnDisplayNameText() const
{
	return LOCTEXT("RevisionControl", "Revision Control");
}

const FSlateBrush* FNavigationToolRevisionControlColumn::GetIconBrush() const
{
	return FRevisionControlStyleManager::Get().GetBrush(TEXT("RevisionControl.Icon"));
}

SHeaderRow::FColumn::FArguments FNavigationToolRevisionControlColumn::ConstructHeaderRowColumn(const TSharedRef<INavigationToolView>& InView
	, const float InFillSize)
{
	const FName ColumnId = GetColumnId();

	return SHeaderRow::Column(ColumnId)
		.FixedWidth(24.f)
		.HAlignHeader(HAlign_Center)
		.VAlignHeader(VAlign_Center)
		.HAlignCell(HAlign_Center)
		.VAlignCell(VAlign_Center)
		.DefaultLabel(GetColumnDisplayNameText())
		.DefaultTooltip(GetColumnDisplayNameText())
		.OnGetMenuContent(InView, &INavigationToolView::GetColumnMenuContent, ColumnId)
		[
			SNew(SImage)
			.ColorAndOpacity(FSlateColor::UseForeground())
			.Image(this, &FNavigationToolRevisionControlColumn::GetIconBrush)
		];
}

TSharedRef<SWidget> FNavigationToolRevisionControlColumn::ConstructRowWidget(const FNavigationToolViewModelPtr& InItem
	, const TSharedRef<INavigationToolView>& InView
	, const TSharedRef<SNavigationToolTreeRow>& InRow)
{
	if (!InItem.AsModel()->IsA<IRevisionControlExtension>())
	{
		return SNullWidget::NullWidget;
	}

	return SNew(SNavigationToolRevisionControl, SharedThis(this), InItem.ImplicitCast(), InView, InRow);
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
