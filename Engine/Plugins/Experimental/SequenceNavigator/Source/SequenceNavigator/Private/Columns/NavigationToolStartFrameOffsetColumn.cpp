// Copyright Epic Games, Inc. All Rights Reserved.

#include "Columns/NavigationToolStartFrameOffsetColumn.h"
#include "INavigationToolView.h"
#include "Widgets/Columns/SNavigationToolStartFrameOffset.h"
#include "Widgets/SNavigationToolTreeRow.h"
#include "Widgets/Views/SHeaderRow.h"

#define LOCTEXT_NAMESPACE "NavigationToolStartFrameOffsetColumn"

namespace UE::SequenceNavigator
{

UE_SEQUENCER_DEFINE_CASTABLE(FNavigationToolStartFrameOffsetColumn)

FText FNavigationToolStartFrameOffsetColumn::GetColumnDisplayNameText() const
{
	return LOCTEXT("StartFrameOffset", "Start Frame Offset");
}

const FSlateBrush* FNavigationToolStartFrameOffsetColumn::GetIconBrush() const
{
	return FAppStyle::GetBrush(TEXT("ClassIcon.TimelineComponent"));
}

SHeaderRow::FColumn::FArguments FNavigationToolStartFrameOffsetColumn::ConstructHeaderRowColumn(const TSharedRef<INavigationToolView>& InView
	, const float InFillSize)
{
	const FName ColumnId = GetColumnId();

	return SHeaderRow::Column(ColumnId)
		.FillWidth(InFillSize)
		.HAlignHeader(HAlign_Center)
		.VAlignHeader(VAlign_Center)
		.HAlignCell(HAlign_Fill)
		.VAlignCell(VAlign_Center)
		.DefaultLabel(GetColumnDisplayNameText())
		.OnGetMenuContent(InView, &INavigationToolView::GetColumnMenuContent, ColumnId);
}

TSharedRef<SWidget> FNavigationToolStartFrameOffsetColumn::ConstructRowWidget(const FNavigationToolViewModelPtr& InItem
	, const TSharedRef<INavigationToolView>& InView
	, const TSharedRef<SNavigationToolTreeRow>& InRow)
{
	return SNew(SNavigationToolStartFrameOffset, InItem.ImplicitCast(), InView, InRow);
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
