// Copyright Epic Games, Inc. All Rights Reserved.

#include "Columns/NavigationToolTakeColumn.h"
#include "Items/NavigationToolSequence.h"
#include "INavigationToolView.h"
#include "Widgets/Columns/SNavigationToolTake.h"
#include "Widgets/SNavigationToolTreeRow.h"
#include "Widgets/Views/SHeaderRow.h"

#define LOCTEXT_NAMESPACE "NavigationToolTakeColumn"

namespace UE::SequenceNavigator
{

UE_SEQUENCER_DEFINE_CASTABLE(FNavigationToolTakeColumn)

FText FNavigationToolTakeColumn::GetColumnDisplayNameText() const
{
	return LOCTEXT("TakeColumn", "Take");
}

const FSlateBrush* FNavigationToolTakeColumn::GetIconBrush() const
{
	return FAppStyle::GetBrush(TEXT("ClassIcon.LevelSequence"));
}

SHeaderRow::FColumn::FArguments FNavigationToolTakeColumn::ConstructHeaderRowColumn(const TSharedRef<INavigationToolView>& InView
	, const float InFillSize)
{
	const FName ColumnId = GetColumnId();

	return SHeaderRow::Column(ColumnId)
		.FillWidth(InFillSize)
		.HAlignHeader(HAlign_Center)
		.VAlignHeader(VAlign_Center)
		.HAlignCell(HAlign_Fill)
		.VAlignCell(VAlign_Center)
		.DefaultLabel(GetColumnDisplayNameText())
		.OnGetMenuContent(InView, &INavigationToolView::GetColumnMenuContent, ColumnId);
}

TSharedRef<SWidget> FNavigationToolTakeColumn::ConstructRowWidget(const FNavigationToolViewModelPtr& InItem
	, const TSharedRef<INavigationToolView>& InView
	, const TSharedRef<SNavigationToolTreeRow>& InRow)
{
	return SNew(SNavigationToolTake, InItem.ImplicitCast(), InView, InRow);
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
