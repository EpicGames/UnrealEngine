// Copyright Epic Games, Inc. All Rights Reserved.

#include "DragDropOps/Handlers/NavigationToolItemDropHandler.h"
#include "DragDropOps/NavigationToolItemDragDropOp.h"

namespace UE::SequenceNavigator
{

using namespace Sequencer;

UE_SEQUENCER_DEFINE_CASTABLE(FNavigationToolItemDropHandler)

void FNavigationToolItemDropHandler::Initialize(const FNavigationToolItemDragDropOp& InDragDropOp)
{
	WeakItems = InDragDropOp.GetItems();
	ActionType = InDragDropOp.GetActionType();

	// Remove all items that are invalid or out of the scope of this handler
	WeakItems.RemoveAll([this](const FNavigationToolViewModelWeakPtr& InWeakItem)
		{
			const FNavigationToolViewModelPtr Item = InWeakItem.Pin();
			return !Item.IsValid() || !IsDraggedItemSupported(Item);
		});
}

} // namespace UE::SequenceNavigator
