// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavigationToolSequenceDropHandler.h"
#include "Framework/Application/SlateApplication.h"
#include "ItemActions/NavigationToolAddItem.h"
#include "Items/NavigationToolSequence.h"
#include "NavigationTool.h"
#include "Widgets/Views/STableRow.h"

namespace UE::SequenceNavigator
{

using namespace Sequencer;

UE_SEQUENCER_DEFINE_CASTABLE(FNavigationToolSequenceDropHandler)

bool FNavigationToolSequenceDropHandler::IsDraggedItemSupported(const FNavigationToolViewModelPtr& InDraggedItem) const
{
	return InDraggedItem.AsModel()->IsA<FNavigationToolSequence>();
}

TOptional<EItemDropZone> FNavigationToolSequenceDropHandler::CanDrop(EItemDropZone InDropZone, const FNavigationToolViewModelPtr& InTargetItem) const
{
	/*switch (ActionType)
	{
		case ENavigationToolDragDropActionType::Move:
			// When moving, make sure the Destination is not one of the Items we're moving
			if (!Items.Contains(InTargetItem))
			{
				return InDropZone;
			}
			break;

		case ENavigationToolDragDropActionType::Copy:
			return InDropZone;
	}*/

	return TOptional<EItemDropZone>();
}

bool FNavigationToolSequenceDropHandler::Drop(EItemDropZone InDropZone, const FNavigationToolViewModelPtr& InTargetItem)
{
	// Disabling drag and drop for now
	switch (ActionType)
	{
	case ENavigationToolDragDropActionType::Move:
		//MoveItems(InDropZone, InTargetItem);
		break;

	case ENavigationToolDragDropActionType::Copy:
		//StaticCastSharedRef<FNavigationTool>(InTargetItem->GetOwnerTool())->DuplicateItems(Items, InTargetItem, InDropZone);
		break;

	default:
		return false;
	}

	return false;
}

void FNavigationToolSequenceDropHandler::MoveItems(EItemDropZone InDropZone, const FNavigationToolViewModelPtr& InTargetItem)
{
	// Can't access this mouse event at this point, so use the Slate Application to check modifiers.
	/*const FAttachmentTransformRules& TransformRules = InTargetItem->GetOwnerTool()->GetProvider().GetTransformRule(!FSlateApplication::Get().GetModifierKeys().IsShiftDown());
	
	FNavigationToolAddItemParams Params;
	Params.RelativeItem             = InTargetItem;
	Params.RelativeDropZone         = InDropZone;
	Params.Flags                    = ENavigationToolAddItemFlags::Select | ENavigationToolAddItemFlags::Transact;
	Params.AttachmentTransformRules = TransformRules;
	Params.SelectionFlags           = ENavigationToolItemSelectionFlags::AppendToCurrentSelection;

	const TSet<FNavigationToolViewModelPtr> DraggedItemSet(Items);

	// Remove all Items whose Parent is in the Item Set 
	Items.RemoveAll([&DraggedItemSet](const FNavigationToolViewModelPtr& InItem)
		{
			return !InItem.IsValid() || !InItem->GetParent().IsValid() || DraggedItemSet.Contains(InItem->GetParent());
		});

	// Reverse order for Onto since Item->AddChild(...) adds it to Index 0, so last item would be at the top, which is reversed
	if (InDropZone == EItemDropZone::OntoItem)
	{
		Algo::Reverse(Items);
	}

	const TSharedRef<FNavigationTool> Tool = StaticCastSharedRef<FNavigationTool>(InTargetItem->GetOwnerTool());

	for (const FNavigationToolViewModelPtr& Item : Items)
	{
		Params.Item = Item;
		Tool->EnqueueItemAction<FNavigationToolAddItem>(Params);
	}*/
}

} // namespace UE::SequenceNavigator
