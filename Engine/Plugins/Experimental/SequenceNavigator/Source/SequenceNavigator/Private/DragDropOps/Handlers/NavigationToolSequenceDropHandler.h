// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DragDropOps/Handlers/NavigationToolItemDropHandler.h"

#define UE_API SEQUENCENAVIGATOR_API

namespace UE::SequenceNavigator
{

/** Class that handles Dropping Sequence Items into a Target Item */
class FNavigationToolSequenceDropHandler
	: public FNavigationToolItemDropHandler
{
public:
	UE_SEQUENCER_DECLARE_CASTABLE_API(UE_API, FNavigationToolSequenceDropHandler
		, FNavigationToolItemDropHandler)

protected:
	//~ Begin FNavigationToolItemDropHandler
	virtual bool IsDraggedItemSupported(const FNavigationToolViewModelPtr& InDraggedItem) const override;
	virtual TOptional<EItemDropZone> CanDrop(EItemDropZone InDropZone, const FNavigationToolViewModelPtr& InTargetItem) const override;
	virtual bool Drop(EItemDropZone InDropZone, const FNavigationToolViewModelPtr& InTargetItem) override;
	//~ End FNavigationToolItemDropHandler

	void MoveItems(EItemDropZone InDropZone, const FNavigationToolViewModelPtr& InTargetItem);
};

} // namespace UE::SequenceNavigator

#undef UE_API
