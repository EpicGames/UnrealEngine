// Copyright Epic Games, Inc. All Rights Reserved.

#include "DragDropOps/NavigationToolItemDragDropOp.h"
#include "DragDropOps/Handlers/NavigationToolSequenceDropHandler.h"
#include "DragDropOps/Handlers/NavigationToolItemDropHandler.h"
#include "GameFramework/Actor.h"
#include "INavigationToolView.h"
#include "Items/INavigationToolItem.h"
#include "LevelSequenceActor.h"
#include "NavigationToolExtender.h"
#include "NavigationToolView.h"
#include "ScopedTransaction.h"
#include "Styling/SlateIconFinder.h"

#define LOCTEXT_NAMESPACE "NavigationToolItemDragDropOp"

namespace UE::SequenceNavigator
{

FText GetActionName(const ENavigationToolDragDropActionType InActionType)
{
	switch (InActionType)
	{
		case ENavigationToolDragDropActionType::Move:
			return LOCTEXT("MoveAction", "Moving");

		case ENavigationToolDragDropActionType::Copy:
			return LOCTEXT("CopyAction", "Copying");
	}
	return FText::GetEmpty();
}

FText GetItemName(const TArray<FNavigationToolViewModelWeakPtr>& InWeakItems)
{
	using namespace Sequencer;

	if (InWeakItems.IsEmpty())
	{
		return LOCTEXT("NoItems", "0 Items");
	}

	const FNavigationToolViewModelPtr FirstItem = InWeakItems[0].Pin();

	if (InWeakItems.Num() == 1)
	{
		return FirstItem->GetDisplayName();
	}

	return FText::Format(LOCTEXT("ManyItems", "{0} and {1} other item(s)")
		, FirstItem->GetDisplayName(), InWeakItems.Num() - 1);
}

TSharedRef<FNavigationToolItemDragDropOp> FNavigationToolItemDragDropOp::New(const TArray<FNavigationToolViewModelWeakPtr>& InWeakItems
	, const TSharedPtr<FNavigationToolView>& InToolView
	, const ENavigationToolDragDropActionType InActionType)
{
	const TSharedRef<FNavigationToolItemDragDropOp> DragDropOp = MakeShared<FNavigationToolItemDragDropOp>();
	DragDropOp->Init(InWeakItems, InToolView, InActionType);
	return DragDropOp;
}

FReply FNavigationToolItemDragDropOp::Drop(EItemDropZone InDropZone, const FNavigationToolViewModelPtr& InTargetItem)
{
	FReply Reply = FReply::Unhandled();

	FScopedTransaction Transaction(LOCTEXT("DropItems", "Sequence Navigator Drop Items"));

	for (const TSharedRef<FNavigationToolItemDropHandler>& DropHandler : DropHandlers)
	{
		if (DropHandler->GetItems().IsEmpty())
		{
			continue;
		}

		if (DropHandler->Drop(InDropZone, InTargetItem))
		{
			Reply = FReply::Handled();
		}
	}

	if (!Reply.IsEventHandled())
	{
		Transaction.Cancel();
	}

	return Reply;
}

TOptional<EItemDropZone> FNavigationToolItemDragDropOp::CanDrop(const EItemDropZone InDropZone, const FNavigationToolViewModelPtr& InTargetItem) const
{
	const TSharedPtr<INavigationToolView> ToolView = GetToolView();

	// Only support Drag/Drop from Same Navigation Tool
	if (!ToolView.IsValid() || &InTargetItem->GetOwnerTool() != ToolView->GetOwnerTool().Get())
	{
		return TOptional<EItemDropZone>();
	}

	for (const TSharedRef<FNavigationToolItemDropHandler>& DropHandler : DropHandlers)
	{
		if (DropHandler->GetItems().IsEmpty())
		{
			continue;
		}

		// Return the Item Drop Zone of the first Drop Handler that supports the Drop Zone and Target Item
		const TOptional<EItemDropZone> DropZone = DropHandler->CanDrop(InDropZone, InTargetItem);
		if (DropZone.IsSet())
		{
			return *DropZone;
		}
	}

	return TOptional<EItemDropZone>();
}

void FNavigationToolItemDragDropOp::Init(const TArray<FNavigationToolViewModelWeakPtr>& InWeakItems
	, const TSharedPtr<FNavigationToolView>& InToolView
	, const ENavigationToolDragDropActionType InActionType)
{
	WeakItems = InWeakItems;
	WeakToolView = InToolView;
	ActionType = InActionType;
	MouseCursor = EMouseCursor::GrabHandClosed;

	CurrentIconBrush = WeakItems.Num() == 1
		? WeakItems[0].Pin()->GetIconBrush()
		: FSlateIconFinder::FindIconForClass(ALevelSequenceActor::StaticClass()).GetIcon();

	CurrentHoverText = FText::Format(LOCTEXT("HoverText", "{0} {1}")
		, GetActionName(ActionType)
		, GetItemName(WeakItems));

	CurrentIconColorAndOpacity = FSlateColor::UseForeground();

	SetupDefaults();
	Construct();

	// Add default drop handlers 
	AddDropHandler<FNavigationToolSequenceDropHandler>();

	FNavigationToolExtender::OnItemDragDropOpInitialized().Broadcast(*this);
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
