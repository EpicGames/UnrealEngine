// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/Filters/NavigationToolBuiltInFilter.h"
#include "Items/INavigationToolItem.h"
#include "Items/NavigationToolSequence.h"

namespace UE::SequenceNavigator
{

using namespace Sequencer;

FNavigationToolBuiltInFilter::FNavigationToolBuiltInFilter(const FNavigationToolBuiltInFilterParams& InFilterParams)
	: FFilterBase<FNavigationToolViewModelPtr>(nullptr)
	, bIsActive(true)
	, FilterParams(InFilterParams)
{
}

FText FNavigationToolBuiltInFilter::GetDisplayName() const
{ 
	return FilterParams.GetDisplayName();
}

FText FNavigationToolBuiltInFilter::GetToolTipText() const
{
	return FilterParams.GetTooltipText();
}

FLinearColor FNavigationToolBuiltInFilter::GetColor() const
{
	return FLinearColor::White;
}

FName FNavigationToolBuiltInFilter::GetIconName() const
{
	if (const FSlateBrush* const IconBrush = FilterParams.GetIconBrush())
	{
		return IconBrush->GetResourceName();
	}
	return NAME_None;
}

FString FNavigationToolBuiltInFilter::GetName() const
{
	return FilterParams.GetFilterId().ToString();
}

FSlateIcon FNavigationToolBuiltInFilter::GetIcon() const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), GetIconName());
}

TSharedPtr<FUICommandInfo> FNavigationToolBuiltInFilter::GetToggleCommand() const
{
	return FilterParams.GetToggleCommand();
}

bool FNavigationToolBuiltInFilter::PassesFilter(const FNavigationToolViewModelPtr InItem) const
{
	auto ItemObjectPassesFilter = [this](const FNavigationToolViewModelPtr& InItem)
		{
			UObject* const ItemObject = InItem->GetItemObject();
			return ItemObject && FilterParams.IsValidObjectClass(ItemObject->GetClass());
		};

	auto ItemTypePassesFilter = [this](const FNavigationToolViewModelPtr& InItem)
		{
			return FilterParams.IsValidItemClass(InItem->ID);
		};

	const ENavigationToolFilterMode FilterMode = FilterParams.GetFilterMode();

	if (EnumHasAnyFlags(FilterMode, ENavigationToolFilterMode::MatchesType))
	{
		if (ItemObjectPassesFilter(InItem) || ItemTypePassesFilter(InItem))
		{
			return true;
		}
	}

	if (EnumHasAnyFlags(FilterMode, ENavigationToolFilterMode::ContainerOfType))
	{
		TArray<FNavigationToolViewModelWeakPtr> WeakRemainingItems = InItem->GetChildren();

		while (!WeakRemainingItems.IsEmpty())
		{
			const FNavigationToolViewModelPtr Item = WeakRemainingItems.Pop().Pin();

			// Stop here if Item is invalid or if it's a Sequence Item as any children of this item will be considered contained by the item itself, not the querying item
			// this could be improved if the Item Type filter as a whole ever moves past only checking UObject types, and moving past that Sequences are the top level items
			if (!Item.IsValid() || Item.AsModel()->IsA<FNavigationToolSequence>())
			{
				continue;
			}

			if (ItemObjectPassesFilter(Item) || ItemTypePassesFilter(Item))
			{
				return true;
			}

			WeakRemainingItems.Append(Item->GetChildren());
		}
	}

	return false;
}

bool FNavigationToolBuiltInFilter::IsActive() const
{
	return bIsActive;
}

void FNavigationToolBuiltInFilter::SetActive(const bool bInActive)
{
	FFilterBase<FNavigationToolViewModelPtr>::SetActive(bInActive);
	
	bIsActive = bInActive;
}

const FNavigationToolBuiltInFilterParams& FNavigationToolBuiltInFilter::GetFilterParams() const
{
	return FilterParams;
}

} // namespace UE::SequenceNavigator
