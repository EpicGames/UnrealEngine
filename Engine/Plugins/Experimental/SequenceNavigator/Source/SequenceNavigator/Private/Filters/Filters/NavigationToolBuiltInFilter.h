// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/Filters/NavigationToolBuiltInFilterParams.h"
#include "Filters/NavigationToolFilterData.h"
#include "NavigationToolDefines.h"

namespace UE::SequenceNavigator
{

class FNavigationToolBuiltInFilter : public FFilterBase<FNavigationToolViewModelPtr>
{
public:
	FNavigationToolBuiltInFilter(const FNavigationToolBuiltInFilterParams& InFilterParams);

	//~ Begin FFilterBase
	virtual FText GetDisplayName() const override;
	virtual FText GetToolTipText() const override;
	virtual FLinearColor GetColor() const override;
	virtual FName GetIconName() const override;
	virtual bool IsInverseFilter() const override { return false; }
	//~ End FFilterBase

	//~ Begin IFilter
	virtual FString GetName() const override;
	virtual bool PassesFilter(const FNavigationToolViewModelPtr InItem) const override;
	//~ End IFilter

	virtual FSlateIcon GetIcon() const;

	virtual TSharedPtr<FUICommandInfo> GetToggleCommand() const;

	virtual bool IsActive() const;
	virtual void SetActive(const bool bInActive);

	const FNavigationToolBuiltInFilterParams& GetFilterParams() const;

protected:
	virtual void ActiveStateChanged(const bool bInActive) override {}
	virtual void ModifyContextMenu(FMenuBuilder& InMenuBuilder) override {}

	virtual void SaveSettings(const FString& InIniFilename, const FString& InIniSection, const FString& InSettingsString) const override {}
	virtual void LoadSettings(const FString& InIniFilename, const FString& InIniSection, const FString& InSettingsString) override {}

	/** Active global filters are not checked in the menu and will hide the item if it does not pass the filter */
	bool bIsActive = false;

	FNavigationToolBuiltInFilterParams FilterParams;
};

} // namespace UE::SequenceNavigator
