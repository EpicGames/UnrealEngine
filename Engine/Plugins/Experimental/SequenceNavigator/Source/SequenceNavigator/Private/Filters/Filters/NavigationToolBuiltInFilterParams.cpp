// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/Filters/NavigationToolBuiltInFilterParams.h"
#include "Filters/NavigationToolFilterCommands.h"
#include "Filters/Filters/NavigationToolFilter_Text.h"
#include "Items/NavigationToolActor.h"
#include "Items/NavigationToolBinding.h"
#include "Items/NavigationToolComponent.h"
#include "Items/NavigationToolMarker.h"
#include "Items/NavigationToolSequence.h"
#include "Items/NavigationToolTrack.h"
#include "LevelSequence.h"
#include "LevelSequenceActor.h"
#include "MVVM/ViewModelTypeID.h"
#include "Styling/SlateIconFinder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NavigationToolBuiltInFilterParams)

#define LOCTEXT_NAMESPACE "NavigationToolBuiltInFilterParams"

FNavigationToolBuiltInFilterParams::FNavigationToolBuiltInFilterParams(const FName InFilterId
	, const TSet<UE::Sequencer::FViewModelTypeID>& InItemClasses
	, const TArray<TSubclassOf<UObject>>& InFilterClasses
	, const ENavigationToolFilterMode InFilterMode
	, const FSlateBrush* const InIconBrush
	, const FText& InDisplayName
	, const FText& InTooltipText
	, const TSharedPtr<FUICommandInfo>& InToggleCommand
	, const bool InEnabledByDefault
	, const EClassFlags InRequiredClassFlags
	, const EClassFlags InRestrictedClassFlags)
	: FilterId(InFilterId)
	, ItemClasses(InItemClasses)
	, ObjectClasses(InFilterClasses)
	, FilterMode(InFilterMode)
	, DisplayName(InDisplayName)
	, TooltipText(InTooltipText)
	, bUseOverrideIcon(false)
	, bEnabledByDefault(InEnabledByDefault)
	, ToggleCommand(InToggleCommand)
	, IconBrush(InIconBrush)
	, RequiredClassFlags(InRequiredClassFlags)
	, RestrictedClassFlags(InRestrictedClassFlags)
{
}

bool FNavigationToolBuiltInFilterParams::HasValidFilterData() const
{
	return !ObjectClasses.IsEmpty() || !FilterText.IsEmptyOrWhitespace();
}

FName FNavigationToolBuiltInFilterParams::GetFilterId() const
{
	return FilterId;
}

FText FNavigationToolBuiltInFilterParams::GetDisplayName() const
{
	return DisplayName;
}

FText FNavigationToolBuiltInFilterParams::GetTooltipText() const
{
	if (!TooltipText.IsEmpty())
	{
		return TooltipText;
	}

	FString SupportedClasses;

	for (TSubclassOf<UObject> FilterClass : ObjectClasses)
	{
		if (FilterClass)
		{
			if (!SupportedClasses.IsEmpty())
			{
				SupportedClasses += TEXT(", ");
			}
			SupportedClasses += FilterClass->GetDisplayNameText().ToString();
		}
	}

	const_cast<FNavigationToolBuiltInFilterParams*>(this)->TooltipText = FText::FromString(MoveTemp(SupportedClasses));
	return TooltipText;
}

const FSlateBrush* FNavigationToolBuiltInFilterParams::GetIconBrush() const
{
	if (bUseOverrideIcon)
	{
		return &OverrideIcon;
	}

	if (IconBrush)
	{
		return IconBrush;
	}

	for (const TSubclassOf<UObject>& FilterClass : ObjectClasses)
	{
		if (FilterClass)
		{
			const_cast<FNavigationToolBuiltInFilterParams*>(this)->IconBrush = FSlateIconFinder::FindIconForClass(FilterClass).GetIcon();
			return IconBrush;
		}
	}

	return nullptr;
}

ENavigationToolFilterMode FNavigationToolBuiltInFilterParams::GetFilterMode() const
{
	return FilterMode;
}

bool FNavigationToolBuiltInFilterParams::IsEnabledByDefault() const
{
	return bEnabledByDefault;
}

TSharedPtr<FUICommandInfo> FNavigationToolBuiltInFilterParams::GetToggleCommand() const
{
	return ToggleCommand;
}

void FNavigationToolBuiltInFilterParams::SetOverrideIconColor(const FSlateColor& InNewIconColor)
{
	OverrideIcon.TintColor = InNewIconColor;
}

bool FNavigationToolBuiltInFilterParams::IsValidItemClass(const UE::Sequencer::FViewModelTypeID& InClassTypeId) const
{
	return ItemClasses.IsEmpty() || ItemClasses.Contains(InClassTypeId);
}

bool FNavigationToolBuiltInFilterParams::IsValidObjectClass(const UClass* const InClass) const
{
	if (!InClass)
	{
		return false;
	}

	if (InClass->HasAllClassFlags(RequiredClassFlags) && !InClass->HasAnyClassFlags(RestrictedClassFlags))
	{
		for (const TSubclassOf<UObject>& FilterClass : ObjectClasses)
		{
			if (InClass->IsChildOf(FilterClass))
			{
				return true;
			}
		}
	}

	return false;
}

void FNavigationToolBuiltInFilterParams::SetFilterText(const FText& InText)
{
	FilterText = InText;
}

bool FNavigationToolBuiltInFilterParams::PassesFilterText(const UE::SequenceNavigator::FNavigationToolViewModelPtr& InItem) const
{
	// Since it needs to be a const function we have to create the TextFilter here instead of being a variable
	if (FilterText.IsEmptyOrWhitespace())
	{
		return false;
	}

	/*FNavigationToolFilter_Text TextFilterTest;
	TextFilterTest.SetFilterText(FilterText);
	return TextFilterTest.PassesFilter(InItem);*/

	return true;
}

FNavigationToolBuiltInFilterParams FNavigationToolBuiltInFilterParams::CreateSequenceFilter()
{
	return FNavigationToolBuiltInFilterParams(TEXT("Level Sequence")
		, { UE::SequenceNavigator::FNavigationToolSequence::ID }
		, { ULevelSequence::StaticClass(), UMovieSceneSequence::StaticClass() }
		, ENavigationToolFilterMode::MatchesType
		, FSlateIconFinder::FindIconForClass(ALevelSequenceActor::StaticClass()).GetIcon()
		, LOCTEXT("LevelSequenceFilterDisplayName", "Level Sequence")
		, LOCTEXT("LevelSequenceFilterTooltip", "Level Sequence")
		, FNavigationToolFilterCommands::Get().ToggleFilter_Sequence);
}

FNavigationToolBuiltInFilterParams FNavigationToolBuiltInFilterParams::CreateTrackFilter()
{
	return FNavigationToolBuiltInFilterParams(TEXT("Track")
		, { UE::SequenceNavigator::FNavigationToolTrack::ID }
		, { UMovieSceneTrack::StaticClass() }
		, ENavigationToolFilterMode::MatchesType
		, FSlateIconFinder::FindIconForClass(UMovieSceneTrack::StaticClass()).GetIcon()
		, LOCTEXT("TrackFilterDisplayName", "Track")
		, LOCTEXT("TrackFilterTooltip", "Track")
		, FNavigationToolFilterCommands::Get().ToggleFilter_Track);
}

FNavigationToolBuiltInFilterParams FNavigationToolBuiltInFilterParams::CreateBindingFilter()
{
	return FNavigationToolBuiltInFilterParams(TEXT("Binding")
		, { UE::SequenceNavigator::FNavigationToolBinding::ID
			, UE::SequenceNavigator::FNavigationToolActor::ID
			, UE::SequenceNavigator::FNavigationToolComponent::ID }
		, { }
		, ENavigationToolFilterMode::MatchesType
		, nullptr
		, LOCTEXT("BindingFilterDisplayName", "Binding")
		, LOCTEXT("BindingFilterTooltip", "Binding")
		, FNavigationToolFilterCommands::Get().ToggleFilter_Binding);
}

FNavigationToolBuiltInFilterParams FNavigationToolBuiltInFilterParams::CreateMarkerFilter()
{
	return FNavigationToolBuiltInFilterParams(TEXT("Marker")
		, { UE::SequenceNavigator::FNavigationToolMarker::ID }
		, { }
		, ENavigationToolFilterMode::MatchesType
		, nullptr
		, LOCTEXT("MarkerFilterDisplayName", "Marker")
		, LOCTEXT("MarkerFilterTooltip", "Marker")
		, FNavigationToolFilterCommands::Get().ToggleFilter_Marker);
}

#undef LOCTEXT_NAMESPACE
