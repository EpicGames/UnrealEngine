// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/Filters/NavigationToolFilterBase.h"
#include "Filters/INavigationToolFilterBar.h"
#include "Filters/ISequencerFilterBar.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NavigationToolFilterBase)

namespace UE::SequenceNavigator
{

using namespace Sequencer;

FNavigationToolFilter::FNavigationToolFilter(INavigationToolFilterBar& InOutFilterInterface, TSharedPtr<FFilterCategory>&& InCategory)
	: FSequencerFilterBase<FNavigationToolViewModelPtr>(InOutFilterInterface, MoveTemp(InCategory))
{
}

INavigationToolFilterBar& FNavigationToolFilter::GetFilterInterface() const
{
	return static_cast<INavigationToolFilterBar&>(FilterInterface);
}

} // namespace UE::SequenceNavigator
