// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/Filters/NavigationToolFilterCollection.h"
#include "Filters/Filters/NavigationToolFilterBase.h"

namespace UE::SequenceNavigator
{

using namespace Sequencer;

FNavigationToolFilterCollection::FNavigationToolFilterCollection(ISequencerFilterBar& InFilterInterface)
	: FilterInterface(InFilterInterface)
{
}

bool FNavigationToolFilterCollection::ContainsFilter(const TSharedRef<FNavigationToolFilter>& InItem) const
{
	bool bOutContains = false;

	ForEachFilter([&InItem, &bOutContains]
		(const TSharedRef<FNavigationToolFilter>& InFilter) -> bool
		{
			if (InItem == InFilter)
			{
				bOutContains = true;
				return false;
			}
			return true;
		});

	return bOutContains;
}

void FNavigationToolFilterCollection::RemoveAll()
{
	using namespace Sequencer;

	for (const TSharedPtr<IFilter<FNavigationToolViewModelPtr>>& Filter : ChildFilters)
	{
		Filter->OnChanged().RemoveAll(this);
	}

	ChildFilters.Empty();

	ChangedEvent.Broadcast();
}

int32 FNavigationToolFilterCollection::Add(const TSharedRef<FNavigationToolFilter>& InFilter)
{
	int32 ExistingIdx = INDEX_NONE;
	if (ChildFilters.Find(InFilter, ExistingIdx))
	{
		// The filter already exists, don't add a new one but return the index where it was found.
		return ExistingIdx;
	}

	InFilter->OnChanged().AddSP(this, &FNavigationToolFilterCollection::OnChildFilterChanged);

	int32 Result = ChildFilters.Add(InFilter);

	ChangedEvent.Broadcast();

	return Result;
}

int32 FNavigationToolFilterCollection::Remove(const TSharedRef<FNavigationToolFilter>& InFilter)
{
	InFilter->OnChanged().RemoveAll(this);

	int32 Result = ChildFilters.Remove(InFilter);

	// Don't broadcast if the collection didn't change
	if (Result > 0)
	{
		ChangedEvent.Broadcast();
	}

	return Result;
}

TSharedRef<FNavigationToolFilter> FNavigationToolFilterCollection::GetFilterAtIndex(const int32 InIndex)
{
	check(ChildFilters.IsValidIndex(InIndex));
	return StaticCastSharedPtr<FNavigationToolFilter>(ChildFilters[InIndex]).ToSharedRef();
}

int32 FNavigationToolFilterCollection::Num() const
{
	return ChildFilters.Num();
}

bool FNavigationToolFilterCollection::IsEmpty() const
{
	return ChildFilters.IsEmpty();
}

void FNavigationToolFilterCollection::Sort()
{
	ChildFilters.Sort([](const TSharedPtr<IFilter<FNavigationToolViewModelPtr>>& LHS, const TSharedPtr<IFilter<FNavigationToolViewModelPtr>>& RHS)
		{
			const TSharedPtr<FNavigationToolFilter> CastedLHS = StaticCastSharedPtr<FNavigationToolFilter>(LHS);
			const TSharedPtr<FNavigationToolFilter> CastedRHS = StaticCastSharedPtr<FNavigationToolFilter>(RHS);
			if (CastedLHS.IsValid() && CastedRHS.IsValid())
			{
				return CastedLHS->GetDisplayName().ToString() < CastedRHS->GetDisplayName().ToString();
			}
			return true;
		});
}

void FNavigationToolFilterCollection::OnChildFilterChanged()
{
	ChangedEvent.Broadcast();
}

TArray<FText> FNavigationToolFilterCollection::GetFilterDisplayNames() const
{
	TArray<FText> OutDisplayNames;

	ForEachFilter([&OutDisplayNames]
		(const TSharedRef<FNavigationToolFilter>& InFilter) -> bool
		{
			OutDisplayNames.Add(InFilter->GetDisplayName());
			return true;
		});

	return OutDisplayNames;
}

TArray<TSharedRef<FNavigationToolFilter>> FNavigationToolFilterCollection::GetAllFilters(const TArray<TSharedRef<FFilterCategory>>& InCategories) const
{
	TArray<TSharedRef<FNavigationToolFilter>> OutFilters;

	ForEachFilter([&OutFilters]
		(const TSharedRef<FNavigationToolFilter>& InFilter) -> bool
		{
			OutFilters.Add(InFilter);
			return true;
		}
		, InCategories);

	return OutFilters;
}

TSet<TSharedRef<FFilterCategory>> FNavigationToolFilterCollection::GetCategories(const TSet<TSharedRef<FNavigationToolFilter>>* InFilters) const
{
	TSet<TSharedRef<FFilterCategory>> OutCategories;

	ForEachFilter([InFilters, &OutCategories]
		(const TSharedRef<FNavigationToolFilter>& InFilter) -> bool
		{
			if (!InFilters || InFilters->Contains(InFilter))
			{
				if (const TSharedPtr<FFilterCategory> Category = InFilter->GetCategory())
				{
					if (const TSharedPtr<FFilterCategory> CategoryCasted = StaticCastSharedPtr<FFilterCategory>(Category))
					{
						OutCategories.Add(CategoryCasted.ToSharedRef());
					}
				}
			}
			return true;
		});

	return OutCategories;
}

TArray<TSharedRef<FNavigationToolFilter>> FNavigationToolFilterCollection::GetCategoryFilters(const TSharedRef<FFilterCategory>& InCategory) const
{
	TArray<TSharedRef<FNavigationToolFilter>> OutFilters;

	ForEachFilter([&InCategory, &OutFilters]
		(const TSharedRef<FNavigationToolFilter>& InFilter) -> bool
		{
			if (InFilter->GetCategory() == InCategory)
			{
				OutFilters.Add(InFilter);
			}
			return true;
		});

	return OutFilters;
}

void FNavigationToolFilterCollection::ForEachFilter(const TFunctionRef<bool(const TSharedRef<FNavigationToolFilter>&)>& InFunction
	, const TArray<TSharedRef<FFilterCategory>>& InCategories) const
{
	using namespace Sequencer;

	for (const TSharedPtr<IFilter<FNavigationToolViewModelPtr>>& Filter : ChildFilters)
	{
		const TSharedPtr<FNavigationToolFilter> FilterCasted = StaticCastSharedPtr<FNavigationToolFilter>(Filter);
		if (!FilterCasted.IsValid())
		{
			continue;
		}

		if (!InCategories.IsEmpty() && InCategories.Contains(FilterCasted->GetCategory()))
		{
			continue;
		}

		if (!InFunction(FilterCasted.ToSharedRef()))
		{
			return;
		}
	}
}

} // namespace UE::SequenceNavigator
