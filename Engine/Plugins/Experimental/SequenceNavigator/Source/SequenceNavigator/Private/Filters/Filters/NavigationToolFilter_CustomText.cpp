// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/Filters/NavigationToolFilter_CustomText.h"

namespace UE::SequenceNavigator
{

using namespace Sequencer;

FNavigationToolFilter_CustomText::FNavigationToolFilter_CustomText(INavigationToolFilterBar& InFilterInterface)
	: FNavigationToolFilter_Text(InFilterInterface)
{
}

void FNavigationToolFilter_CustomText::SetFromCustomTextFilterData(const FCustomTextFilterData& InFilterData)
{
	Color = InFilterData.FilterColor;
	DisplayName = InFilterData.FilterLabel;
	SetRawFilterText(InFilterData.FilterString);
}

FCustomTextFilterData FNavigationToolFilter_CustomText::CreateCustomTextFilterData() const
{
	FCustomTextFilterData CustomTextFilterData;
	CustomTextFilterData.FilterColor = Color;
	CustomTextFilterData.FilterLabel = DisplayName;
	CustomTextFilterData.FilterString = GetRawFilterText();
	return CustomTextFilterData;
}

TSharedPtr<FFilterBase<FNavigationToolViewModelPtr>> FNavigationToolFilter_CustomText::GetFilter()
{
	return AsShared();
}

bool FNavigationToolFilter_CustomText::IsCustomTextFilter() const
{
	return true;
}

FText FNavigationToolFilter_CustomText::GetDefaultToolTipText() const
{
	return GetDisplayName(); 
}

TSharedPtr<FUICommandInfo> FNavigationToolFilter_CustomText::GetToggleCommand() const
{
	return nullptr;
}

FText FNavigationToolFilter_CustomText::GetDisplayName() const
{
	return DisplayName;
}
FText FNavigationToolFilter_CustomText::GetToolTipText() const
{
	return GetRawFilterText();
}

FLinearColor FNavigationToolFilter_CustomText::GetColor() const
{
	return Color;
}

FString FNavigationToolFilter_CustomText::GetName() const
{
	return FCustomTextFilter<FNavigationToolViewModelPtr>::GetFilterTypeName().ToString();
}

} // namespace UE::SequenceNavigator
