// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/Filters/NavigationToolFilter_Dirty.h"
#include "Filters/NavigationToolFilterCommands.h"
#include "Items/NavigationToolBinding.h"
#include "Items/NavigationToolSequence.h"
#include "Items/NavigationToolTrack.h"

#define LOCTEXT_NAMESPACE "NavigationToolFilter_Dirty"

namespace UE::SequenceNavigator
{

using namespace Sequencer;

FNavigationToolFilter_Dirty::FNavigationToolFilter_Dirty(INavigationToolFilterBar& InFilterInterface, TSharedPtr<FFilterCategory> InCategory)
	: FNavigationToolFilter(InFilterInterface, MoveTemp(InCategory))
{
}

FText FNavigationToolFilter_Dirty::GetDefaultToolTipText() const
{
	return LOCTEXT("Tooltip", "Show only items that are Dirty");
}

TSharedPtr<FUICommandInfo> FNavigationToolFilter_Dirty::GetToggleCommand() const
{
	return FNavigationToolFilterCommands::Get().ToggleFilter_Dirty;
}

FText FNavigationToolFilter_Dirty::GetDisplayName() const
{
	return LOCTEXT("DisplayName", "Dirty");
}

FSlateIcon FNavigationToolFilter_Dirty::GetIcon() const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.DirtyBadge"));
}

FString FNavigationToolFilter_Dirty::GetName() const
{
	return StaticName();
}

bool FNavigationToolFilter_Dirty::PassesFilter(const FNavigationToolViewModelPtr InItem) const
{
	if (const TViewModelPtr<FNavigationToolSequence> SequenceItem = InItem.ImplicitCast())
	{
		const UMovieSceneSequence* const Sequence = SequenceItem->GetSequence();
		return Sequence && IsObjectPackageDirty(Sequence);
	}

	if (const TViewModelPtr<FNavigationToolTrack> TrackItem = InItem.ImplicitCast())
	{
		const UMovieSceneTrack* const Track = TrackItem->GetTrack();
		return Track && IsObjectPackageDirty(Track);
	}

	if (const TViewModelPtr<FNavigationToolBinding> BindingItem = InItem.ImplicitCast())
	{
		const UObject* const BoundObject = BindingItem->GetCachedBoundObject();
		return BoundObject && IsObjectPackageDirty(BoundObject);
	}

	return false;
}

bool FNavigationToolFilter_Dirty::IsObjectPackageDirty(const UObject* const InObject)
{
	if (InObject)
	{
		const UPackage* const Package = InObject->GetPackage();
		return Package && Package->IsDirty();
	}
	return false;
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
