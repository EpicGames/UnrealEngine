// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/Filters/NavigationToolFilter_Marks.h"
#include "Filters/NavigationToolFilterCommands.h"
#include "Items/NavigationToolSequence.h"
#include "MovieScene.h"

#define LOCTEXT_NAMESPACE "NavigationToolFilter_Marks"

namespace UE::SequenceNavigator
{

using namespace Sequencer;

FNavigationToolFilter_Marks::FNavigationToolFilter_Marks(INavigationToolFilterBar& InFilterInterface, TSharedPtr<FFilterCategory> InCategory)
	: FNavigationToolFilter(InFilterInterface, MoveTemp(InCategory))
{
}

FText FNavigationToolFilter_Marks::GetDefaultToolTipText() const
{
	return LOCTEXT("NavigationToolFilter_MarksToolTip", "Show only sequences that contain marked frames");
}

TSharedPtr<FUICommandInfo> FNavigationToolFilter_Marks::GetToggleCommand() const
{
	return FNavigationToolFilterCommands::Get().ToggleFilter_Marks;
}

FText FNavigationToolFilter_Marks::GetDisplayName() const
{
	return LOCTEXT("NavigationToolFilter_Marks", "Marks");
}

FSlateIcon FNavigationToolFilter_Marks::GetIcon() const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("AnimTimeline.SectionMarker"));
}

FString FNavigationToolFilter_Marks::GetName() const
{
	return StaticName();
}

bool FNavigationToolFilter_Marks::PassesFilter(const FNavigationToolViewModelPtr InItem) const
{
	const TViewModelPtr<FNavigationToolSequence> SequenceItem = InItem.ImplicitCast();
	if (!SequenceItem)
	{
		return false;
	}

	UMovieSceneSequence* const Sequence = SequenceItem->GetSequence();
	if (!Sequence)
	{
		return false;
	}

	UMovieScene* const MovieScene = Sequence->GetMovieScene();
	if (!MovieScene)
	{
		return false;
	}

	return MovieScene->GetMarkedFrames().Num() > 0;
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
