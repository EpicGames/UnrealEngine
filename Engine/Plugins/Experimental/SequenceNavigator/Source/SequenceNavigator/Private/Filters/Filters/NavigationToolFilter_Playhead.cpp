// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/Filters/NavigationToolFilter_Playhead.h"
#include "Engine/World.h"
#include "Extensions/IPlayheadExtension.h"
#include "Filters/NavigationToolFilterCommands.h"
#include "Items/INavigationToolItem.h"
#include "ISequencer.h"
#include "TimerManager.h"

#define LOCTEXT_NAMESPACE "NavigationToolFilter_Playhead"

namespace UE::SequenceNavigator
{

using namespace Sequencer;

FNavigationToolFilter_Playhead::FNavigationToolFilter_Playhead(INavigationToolFilterBar& InFilterInterface, TSharedPtr<FFilterCategory> InCategory)
	: FNavigationToolFilter(InFilterInterface, MoveTemp(InCategory))
{
}

FNavigationToolFilter_Playhead::~FNavigationToolFilter_Playhead()
{
	UnbindEvents();
}

void FNavigationToolFilter_Playhead::BindEvents()
{
	ISequencer& Sequencer = FilterInterface.GetSequencer();
	WeakSequencer = Sequencer.AsWeak();

	Sequencer.OnPlayEvent().AddSP(this, &FNavigationToolFilter_Playhead::OnPlayEvent);
	Sequencer.OnStopEvent().AddSP(this, &FNavigationToolFilter_Playhead::OnStopEvent);
	Sequencer.OnBeginScrubbingEvent().AddSP(this, &FNavigationToolFilter_Playhead::OnBeginScrubbingEvent);
	Sequencer.OnEndScrubbingEvent().AddSP(this, &FNavigationToolFilter_Playhead::OnEndScrubbingEvent);
}

void FNavigationToolFilter_Playhead::UnbindEvents()
{
	const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	Sequencer->OnPlayEvent().RemoveAll(this);
	Sequencer->OnStopEvent().RemoveAll(this);
	Sequencer->OnBeginScrubbingEvent().RemoveAll(this);
	Sequencer->OnEndScrubbingEvent().RemoveAll(this);
}

FText FNavigationToolFilter_Playhead::GetDefaultToolTipText() const
{
	return LOCTEXT("NavigationToolFilter_PlayheadToolTip", "Show only items whose range contains the current playhead location");
}

TSharedPtr<FUICommandInfo> FNavigationToolFilter_Playhead::GetToggleCommand() const
{
	return FNavigationToolFilterCommands::Get().ToggleFilter_Playhead;
}

void FNavigationToolFilter_Playhead::ActiveStateChanged(const bool bInActive)
{
	FNavigationToolFilter::ActiveStateChanged(bInActive);

	if (bInActive)
	{
		BindEvents();
	}
	else
	{
		UnbindEvents();
	}
}

FText FNavigationToolFilter_Playhead::GetDisplayName() const
{
	return LOCTEXT("NavigationToolFilter_Playhead", "Playhead");
}

FSlateIcon FNavigationToolFilter_Playhead::GetIcon() const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("PlayWorld.ContinueExecution.Small"));
}

FString FNavigationToolFilter_Playhead::GetName() const
{
	return StaticName();
}

bool FNavigationToolFilter_Playhead::PassesFilter(const FNavigationToolViewModelPtr InItem) const
{
	if (const TViewModelPtr<IPlayheadExtension> PlayheadItem = InItem.ImplicitCast())
	{
		return PlayheadItem->ContainsPlayhead() != EItemContainsPlayhead::None;
	}
	return false;
}

void FNavigationToolFilter_Playhead::OnPlayEvent()
{
	StartRefreshTimer();
}

void FNavigationToolFilter_Playhead::OnStopEvent()
{
	StopRefreshTimer();
}

void FNavigationToolFilter_Playhead::OnBeginScrubbingEvent()
{
	StartRefreshTimer();
}

void FNavigationToolFilter_Playhead::OnEndScrubbingEvent()
{
	StopRefreshTimer();
}

FTimerManager* FNavigationToolFilter_Playhead::GetTimerManager() const
{
	UObject* const ContextObject = FilterInterface.GetSequencer().GetPlaybackContext();
	if (!ContextObject)
	{
		return nullptr;
	}

	UWorld* const World = ContextObject->GetWorld();
	if (!World)
	{
		return nullptr;
	}

	return &World->GetTimerManager();
}

void FNavigationToolFilter_Playhead::StartRefreshTimer()
{
	if (TimerHandle.IsValid())
	{
		return;
	}

	if (FTimerManager* const TimerManager = GetTimerManager())
	{
		TimerManager->SetTimer(TimerHandle, [this]()
			{
				FilterInterface.RequestFilterUpdate();
			}, 0.1f, true);
	}
}

void FNavigationToolFilter_Playhead::StopRefreshTimer()
{
	if (FTimerManager* const TimerManager = GetTimerManager())
	{
		TimerManager->ClearTimer(TimerHandle);
	}
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
