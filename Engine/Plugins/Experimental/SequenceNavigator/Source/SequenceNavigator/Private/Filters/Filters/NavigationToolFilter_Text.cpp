// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/Filters/NavigationToolFilter_Text.h"
#include "Filters/INavigationToolFilterBar.h"
#include "Filters/ISequencerTextFilterExpressionContext.h"
#include "Filters/TextExpressions/NavigationToolFilterTextExpressionContext.h"
#include "Filters/TextExpressions/NavigationToolFilterTextExpressionExtension.h"
#include "Filters/TextExpressions/NavigationToolFilterTextExpression_Name.h"
#include "Filters/TextExpressions/NavigationToolFilterTextExpression_Unbound.h"
#include "Misc/TextFilterExpressionEvaluator.h"
#include "UObject/UObjectIterator.h"

#define LOCTEXT_NAMESPACE "NavigationToolFilter_Text"

namespace UE::SequenceNavigator
{

using namespace Sequencer;

FNavigationToolFilter_Text::FNavigationToolFilter_Text(INavigationToolFilterBar& InFilterInterface)
	: FNavigationToolFilter(InFilterInterface, nullptr)
	, TextFilterExpressionEvaluator(ETextFilterExpressionEvaluatorMode::Complex)
{
	IsActiveEvent = FIsActiveEvent::CreateLambda([this]()
		{
			return !GetRawFilterText().IsEmpty();
		});

	// Ordered by importance and most often used. This will dictate the order of display in the text expressions help dialog.
	TextFilterExpressionContexts.Add(MakeShared<FNavigationToolFilterTextExpression_Name>(InFilterInterface));
	TextFilterExpressionContexts.Add(MakeShared<FNavigationToolFilterTextExpression_Unbound>(InFilterInterface));

	// Add global user-defined text expressions
	for (TObjectIterator<UNavigationToolFilterTextExpressionExtension> ExtensionIt(RF_NoFlags); ExtensionIt; ++ExtensionIt)
	{
		const UNavigationToolFilterTextExpressionExtension* const PotentialExtension = *ExtensionIt;
		if (PotentialExtension
			&& PotentialExtension->HasAnyFlags(RF_ClassDefaultObject)
			&& !PotentialExtension->GetClass()->HasAnyClassFlags(CLASS_Deprecated | CLASS_Abstract))
		{
			TArray<TSharedRef<FNavigationToolFilterTextExpressionContext>> ExtendedTextExpressions;
			PotentialExtension->AddFilterTextExpressionExtensions(static_cast<INavigationToolFilterBar&>(FilterInterface), ExtendedTextExpressions);

			for (const TSharedRef<FNavigationToolFilterTextExpressionContext>& TextExpression : ExtendedTextExpressions)
			{
				TextFilterExpressionContexts.Add(TextExpression);
			}
		}
	}
}

FText FNavigationToolFilter_Text::GetDisplayName() const
{
	return LOCTEXT("NavigationToolFilter_Text", "Text");
}

FText FNavigationToolFilter_Text::GetToolTipText() const
{
	return LOCTEXT("NavigationToolFilter_TextTooltip", "Show only items that match the input text");
}

FString FNavigationToolFilter_Text::GetName() const
{
	return TEXT("TextFilter");
}

bool FNavigationToolFilter_Text::PassesFilter(const FNavigationToolViewModelPtr InItem) const
{
	for (const TSharedRef<FNavigationToolFilterTextExpressionContext>& TextFilterExpressionContext : TextFilterExpressionContexts)
	{
		TextFilterExpressionContext->SetFilterItem(InItem);

		const bool bPassedFilter = TextFilterExpressionEvaluator.TestTextFilter(*TextFilterExpressionContext);

		TextFilterExpressionContext->SetFilterItem(nullptr);

		if (!bPassedFilter)
		{
			return false;
		}
	}

	return true;
}

bool FNavigationToolFilter_Text::IsActive() const
{
	return !GetRawFilterText().IsEmpty();
}

FText FNavigationToolFilter_Text::GetRawFilterText() const
{
	return TextFilterExpressionEvaluator.GetFilterText();
}

FText FNavigationToolFilter_Text::GetFilterErrorText() const
{
	return TextFilterExpressionEvaluator.GetFilterErrorText();
}

void FNavigationToolFilter_Text::SetRawFilterText(const FText& InFilterText)
{
	if (TextFilterExpressionEvaluator.SetFilterText(InFilterText))
	{
		BroadcastChangedEvent();
	}
}

const FTextFilterExpressionEvaluator& FNavigationToolFilter_Text::GetTextFilterExpressionEvaluator() const
{
	return TextFilterExpressionEvaluator;
}

TArray<TSharedRef<ISequencerTextFilterExpressionContext>> FNavigationToolFilter_Text::GetTextFilterExpressionContexts() const
{
	TArray<TSharedRef<ISequencerTextFilterExpressionContext>> OutTextExpressions;
	Algo::Transform(TextFilterExpressionContexts, OutTextExpressions
		, [](TSharedRef<FNavigationToolFilterTextExpressionContext> InExpressionContext)
		{
			return StaticCastSharedRef<ISequencerTextFilterExpressionContext>(InExpressionContext);
		});
	return OutTextExpressions;
}

bool FNavigationToolFilter_Text::DoesTextFilterStringContainExpressionPair(const ISequencerTextFilterExpressionContext& InExpression) const
{
	const TArray<FExpressionToken>& ExpressionTokens = TextFilterExpressionEvaluator.GetFilterExpressionTokens();
	const int32 ExpressionCount = ExpressionTokens.Num();

	// Need atleast three tokens: key, operator, and value
	if (ExpressionTokens.Num() < 3)
	{
		return false;
	}

	const TSet<FName> Keys = InExpression.GetKeys();
	const ESequencerTextFilterValueType ValueType = InExpression.GetValueType();
	const TArray<FSequencerTextFilterKeyword> ValueKeywords = InExpression.GetValueKeywords();

	for (int32 Index = 0; Index < ExpressionCount - 2; ++Index)
	{
		// Match key
		const FExpressionToken& KeyToken = ExpressionTokens[Index];
		const FString KeyTokenString = KeyToken.Context.GetString();

		if (IsTokenKey(KeyToken, Keys))
		{
			// Match operator
			const int32 OperatorIndex = Index + 1;
			const FExpressionToken& OperatorToken = ExpressionTokens[OperatorIndex];
			const FString OperatorTokenString = OperatorToken.Context.GetString();

			if (IsTokenOperator(OperatorToken, ValueType))
			{
				// Match value
				const int32 ValueIndex = Index + 2;
				const FExpressionToken& ValueToken = ExpressionTokens[ValueIndex];
				const FString ValueTokenString = ValueToken.Context.GetString();

				if (IsTokenValueValid(ValueToken, ValueType))
				{
					return true;
				}
			}
		}
	}

	return false;
}

bool FNavigationToolFilter_Text::IsTokenKey(const FExpressionToken& InToken, const TSet<FName>& InKeys)
{
	const FString KeyTokenString = InToken.Context.GetString();
	
	for (const FName& Key : InKeys)
	{
		if (KeyTokenString.Equals(Key.ToString(), ESearchCase::IgnoreCase))
		{
			return true;
		}
	}

	return false;
};

bool FNavigationToolFilter_Text::IsTokenOperator(const FExpressionToken& InToken, const ESequencerTextFilterValueType InValueType)
{
	if (InValueType == ESequencerTextFilterValueType::String)
	{
		return InToken.Node.Cast<TextFilterExpressionParser::FEqual>()
			|| InToken.Node.Cast<TextFilterExpressionParser::FNotEqual>();
	}

	return InToken.Node.Cast<TextFilterExpressionParser::FEqual>()
		|| InToken.Node.Cast<TextFilterExpressionParser::FNotEqual>()
		|| InToken.Node.Cast<TextFilterExpressionParser::FLess>()
		|| InToken.Node.Cast<TextFilterExpressionParser::FLessOrEqual>()
		|| InToken.Node.Cast<TextFilterExpressionParser::FGreater>()
		|| InToken.Node.Cast<TextFilterExpressionParser::FGreaterOrEqual>();
}

bool FNavigationToolFilter_Text::IsTokenValueValid(const FExpressionToken& InToken, const ESequencerTextFilterValueType InValueType)
{
	if (!InToken.Node.Cast<TextFilterExpressionParser::FTextToken>())
	{
		return false;
	}

	// @TODO: better value checking?

	return true;
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
