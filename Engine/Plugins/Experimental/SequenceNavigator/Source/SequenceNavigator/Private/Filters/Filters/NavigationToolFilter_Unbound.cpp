// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/Filters/NavigationToolFilter_Unbound.h"
#include "Filters/NavigationToolFilterCommands.h"
#include "Items/NavigationToolBinding.h"

#define LOCTEXT_NAMESPACE "NavigationToolFilter_Unbound"

namespace UE::SequenceNavigator
{

using namespace Sequencer;

FNavigationToolFilter_Unbound::FNavigationToolFilter_Unbound(INavigationToolFilterBar& InFilterInterface, TSharedPtr<FFilterCategory> InCategory)
	: FNavigationToolFilter(InFilterInterface, MoveTemp(InCategory))
{
}

FText FNavigationToolFilter_Unbound::GetDefaultToolTipText() const
{
	return LOCTEXT("NavigationToolFilter_UnboundToolTip", "Show only sequences with Unbound tracks");
}

TSharedPtr<FUICommandInfo> FNavigationToolFilter_Unbound::GetToggleCommand() const
{
	return FNavigationToolFilterCommands::Get().ToggleFilter_Unbound;
}

FText FNavigationToolFilter_Unbound::GetDisplayName() const
{
	return LOCTEXT("NavigationToolFilter_Unbound", "Unbound");
}

FSlateIcon FNavigationToolFilter_Unbound::GetIcon() const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("BTEditor.Graph.BTNode.Decorator.DoesPathExist.Icon"));
}

FString FNavigationToolFilter_Unbound::GetName() const
{
	return StaticName();
}

bool FNavigationToolFilter_Unbound::PassesFilter(const FNavigationToolViewModelPtr InItem) const
{
	const TViewModelPtr<FNavigationToolBinding> BindingItem = InItem.ImplicitCast();
	if (!BindingItem)
	{
		return false;
	}

	const UObject* const BoundObject = BindingItem->GetCachedBoundObject();
	return !BoundObject;
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
