// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/Filters/NavigationToolFilters.h"
#include "Filters/Filters/NavigationToolFilterBase.h"
#include "Filters/NavigationToolFilterCommands.h"
#include "Items/NavigationToolSequence.h"
#include "LevelSequenceActor.h"
#include "Misc/IFilter.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateIconFinder.h"

#define LOCTEXT_NAMESPACE "NavigationToolFilters"

namespace UE::SequenceNavigator
{

using namespace Sequencer;

FNavigationToolFilter_Sequence::FNavigationToolFilter_Sequence(INavigationToolFilterBar& InFilterInterface, TSharedPtr<FFilterCategory> InCategory)
	: FNavigationToolFilter_ItemType<FNavigationToolSequence>(InFilterInterface, InCategory)
{
}

FText FNavigationToolFilter_Sequence::GetDisplayName() const
{
	return LOCTEXT("NavigationToolFilter_Sequence", "Sequence");
}

FSlateIcon FNavigationToolFilter_Sequence::GetIcon() const
{
	return FSlateIconFinder::FindIconForClass(ALevelSequenceActor::StaticClass());
}

FText FNavigationToolFilter_Sequence::GetDefaultToolTipText() const
{
	return LOCTEXT("NavigationToolFilter_SequenceToolTip", "Show only Sequence items");
}

TSharedPtr<FUICommandInfo> FNavigationToolFilter_Sequence::GetToggleCommand() const
{
	return FNavigationToolFilterCommands::Get().ToggleFilter_Sequence;
}

bool FNavigationToolFilter_Sequence::SupportsSequence(UMovieSceneSequence* const InSequence) const
{
	return true;
}

//////////////////////////////////////////////////////////////////////////
//

FNavigationToolFilter_Track::FNavigationToolFilter_Track(INavigationToolFilterBar& InFilterInterface, TSharedPtr<FFilterCategory> InCategory)
	: FNavigationToolFilter_ItemType<FNavigationToolTrack>(InFilterInterface, InCategory)
{
}

FText FNavigationToolFilter_Track::GetDisplayName() const
{
	return LOCTEXT("NavigationToolFilter_Track", "Track");
}

FSlateIcon FNavigationToolFilter_Track::GetIcon() const
{
	return FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Sequencer.SetAutoTrack"));
}

FText FNavigationToolFilter_Track::GetDefaultToolTipText() const
{
	return LOCTEXT("NavigationToolFilter_TrackToolTip", "Show only Track items");
}

TSharedPtr<FUICommandInfo> FNavigationToolFilter_Track::GetToggleCommand() const
{
	return FNavigationToolFilterCommands::Get().ToggleFilter_Track;
}

bool FNavigationToolFilter_Track::SupportsSequence(UMovieSceneSequence* const InSequence) const
{
	return true;
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
