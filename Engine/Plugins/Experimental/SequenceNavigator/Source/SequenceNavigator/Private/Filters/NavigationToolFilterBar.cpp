// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/NavigationToolFilterBar.h"
#include "Filters/Widgets/SFilterExpressionHelpDialog.h"
#include "Filters/Filters/NavigationToolFilter_CustomText.h"
#include "Filters/Filters/NavigationToolFilter_Dirty.h"
#include "Filters/Filters/NavigationToolFilter_Marks.h"
#include "Filters/Filters/NavigationToolFilter_Playhead.h"
#include "Filters/Filters/NavigationToolFilter_Unbound.h"
#include "Filters/Filters/NavigationToolFilterBase.h"
#include "Filters/Filters/NavigationToolFilterCollection.h"
#include "Filters/Filters/NavigationToolFilterExtension.h"
#include "Filters/Filters/NavigationToolFilters.h"
#include "Filters/NavigationToolFilterCommands.h"
#include "Filters/SequencerFilterBarConfig.h"
#include "Filters/Widgets/SSequencerSearchBox.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/SToolBarButtonBlock.h"
#include "ISequencer.h"
#include "Items/NavigationToolTreeRoot.h"
#include "Menus/NavigationToolFilterMenu.h"
#include "NavigationTool.h"
#include "NavigationToolSettings.h"
#include "NavigationToolView.h"
#include "SequencerSettings.h"
#include "Stats/StatsMisc.h"
#include "UObject/UObjectIterator.h"
#include "Widgets/SNavigationToolCustomTextFilterDialog.h"
#include "Widgets/SNavigationToolFilterBar.h"

#define LOCTEXT_NAMESPACE "NavigationToolFilterBar"

namespace UE::SequenceNavigator
{

using namespace Sequencer;

const FName FNavigationToolFilterBar::SharedIdentifier = TEXT("SharedNavigationToolFilter");

int32 FNavigationToolFilterBar::InstanceCount = 0;

FNavigationToolFilterBar::FNavigationToolFilterBar(FNavigationTool& InTool)
	: Tool(InTool)
	, CommandList(MakeShared<FUICommandList>())
	, ClassTypeCategory(MakeShared<FFilterCategory>(LOCTEXT("ActorTypeFilterCategory", "Actor Type Filters"), FText()))
	, ComponentTypeCategory(MakeShared<FFilterCategory>(LOCTEXT("ObjectTypeFilterCategory", "Object Type Filters"), FText()))
	, MiscCategory(MakeShared<FFilterCategory>(LOCTEXT("MiscFilterCategory", "Misc Filters"), FText()))
	, TransientCategory(MakeShared<FFilterCategory>(LOCTEXT("TransientFilterCategory", "Transient Filters"), FText()))
	, CommonFilters(MakeShared<FNavigationToolFilterCollection>(*this))
	, TextFilter(MakeShared<FNavigationToolFilter_Text>(*this))
	//, SelectedFilter(MakeShared<FNavigationToolFilter_Selected>(*this, MiscCategory))
	, FilterMenu(MakeShared<FNavigationToolFilterMenu>())
	, FilterData(FString())
{
	InstanceCount++;

	FNavigationToolFilterCommands::Register();
}

FNavigationToolFilterBar::~FNavigationToolFilterBar()
{
	InstanceCount--;

	if (InstanceCount == 0)
	{
		FNavigationToolFilterCommands::Unregister();
	}

	CommonFilters->OnChanged().RemoveAll(this);
	TextFilter->OnChanged().RemoveAll(this);
	//SelectedFilter->OnChanged().RemoveAll(this);
}

void FNavigationToolFilterBar::Init()
{
	CommonFilters->OnChanged().AddSP(this, &FNavigationToolFilterBar::RequestFilterUpdate);
	TextFilter->OnChanged().AddSP(this, &FNavigationToolFilterBar::HandleTextFilterChanged);
	//SelectedFilter->OnChanged().AddSP(this, &FNavigationToolFilterBar::RequestFilterUpdate);

	CreateDefaultFilters();

	CreateCustomTextFiltersFromConfig();
}

TSharedPtr<ICustomTextFilter<FNavigationToolViewModelPtr>> FNavigationToolFilterBar::CreateTextFilter()
{
	return MakeShared<FNavigationToolFilter_CustomText>(*this);
}

void FNavigationToolFilterBar::CreateDefaultFilters()
{
	auto AddFilterIfSupported = [this]
		(const TSharedPtr<FNavigationToolFilterCollection>& InFilterCollection, const TSharedRef<FNavigationToolFilter>& InFilter)
	{
		if (IsFilterSupported(InFilter))
		{
			InFilterCollection->Add(InFilter);
		}
	};

	// Add class type category filters
	CommonFilters->RemoveAll();

	//CommonFilters->Add(MakeShared<FNavigationToolFilter_Sequence>(*this, ClassTypeCategory));
	//CommonFilters->Add(MakeShared<FNavigationToolFilter_Track>(*this, ClassTypeCategory));

	// Add misc category filters
	AddFilterIfSupported(CommonFilters, MakeShared<FNavigationToolFilter_Unbound>(*this, MiscCategory));
	AddFilterIfSupported(CommonFilters, MakeShared<FNavigationToolFilter_Marks>(*this, MiscCategory));
	AddFilterIfSupported(CommonFilters, MakeShared<FNavigationToolFilter_Playhead>(*this, MiscCategory));
	AddFilterIfSupported(CommonFilters, MakeShared<FNavigationToolFilter_Dirty>(*this, MiscCategory));

	// Add global user-defined filters
	for (TObjectIterator<UNavigationToolFilterExtension> ExtensionIt(RF_NoFlags); ExtensionIt; ++ExtensionIt)
	{
		const UNavigationToolFilterExtension* const PotentialExtension = *ExtensionIt;
		if (PotentialExtension
			&& PotentialExtension->HasAnyFlags(RF_ClassDefaultObject)
			&& !PotentialExtension->GetClass()->HasAnyClassFlags(CLASS_Deprecated | CLASS_Abstract))
		{
			TArray<TSharedRef<FNavigationToolFilter>> ExtendedFilters;
			PotentialExtension->AddFilterExtensions(*this, ClassTypeCategory, ExtendedFilters);

			for (const TSharedRef<FNavigationToolFilter>& ExtendedFilter : ExtendedFilters)
			{
				AddFilterIfSupported(CommonFilters, ExtendedFilter);
			}
		}
	}

	CommonFilters->Sort();
}

void FNavigationToolFilterBar::BindCommands(const TSharedPtr<FUICommandList>& InBaseCommandList)
{
	if (InBaseCommandList.IsValid())
	{
		InBaseCommandList->Append(CommandList);
	}

	const FNavigationToolFilterCommands& FilterCommands = FNavigationToolFilterCommands::Get();

	CommandList->MapAction(FilterCommands.ToggleFilterBarVisibility,
		FExecuteAction::CreateSP(this, &FNavigationToolFilterBar::ToggleFilterBarVisibility),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FNavigationToolFilterBar::IsFilterBarVisible));

	CommandList->MapAction(FilterCommands.SetToVerticalLayout,
		FExecuteAction::CreateSP(this, &FNavigationToolFilterBar::SetToVerticalLayout),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FNavigationToolFilterBar::IsFilterBarLayout, EFilterBarLayout::Vertical));

	CommandList->MapAction(FilterCommands.SetToHorizontalLayout,
		FExecuteAction::CreateSP(this, &FNavigationToolFilterBar::SetToHorizontalLayout),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FNavigationToolFilterBar::IsFilterBarLayout, EFilterBarLayout::Horizontal));

	CommandList->MapAction(FilterCommands.ResetFilters,
		FExecuteAction::CreateSP(this, &FNavigationToolFilterBar::ResetFilters),
		FCanExecuteAction::CreateSP(this, &FNavigationToolFilterBar::CanResetFilters));

	CommandList->MapAction(FilterCommands.ToggleMuteFilters,
		FExecuteAction::CreateSP(this, &FNavigationToolFilterBar::ToggleMuteFilters),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FNavigationToolFilterBar::AreFiltersMuted));

	CommandList->MapAction(FilterCommands.DisableAllFilters,
		FExecuteAction::CreateSPLambda(this, [this]()
			{
				EnableAllFilters(false, {});
			}),
		FCanExecuteAction::CreateSP(this, &FNavigationToolFilterBar::HasAnyFilterEnabled));

	CommandList->MapAction(FilterCommands.ToggleActivateEnabledFilters,
		FExecuteAction::CreateSP(this, &FNavigationToolFilterBar::ToggleActivateAllEnabledFilters),
		FCanExecuteAction::CreateSP(this, &FNavigationToolFilterBar::HasAnyFilterEnabled));

	CommandList->MapAction(FilterCommands.ActivateAllFilters,
		FExecuteAction::CreateSP(this, &FNavigationToolFilterBar::ActivateAllEnabledFilters, true, TArray<FString>()));

	CommandList->MapAction(FilterCommands.DeactivateAllFilters,
		FExecuteAction::CreateSP(this, &FNavigationToolFilterBar::ActivateAllEnabledFilters, false, TArray<FString>()));

	// Bind all filter actions
	const TSharedPtr<ISequencer> Sequencer = Tool.GetSequencer();
	if (!Sequencer.IsValid())
	{
		return;
	}

	UMovieSceneSequence* const FocusedSequence = Sequencer->GetFocusedMovieSceneSequence();
	if (!FocusedSequence)
	{
		return;
	}

	const TArray<TSharedRef<FNavigationToolFilter>> AllFilters = GetFilterList(true);
	for (const TSharedRef<FNavigationToolFilter>& Filter : AllFilters)
	{
		if (Filter->SupportsSequence(FocusedSequence))
		{
			Filter->BindCommands();
		}
	}
}

void FNavigationToolFilterBar::CreateCustomTextFiltersFromConfig()
{
	UNavigationToolSettings* const ToolSettings = GetMutableDefault<UNavigationToolSettings>();
	if (!ToolSettings)
	{
		return;
	}

	CustomTextFilters.Empty();

	FSequencerFilterBarConfig& Config = ToolSettings->FindOrAddFilterBar(GetIdentifier(), false);

	for (const FCustomTextFilterData& CustomTextFilterData : Config.GetCustomTextFilters())
	{
		const TSharedRef<FNavigationToolFilter_CustomText> NewCustomTextFilter = MakeShared<FNavigationToolFilter_CustomText>(*this);
		NewCustomTextFilter->SetFromCustomTextFilterData(CustomTextFilterData);
		CustomTextFilters.Add(NewCustomTextFilter);
	}
}

ISequencer& FNavigationToolFilterBar::GetSequencer() const
{
	return *Tool.GetSequencer();
}

TSharedPtr<FUICommandList> FNavigationToolFilterBar::GetCommandList() const
{
	return CommandList;
}

FName FNavigationToolFilterBar::GetIdentifier() const
{
	const FName DefaultIdentifier = TEXT("NavigationToolMain");

	const TSharedPtr<ISequencer> Sequencer = Tool.GetSequencer();
	if (!Sequencer.IsValid())
	{
		return DefaultIdentifier;
	}

	USequencerSettings* const SequencerSettings = Sequencer->GetSequencerSettings();
	if (!SequencerSettings)
	{
		return DefaultIdentifier;
	}

	return *SequencerSettings->GetName();
}

TSharedRef<SSequencerSearchBox> FNavigationToolFilterBar::GetOrCreateSearchBoxWidget()
{
	return SAssignNew(WeakSearchBoxWidget, SSequencerSearchBox, SharedThis(this))
		.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("NavigationToolFilterSearch")))
		.HintText(LOCTEXT("FilterSearch", "Search..."))
		.ToolTipText(LOCTEXT("FilterSearchHint", "Type here to search (pressing enter selects the results)"))
		.OnTextChanged(this, &FNavigationToolFilterBar::OnSearchTextChanged)
		.OnTextCommitted(this, &FNavigationToolFilterBar::OnSearchTextCommitted)
		.OnSaveSearchClicked(this, &FNavigationToolFilterBar::OnSearchTextSaved);
}

TSharedRef<SNavigationToolFilterBar> FNavigationToolFilterBar::GenerateWidget()
{
	EFilterBarLayout Layout = EFilterBarLayout::Horizontal;

	const UNavigationToolSettings* const ToolSettings = GetMutableDefault<UNavigationToolSettings>();
	if (ensure(ToolSettings))
	{
		Layout = ToolSettings->GetFilterBarLayout();
	}

	return SAssignNew(FilterBarWidget, SNavigationToolFilterBar, SharedThis(this))
		.FilterBarLayout(Layout)
		.FiltersMuted(AreFiltersMuted())
		.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("NavigationToolFilters")))
		.FilterSearchBox(WeakSearchBoxWidget.Pin());
}

bool FNavigationToolFilterBar::AreFiltersMuted() const
{
	return bFiltersMuted;
}

void FNavigationToolFilterBar::MuteFilters(const bool bInMute)
{
	bFiltersMuted = bInMute;

	if (FilterBarWidget.IsValid())
	{
		FilterBarWidget->SetMuted(bFiltersMuted);
	}

	MuteFiltersChangedEvent.Broadcast(bFiltersMuted);
	RequestFilterUpdate();
}

void FNavigationToolFilterBar::ToggleMuteFilters()
{
	MuteFilters(!AreFiltersMuted());
}

void FNavigationToolFilterBar::ResetFilters()
{
	EnableAllFilters(false, {});
	EnableCustomTextFilters(false);
}

bool FNavigationToolFilterBar::CanResetFilters() const
{
	const TArray<TSharedRef<FFilterCategory>> Categories = { GetClassTypeCategory(), GetComponentTypeCategory(), GetMiscCategory() };
	const TArray<TSharedRef<FNavigationToolFilter>> ClassAndCompFilters = GetCommonFilters(Categories);
	return HasEnabledFilter(ClassAndCompFilters);
}

FText FNavigationToolFilterBar::GetTextFilterText() const
{
	return TextFilter->GetRawFilterText();
}

FString FNavigationToolFilterBar::GetTextFilterString() const
{
	return GetTextFilterText().ToString();
}

void FNavigationToolFilterBar::SetTextFilterString(const FString& InText)
{
	TextFilter->SetRawFilterText(FText::FromString(InText));

	if (FilterBarWidget.IsValid())
	{
		FilterBarWidget->SetTextFilterString(InText);
	}
}

bool FNavigationToolFilterBar::DoesTextFilterStringContainExpressionPair(const ISequencerTextFilterExpressionContext& InExpression) const
{
	return TextFilter->DoesTextFilterStringContainExpressionPair(InExpression);
}

TSharedRef<FNavigationToolFilter_Text> FNavigationToolFilterBar::GetTextFilter() const
{
	return TextFilter;
}

FText FNavigationToolFilterBar::GetFilterErrorText() const
{
	return TextFilter->GetFilterErrorText();
}

void FNavigationToolFilterBar::RequestFilterUpdate()
{
	if (const TSharedPtr<INavigationToolView> RecentToolView = Tool.GetMostRecentToolView())
	{
		RecentToolView->RequestRefresh();
	}

	Tool.RequestRefresh();

	RequestUpdateEvent.Broadcast();
}

TSharedPtr<FNavigationToolFilter> FNavigationToolFilterBar::FindFilterByDisplayName(const FString& InFilterName) const
{
	TSharedPtr<FNavigationToolFilter> OutFilter;

	CommonFilters->ForEachFilter([&InFilterName, &OutFilter]
		(const TSharedRef<FNavigationToolFilter>& InFilter)
		{
			const FString FilterName = InFilter->GetDisplayName().ToString();
			if (FilterName.Equals(InFilterName, ESearchCase::IgnoreCase))
			{
				OutFilter = InFilter;
				return false;
			}
			return true;
		});

	return OutFilter;
}

TSharedPtr<FNavigationToolFilter_CustomText> FNavigationToolFilterBar::FindCustomTextFilterByDisplayName(const FString& InFilterName) const
{
	TSharedPtr<FNavigationToolFilter_CustomText> OutFilter;

	for (const TSharedRef<FNavigationToolFilter_CustomText>& CustomTextFilter : CustomTextFilters)
	{
		const FString FilterName = CustomTextFilter->GetDisplayName().ToString();
		if (FilterName.Equals(InFilterName, ESearchCase::IgnoreCase))
		{
			OutFilter = CustomTextFilter;
			break;
		}
	}

	return OutFilter;
}

bool FNavigationToolFilterBar::HasAnyFiltersEnabled() const
{
	return HasEnabledCommonFilters() || HasEnabledCustomTextFilters();
}

bool FNavigationToolFilterBar::IsFilterActiveByDisplayName(const FString& InFilterName) const
{
	if (const TSharedPtr<FNavigationToolFilter> Filter = FindFilterByDisplayName(InFilterName))
	{
		return IsFilterActive(Filter.ToSharedRef());
	}
	return false;
}

bool FNavigationToolFilterBar::IsFilterEnabledByDisplayName(const FString& InFilterName) const
{
	if (const TSharedPtr<FNavigationToolFilter> Filter = FindFilterByDisplayName(InFilterName))
	{
		return IsFilterEnabled(Filter.ToSharedRef());
	}
	return false;
}

bool FNavigationToolFilterBar::SetFilterActiveByDisplayName(const FString& InFilterName, const bool bInActive, const bool bInRequestFilterUpdate)
{
	if (const TSharedPtr<FNavigationToolFilter> Filter = FindFilterByDisplayName(InFilterName))
	{
		return SetFilterActive(Filter.ToSharedRef(), bInActive, bInRequestFilterUpdate);
	}

	if (const TSharedPtr<FNavigationToolFilter> Filter = FindCustomTextFilterByDisplayName(InFilterName))
	{
		return SetFilterActive(Filter.ToSharedRef(), bInActive, bInRequestFilterUpdate);
	}

	return false;
}

bool FNavigationToolFilterBar::SetFilterEnabledByDisplayName(const FString& InFilterName, const bool bInEnabled, const bool bInRequestFilterUpdate)
{
	if (const TSharedPtr<FNavigationToolFilter> Filter = FindFilterByDisplayName(InFilterName))
	{
		return SetFilterEnabled(Filter.ToSharedRef(), bInEnabled, bInRequestFilterUpdate);
	}

	if (const TSharedPtr<FNavigationToolFilter> Filter = FindCustomTextFilterByDisplayName(InFilterName))
	{
		return SetFilterEnabled(Filter.ToSharedRef(), bInEnabled, bInRequestFilterUpdate);
	}

	return false;
}

bool FNavigationToolFilterBar::AnyCommonFilterActive() const
{
	bool bOutActiveFilter = false;

	CommonFilters->ForEachFilter([this, &bOutActiveFilter]
		(const TSharedRef<FNavigationToolFilter>& InFilter)
		{
			if (IsFilterActive(InFilter))
			{
				bOutActiveFilter = true;
				return false;
			}
			return true;
		});

	return bOutActiveFilter;
}

bool FNavigationToolFilterBar::HasAnyFilterActive(const bool bCheckTextFilter
	, const bool bInCheckHideIsolateFilter
	, const bool bInCheckCommonFilters
	, const bool bInCheckInternalFilters
	, const bool bInCheckCustomTextFilters) const
{
	if (bFiltersMuted)
	{
		return false;
	}

	const bool bTextFilterActive = bCheckTextFilter && TextFilter->IsActive();
	const bool bCommonFilterActive = bInCheckCommonFilters && AnyCommonFilterActive();
	const bool bCustomTextFilterActive = bInCheckCustomTextFilters && AnyCustomTextFilterActive();

	return bTextFilterActive
		|| bCommonFilterActive
		|| bCustomTextFilterActive;
}

bool FNavigationToolFilterBar::IsFilterActive(const TSharedRef<FNavigationToolFilter> InFilter) const
{
	UNavigationToolSettings* const ToolSettings = GetMutableDefault<UNavigationToolSettings>();
	if (!ToolSettings)
	{
		return false;
	}

	const FSequencerFilterBarConfig& Config = ToolSettings->FindOrAddFilterBar(GetIdentifier(), false);

	const FString FilterName = InFilter->GetDisplayName().ToString();
	return Config.IsFilterActive(FilterName);
}

bool FNavigationToolFilterBar::SetFilterActive(const TSharedRef<FNavigationToolFilter>& InFilter, const bool bInActive, const bool bInRequestFilterUpdate)
{
	UNavigationToolSettings* const ToolSettings = GetMutableDefault<UNavigationToolSettings>();
	if (!ToolSettings)
	{
		return false;
	}

	const bool bNewActive = InFilter->IsInverseFilter() ? !bInActive : bInActive;

	FSequencerFilterBarConfig& Config = ToolSettings->FindOrAddFilterBar(GetIdentifier(), /*bInSaveConfig=*/false);

	const FString FilterName = InFilter->GetDisplayName().ToString();
	const bool bSuccess = (Config.IsFilterActive(FilterName) == bInActive)
		|| Config.SetFilterActive(FilterName, bNewActive);

	if (bSuccess)
	{
		ToolSettings->SaveConfig();

		InFilter->SetActive(bNewActive);
		InFilter->ActiveStateChanged(bNewActive);

		const ENavigationToolFilterChange FilterChangeType = bNewActive
			? ENavigationToolFilterChange::Activate : ENavigationToolFilterChange::Deactivate;
		BroadcastFiltersChanged(InFilter, FilterChangeType);

		if (bInRequestFilterUpdate)
		{
			RequestFilterUpdate();
		}
	}

	return bSuccess;
}

void FNavigationToolFilterBar::EnableAllFilters(const bool bInEnable, const TArray<FString>& InExceptionFilterNames)
{
	TArray<TSharedRef<FNavigationToolFilter>> ExceptionFilters;
	TArray<TSharedRef<FNavigationToolFilter_CustomText>> ExceptionCustomTextFilters;

	for (const FString& FilterName : InExceptionFilterNames)
	{
		if (const TSharedPtr<FNavigationToolFilter> Filter = FindFilterByDisplayName(FilterName))
		{
			ExceptionFilters.Add(Filter.ToSharedRef());
		}
		else if (const TSharedPtr<FNavigationToolFilter_CustomText> CustomTextFilter = FindCustomTextFilterByDisplayName(FilterName))
		{
			ExceptionCustomTextFilters.Add(CustomTextFilter.ToSharedRef());
		}
	}

	EnableFilters(bInEnable, {}, ExceptionFilters);
	EnableCustomTextFilters(bInEnable, ExceptionCustomTextFilters);
}

void FNavigationToolFilterBar::GetCommonFiltersImpl(TArray<TSharedPtr<void>>& OutTypeErasedFilters) const
{
	for (const TSharedRef<FNavigationToolFilter>& Filter : CommonFilters->GetAllFilters())
	{
		OutTypeErasedFilters.Add(Filter);
	}
}

void FNavigationToolFilterBar::ActivateCommonFilters(const bool bInActivate, const TArray<FString>& InExceptionFilterNames)
{
	TArray<TSharedRef<FNavigationToolFilter>> ExceptionFilters;

	for (const FString& FilterName : InExceptionFilterNames)
	{
		if (const TSharedPtr<FNavigationToolFilter> Filter = FindFilterByDisplayName(FilterName))
		{
			ExceptionFilters.Add(Filter.ToSharedRef());
		}
	}

	return ActivateCommonFilters(bInActivate, {}, ExceptionFilters);
}

void FNavigationToolFilterBar::ActivateCommonFilters(const bool bInActivate
    , const TArray<TSharedRef<FFilterCategory>>& InMatchCategories
    , const TArray<TSharedRef<FNavigationToolFilter>>& InExceptions)
{
	UNavigationToolSettings* const ToolSettings = GetMutableDefault<UNavigationToolSettings>();
	if (!ToolSettings)
	{
		return;
	}

	FSequencerFilterBarConfig& Config = ToolSettings->FindOrAddFilterBar(GetIdentifier(), /*bInSaveConfig=*/false);

	bool bNeedsSave = false;

	CommonFilters->ForEachFilter([this, bInActivate, &InExceptions, &Config, &bNeedsSave]
		(const TSharedRef<FNavigationToolFilter>& InFilter)
		{
			if (InExceptions.Contains(InFilter))
			{
				return true;
			}

			const FString FilterName = InFilter->GetDisplayName().ToString();
			if (Config.SetFilterActive(FilterName, bInActivate))
			{
				const ENavigationToolFilterChange FilterChangeType = bInActivate ? ENavigationToolFilterChange::Activate : ENavigationToolFilterChange::Deactivate;
				BroadcastFiltersChanged(InFilter, FilterChangeType);

				InFilter->SetActive(bInActivate);
				InFilter->ActiveStateChanged(bInActivate);

				bNeedsSave = true;
			}

			return true;
		}
		, InMatchCategories);

	if (bNeedsSave)
	{
		ToolSettings->SaveConfig();
	}

	RequestFilterUpdate();
}

bool FNavigationToolFilterBar::AreAllEnabledFiltersActive(const bool bInActive, const TArray<FString> InExceptionFilterNames) const
{
	const TArray<TSharedRef<FNavigationToolFilter>> EnabledFilters = GetEnabledFilters();
	for (const TSharedRef<FNavigationToolFilter>& Filter : EnabledFilters)
	{
		const FString FilterName = Filter->GetDisplayName().ToString();
		if (InExceptionFilterNames.Contains(FilterName))
		{
			continue;
		}

		if (IsFilterActive(Filter) != bInActive)
		{
			return false;
		}
	}

	const TArray<TSharedRef<FNavigationToolFilter_CustomText>> EnabledCustomTextFilters = GetEnabledCustomTextFilters();
	for (const TSharedRef<FNavigationToolFilter_CustomText>& CustomTextFilter : EnabledCustomTextFilters)
	{
		const FString FilterName = CustomTextFilter->GetDisplayName().ToString();
		if (InExceptionFilterNames.Contains(FilterName))
		{
			continue;
		}

		if (IsFilterActive(CustomTextFilter) != bInActive)
		{
			return false;
		}
	}

	return true;
}

void FNavigationToolFilterBar::ActivateAllEnabledFilters(const bool bInActivate, const TArray<FString> InExceptionFilterNames)
{
	const TArray<TSharedRef<FNavigationToolFilter>> EnabledFilters = GetEnabledFilters();
	for (const TSharedRef<FNavigationToolFilter>& Filter : EnabledFilters)
	{
		const FString FilterName = Filter->GetDisplayName().ToString();
		if (InExceptionFilterNames.Contains(FilterName))
        {
        	continue;
        }

		if (IsFilterActive(Filter) != bInActivate)
		{
			SetFilterActive(Filter, bInActivate);
		}
	}

	const TArray<TSharedRef<FNavigationToolFilter_CustomText>> EnabledCustomTextFilters = GetEnabledCustomTextFilters();
	for (const TSharedRef<FNavigationToolFilter_CustomText>& CustomTextFilter : EnabledCustomTextFilters)
	{
		const FString FilterName = CustomTextFilter->GetDisplayName().ToString();
		if (InExceptionFilterNames.Contains(FilterName))
		{
			continue;
		}

		if (IsFilterActive(CustomTextFilter) != bInActivate)
		{
			SetFilterActive(CustomTextFilter, bInActivate);
		}
	}

	// Broadcast state change if any filter is active, or all filters are being deactivated.
	if (!bInActivate || HasAnyFilterActive(false, false, true, true, true))
	{
		ActivationStateChangedEvent.Broadcast();
	}
}

void FNavigationToolFilterBar::ToggleActivateAllEnabledFilters()
{
	const bool bNewActive = !AreAllEnabledFiltersActive(true, {});
	ActivateAllEnabledFilters(bNewActive, {});
}

TArray<TSharedRef<FNavigationToolFilter>> FNavigationToolFilterBar::GetActiveFilters() const
{
	TArray<TSharedRef<FNavigationToolFilter>> OutFilters;

	CommonFilters->ForEachFilter([this, &OutFilters]
		(const TSharedRef<FNavigationToolFilter>& InFilter)
		{
			if (IsFilterActive(InFilter))
			{
				OutFilters.Add(InFilter);
			}
			return true;
		});

	return OutFilters;
}

bool FNavigationToolFilterBar::HasEnabledCommonFilters() const
{
	bool bOutReturn = false;

	CommonFilters->ForEachFilter([this, &bOutReturn]
		(const TSharedRef<FNavigationToolFilter>& InFilter)
		{
			if (IsFilterEnabled(InFilter))
			{
				bOutReturn = true;
				return false;
			}
			return true;
		});

	if (bOutReturn)
	{
		return true;
	}

	return bOutReturn;
}

bool FNavigationToolFilterBar::HasEnabledFilter(const TArray<TSharedRef<FNavigationToolFilter>>& InFilters) const
{
	const TArray<TSharedRef<FNavigationToolFilter>>& Filters = InFilters.IsEmpty() ? GetCommonFilters() : InFilters;

	for (const TSharedRef<FNavigationToolFilter>& Filter : Filters)
	{
		if (IsFilterEnabled(Filter))
		{
			return true;
		}
	}

	return false;
}

bool FNavigationToolFilterBar::HasAnyFilterEnabled() const
{
	const bool bCommonFilterEnabled = HasEnabledCommonFilters();
	const bool bCustomTextFilterEnabled = HasEnabledCustomTextFilters();

	return bCommonFilterEnabled
		|| bCustomTextFilterEnabled;
}

bool FNavigationToolFilterBar::IsFilterEnabled(const TSharedRef<FNavigationToolFilter> InFilter) const
{
	UNavigationToolSettings* const ToolSettings = GetMutableDefault<UNavigationToolSettings>();
	if (!ToolSettings)
	{
		return false;
	}

	FSequencerFilterBarConfig& Config = ToolSettings->FindOrAddFilterBar(GetIdentifier(), false);

	const FString FilterName = InFilter->GetDisplayName().ToString();
	return Config.IsFilterEnabled(FilterName);
}

bool FNavigationToolFilterBar::SetFilterEnabled(const TSharedRef<FNavigationToolFilter> InFilter, const bool bInEnabled, const bool bInRequestFilterUpdate)
{
	UNavigationToolSettings* const ToolSettings = GetMutableDefault<UNavigationToolSettings>();
	if (!ToolSettings)
	{
		return false;
	}

	FSequencerFilterBarConfig& Config = ToolSettings->FindOrAddFilterBar(GetIdentifier(), /*bInSaveConfig=*/false);

	const FString FilterName = InFilter->GetDisplayName().ToString();
	const bool bSuccess = Config.SetFilterEnabled(FilterName, bInEnabled);

	if (bSuccess)
	{
		ToolSettings->SaveConfig();

		const ENavigationToolFilterChange FilterChangeType = bInEnabled
			? ENavigationToolFilterChange::Enable : ENavigationToolFilterChange::Disable;
		BroadcastFiltersChanged(InFilter, FilterChangeType);

		if (!bInEnabled && IsFilterActive(InFilter))
		{
			InFilter->SetActive(false);
			InFilter->ActiveStateChanged(false);
		}

		if (bInRequestFilterUpdate)
		{
			RequestFilterUpdate();
		}
	}

	return bSuccess;
}

void FNavigationToolFilterBar::EnableFilters(const bool bInEnable
	, const TArray<TSharedRef<FFilterCategory>> InMatchCategories
	, const TArray<TSharedRef<FNavigationToolFilter>> InExceptions)
{
	UNavigationToolSettings* const ToolSettings = GetMutableDefault<UNavigationToolSettings>();
	if (!ToolSettings)
	{
		return;
	}

	FSequencerFilterBarConfig& Config = ToolSettings->FindOrAddFilterBar(GetIdentifier(), true);

	CommonFilters->ForEachFilter([this, bInEnable, &InExceptions, &Config]
		(const TSharedRef<FNavigationToolFilter>& InFilter)
		{
			if (InExceptions.IsEmpty() || !InExceptions.Contains(InFilter))
			{
				const FString FilterName = InFilter->GetDisplayName().ToString();
				if (Config.SetFilterEnabled(FilterName, bInEnable))
				{
					const ENavigationToolFilterChange FilterChangeType = bInEnable
						? ENavigationToolFilterChange::Enable : ENavigationToolFilterChange::Disable;
					BroadcastFiltersChanged(InFilter, FilterChangeType);

					if (!bInEnable && IsFilterActive(InFilter))
					{
						InFilter->SetActive(false);
						InFilter->ActiveStateChanged(false);
					}
				}
			}
			return true;
		}
		, InMatchCategories);

	ToolSettings->SaveConfig();

	RequestFilterUpdate();
}

void FNavigationToolFilterBar::ToggleFilterEnabled(const TSharedRef<FNavigationToolFilter> InFilter)
{
	SetFilterEnabled(InFilter, !IsFilterEnabled(InFilter), true);
}

TArray<TSharedRef<FNavigationToolFilter>> FNavigationToolFilterBar::GetEnabledFilters() const
{
	TArray<TSharedRef<FNavigationToolFilter>> OutFilters;

	CommonFilters->ForEachFilter([this, &OutFilters]
		(const TSharedRef<FNavigationToolFilter>& InFilter)
		{
			if (IsFilterEnabled(InFilter))
			{
				OutFilters.Add(InFilter);
			}
			return true;
		});

	return OutFilters;
}

bool FNavigationToolFilterBar::HasAnyCommonFilters() const
{
	return !CommonFilters->IsEmpty();
}

bool FNavigationToolFilterBar::AddFilter(const TSharedRef<FNavigationToolFilter>& InFilter)
{
	const bool bSuccess = CommonFilters->Add(InFilter) == 1;

	return bSuccess;
}

bool FNavigationToolFilterBar::RemoveFilter(const TSharedRef<FNavigationToolFilter>& InFilter)
{
	const bool bSuccess = CommonFilters->Remove(InFilter) == 1;

	if (bSuccess)
	{
		BroadcastFiltersChanged(InFilter, ENavigationToolFilterChange::Disable);
	}

	return bSuccess;
}

TArray<FText> FNavigationToolFilterBar::GetFilterDisplayNames() const
{
	return CommonFilters->GetFilterDisplayNames();
}

TArray<FText> FNavigationToolFilterBar::GetCustomTextFilterNames() const
{
	TArray<FText> OutLabels;

	for (const TSharedRef<FNavigationToolFilter_CustomText>& CustomTextFilter : CustomTextFilters)
	{
		const FCustomTextFilterData TextFilterData = CustomTextFilter->CreateCustomTextFilterData();
		OutLabels.Add(TextFilterData.FilterLabel);
	}

	return OutLabels;
}

int32 FNavigationToolFilterBar::GetTotalDisplayNodeCount() const
{
	return FilterData.GetTotalNodeCount();
}

int32 FNavigationToolFilterBar::GetFilteredDisplayNodeCount() const
{
	return FilterData.GetDisplayNodeCount();
}

TArray<TSharedRef<FNavigationToolFilter>> FNavigationToolFilterBar::GetCommonFilters(const TArray<TSharedRef<FFilterCategory>>& InCategories) const
{
	return CommonFilters->GetAllFilters(InCategories);
}

bool FNavigationToolFilterBar::AnyCustomTextFilterActive() const
{
	for (const TSharedRef<FNavigationToolFilter_CustomText>& Filter : CustomTextFilters)
	{
		if (IsFilterActive(Filter))
		{
			return true;
		}
	}

	return false;
}

bool FNavigationToolFilterBar::HasEnabledCustomTextFilters() const
{
	for (const TSharedRef<FNavigationToolFilter_CustomText>& Filter : CustomTextFilters)
	{
		if (IsFilterEnabled(Filter))
		{
			return true;
		}
	}
	return false;
}

TArray<TSharedRef<FNavigationToolFilter_CustomText>> FNavigationToolFilterBar::GetAllCustomTextFilters() const
{
	return CustomTextFilters;
}

bool FNavigationToolFilterBar::AddCustomTextFilter(const TSharedRef<FNavigationToolFilter_CustomText>& InFilter, const bool bInAddToConfig)
{
	if (CustomTextFilters.Add(InFilter) != 1)
	{
		return false;
	}

	if (bInAddToConfig)
	{
		if (UNavigationToolSettings* const ToolSettings = GetMutableDefault<UNavigationToolSettings>())
		{
			FSequencerFilterBarConfig& Config = ToolSettings->FindOrAddFilterBar(GetIdentifier(), /*bInSaveConfig=*/false);

			if (Config.AddCustomTextFilter(InFilter->CreateCustomTextFilterData()))
			{
				ToolSettings->SaveConfig();
			}
		}
	}

	BroadcastFiltersChanged(InFilter, ENavigationToolFilterChange::Activate);

	return true;
}

bool FNavigationToolFilterBar::RemoveCustomTextFilter(const TSharedRef<FNavigationToolFilter_CustomText>& InFilter, const bool bInRemoveFromConfig)
{
	if (CustomTextFilters.Remove(InFilter) != 1)
	{
		return false;
	}

	if (bInRemoveFromConfig)
	{
		if (UNavigationToolSettings* const ToolSettings = GetMutableDefault<UNavigationToolSettings>())
		{
			FSequencerFilterBarConfig& Config = ToolSettings->FindOrAddFilterBar(GetIdentifier(), /*bInSaveConfig=*/false);

			const FString FilterName = InFilter->GetDisplayName().ToString();
			if (Config.RemoveCustomTextFilter(FilterName))
			{
				ToolSettings->SaveConfig();
			}
		}
	}

	BroadcastFiltersChanged(InFilter, ENavigationToolFilterChange::Disable);

	return true;
}

void FNavigationToolFilterBar::ActivateCustomTextFilters(const bool bInActivate, const TArray<TSharedRef<FNavigationToolFilter_CustomText>> InExceptions)
{
	UNavigationToolSettings* const ToolSettings = GetMutableDefault<UNavigationToolSettings>();
	if (!ToolSettings)
	{
		return;
	}

	FSequencerFilterBarConfig& Config = ToolSettings->FindOrAddFilterBar(GetIdentifier(), /*bInSaveConfig=*/false);

	bool bNeedsSave = false;

	for (const TSharedRef<FNavigationToolFilter_CustomText>& CustomTextFilter : CustomTextFilters)
	{
		if (InExceptions.IsEmpty() || !InExceptions.Contains(CustomTextFilter))
		{
			const FString FilterName = CustomTextFilter->GetDisplayName().ToString();
			if (Config.SetFilterActive(FilterName, bInActivate))
			{
				if (!bInActivate && IsFilterActive(CustomTextFilter))
				{
					CustomTextFilter->SetActive(false);
					CustomTextFilter->ActiveStateChanged(false);
				}

				const ENavigationToolFilterChange FilterChangeType = bInActivate
					? ENavigationToolFilterChange::Activate : ENavigationToolFilterChange::Deactivate;
				BroadcastFiltersChanged(CustomTextFilter, FilterChangeType);

				bNeedsSave = true;
			}
		}
	}

	if (bNeedsSave)
	{
		ToolSettings->SaveConfig();
	}

	RequestFilterUpdate();
}

void FNavigationToolFilterBar::EnableCustomTextFilters(const bool bInEnable, const TArray<TSharedRef<FNavigationToolFilter_CustomText>> InExceptions)
{
	UNavigationToolSettings* const ToolSettings = GetMutableDefault<UNavigationToolSettings>();
	if (!ToolSettings)
	{
		return;
	}

	FSequencerFilterBarConfig& Config = ToolSettings->FindOrAddFilterBar(GetIdentifier(), /*bInSaveConfig=*/false);

	bool bNeedsSave = false;

	for (const TSharedRef<FNavigationToolFilter_CustomText>& CustomTextFilter : CustomTextFilters)
	{
		if (InExceptions.IsEmpty() || !InExceptions.Contains(CustomTextFilter))
		{
			const FString FilterName = CustomTextFilter->GetDisplayName().ToString();
			if (Config.SetFilterEnabled(FilterName, bInEnable))
			{
				if (!bInEnable && IsFilterActive(CustomTextFilter))
				{
					CustomTextFilter->SetActive(false);
					CustomTextFilter->ActiveStateChanged(false);
				}

				const ENavigationToolFilterChange FilterChangeType = bInEnable
					? ENavigationToolFilterChange::Enable : ENavigationToolFilterChange::Disable;
				BroadcastFiltersChanged(CustomTextFilter, FilterChangeType);

				bNeedsSave = true;
			}
		}
	}

	if (bNeedsSave)
	{
		ToolSettings->SaveConfig();
	}

	RequestFilterUpdate();
}

TArray<TSharedRef<FNavigationToolFilter_CustomText>> FNavigationToolFilterBar::GetEnabledCustomTextFilters() const
{
	TArray<TSharedRef<FNavigationToolFilter_CustomText>> OutFilters;

	for (const TSharedRef<FNavigationToolFilter_CustomText>& CustomTextFilter : CustomTextFilters)
	{
		if (IsFilterEnabled(CustomTextFilter))
		{
			OutFilters.Add(CustomTextFilter);
		}
	}

	return OutFilters;
}

TSet<TSharedRef<FFilterCategory>> FNavigationToolFilterBar::GetFilterCategories(const TSet<TSharedRef<FNavigationToolFilter>>* InFilters) const
{
	return CommonFilters->GetCategories(InFilters);
}

TSet<TSharedRef<FFilterCategory>> FNavigationToolFilterBar::GetConfigCategories() const
{
	return { ClassTypeCategory, ComponentTypeCategory, MiscCategory };
}

TSharedRef<FFilterCategory> FNavigationToolFilterBar::GetClassTypeCategory() const
{
	return ClassTypeCategory;
}

TSharedRef<FFilterCategory> FNavigationToolFilterBar::GetComponentTypeCategory() const
{
	return ComponentTypeCategory;
}

TSharedRef<FFilterCategory> FNavigationToolFilterBar::GetMiscCategory() const
{
	return MiscCategory;
}

bool FNavigationToolFilterBar::PassesAnyCommonFilter(const FNavigationToolViewModelPtr& InNode)
{
	bool bPassedAnyFilters = false;
	bool bAnyFilterActive = false;

	// Only one common filter needs to pass for this node to be included in the filtered set
	CommonFilters->ForEachFilter([this, &InNode, &bPassedAnyFilters, &bAnyFilterActive]
		(const TSharedRef<FNavigationToolFilter>& InFilter)
		{
			if (IsFilterActive(InFilter))
			{
				bAnyFilterActive = true;
				
				if (InFilter->PassesFilter(InNode))
				{
					bPassedAnyFilters = true;
					return false; // Stop processing filters
				}
			}

			return true;
		});

	if (!bAnyFilterActive)
	{
		return true;
	}

	return bPassedAnyFilters;
}

bool FNavigationToolFilterBar::PassesAllCustomTextFilters(const FNavigationToolViewModelPtr& InNode)
{
	for (const TSharedRef<FNavigationToolFilter_CustomText>& Filter : CustomTextFilters)
	{
		if (IsFilterActive(Filter))
		{
			if (!Filter->PassesFilter(InNode))
			{
				return false;
			}
		}
	}

	return true;
}

const FNavigationToolFilterData& FNavigationToolFilterBar::FilterNodes()
{
	//SCOPE_LOG_TIME_IN_SECONDS(TEXT("FNavigationToolFilterBar::FilterNodes()"), nullptr);

	const TSharedPtr<INavigationToolView> ToolView = Tool.GetMostRecentToolView();
	if (!ToolView.IsValid())
	{
		return FilterData;
	}

	const FNavigationToolViewModelPtr RootItem = Tool.GetTreeRoot().Pin();
    if (!RootItem.IsValid())
    {
    	return FilterData;
    }

	const UNavigationToolSettings* const ToolSettings = GetDefault<UNavigationToolSettings>();
	if (!ToolSettings)
	{
		return FilterData;
	}

	// Reset all filter data
	FilterData.Reset();

	// Loop through all nodes and filter recursively
	const bool bHasActiveFilter = HasAnyFilterActive();
	for (const FNavigationToolViewModelWeakPtr& RootNode : RootItem->GetChildren())
	{
		FilterNodesRecursive(*ToolView, *ToolSettings, bHasActiveFilter, RootNode);
	}

	return FilterData;
}

FNavigationToolFilterData& FNavigationToolFilterBar::GetFilterData()
{
	return FilterData;
}

const FTextFilterExpressionEvaluator& FNavigationToolFilterBar::GetTextFilterExpressionEvaluator() const
{
	return GetTextFilter()->GetTextFilterExpressionEvaluator();
}

TArray<TSharedRef<ISequencerTextFilterExpressionContext>> FNavigationToolFilterBar::GetTextFilterExpressionContexts() const
{
	return TextFilter->GetTextFilterExpressionContexts();
}

FString FNavigationToolFilterBar::GenerateTextFilterStringFromEnabledFilters() const
{
	TArray<TSharedRef<FNavigationToolFilter>> FiltersToSave;

	FiltersToSave.Append(GetCommonFilters());

	for (const TSharedRef<FNavigationToolFilter_CustomText>& Filter : CustomTextFilters)
	{
		FiltersToSave.Add(Filter);
	}

	FString GeneratedFilterString = TextFilter->GetRawFilterText().ToString();

	for (const TSharedRef<FNavigationToolFilter>& Filter : FiltersToSave)
	{
		if (IsFilterActive(Filter) && IsFilterEnabled(Filter))
		{
			const FString AndAddString = GeneratedFilterString.IsEmpty() ? TEXT("") : TEXT(" AND ");
			const FString ThisFilterGeneratedString = FString::Format(TEXT("{0}{1}==TRUE"), { AndAddString, *Filter->GetName() });
			GeneratedFilterString.Append(ThisFilterGeneratedString);
		}
	}

	return GeneratedFilterString;
}

bool FNavigationToolFilterBar::FilterNodesRecursive(INavigationToolView& InToolView
	, const UNavigationToolSettings& InSettings
	, const bool bInHasActiveFilter
	, const FNavigationToolViewModelWeakPtr& InWeakStartNode)
{
	const FNavigationToolViewModelPtr StartNode = InWeakStartNode.Pin();
	if (!StartNode.IsValid())
	{
		return false;
	}

	/**
	 * Main Filtering Logic
	 *
	 * - Pinning overrides all other filters
	 * - Hidden/Isolated items will take precedence over common filters
	 * - Can hide sub items of isolated items
	 */

	bool bAnyChildPassed = false;

	// Child nodes should always be processed, as they may force their parents to pass
	for (const FNavigationToolViewModelWeakPtr& WeakNode : StartNode->GetChildren())
	{
		if (FilterNodesRecursive(InToolView, InSettings, bInHasActiveFilter, WeakNode))
		{
			bAnyChildPassed = true;
		}
	}

	// Increment the total node count so we can remove the code to loop again just to count
	FilterData.IncrementTotalNodeCount();

	// Early out if no filter
	if (!bInHasActiveFilter)
	{
		FilterData.FilterInNode(StartNode);
		return false;
	}

	const bool bPassedTextFilter = !TextFilter->IsActive() || TextFilter->PassesFilter(StartNode);
	const bool bPassedAnyCommonFilters = PassesAnyCommonFilter(StartNode);
	const bool bPassedAnyCustomTextFilters = PassesAllCustomTextFilters(StartNode);

	const bool bAllFiltersPassed = bPassedTextFilter
		&& bPassedAnyCommonFilters
		&& bPassedAnyCustomTextFilters;

	if (bAllFiltersPassed || bAnyChildPassed)
	{
		if (InSettings.ShouldAutoExpandNodesOnFilterPass())
		{
			InToolView.SetParentItemExpansions(InWeakStartNode, true);
		}

		FilterData.FilterInNodeWithAncestors(InWeakStartNode);
		return true;
	}

	// After child nodes are processed, fail anything that didn't pass
	FilterData.FilterOutNode(InWeakStartNode);
	return false;
}

TArray<TSharedRef<FNavigationToolFilter>> FNavigationToolFilterBar::GetFilterList(const bool bInIncludeCustomTextFilters) const
{
	TArray<TSharedRef<FNavigationToolFilter>> AllFilters;

	AllFilters.Append(CommonFilters->GetAllFilters());

	AllFilters.Add(TextFilter);

	if (bInIncludeCustomTextFilters)
	{
		for (const TSharedRef<FNavigationToolFilter_CustomText>& Filter : CustomTextFilters)
		{
			AllFilters.Add(Filter);
		}
	}

	return AllFilters;
}

bool FNavigationToolFilterBar::ShouldUpdateOnTrackValueChanged() const
{
	if (bFiltersMuted)
	{
		return false;
	}

	const TArray<TSharedRef<FNavigationToolFilter>> AllFilters = GetFilterList();

	for (const TSharedRef<FNavigationToolFilter>& Filter : AllFilters)
	{
		if (IsFilterActive(Filter))
		{
			return true;
		}
	}

	return false;
}

TSharedRef<SComboButton> FNavigationToolFilterBar::MakeAddFilterButton()
{
	const TSharedPtr<SLayeredImage> FilterImage = SNew(SLayeredImage)
		.Image(FAppStyle::Get().GetBrush(TEXT("Icons.Filter")))
		.ColorAndOpacity_Lambda([this]()
			{
				return AreFiltersMuted() ? FLinearColor(1.f, 1.f, 1.f, 0.2f) : FSlateColor::UseForeground();
			});

	// Badge the filter icon if there are filters enabled or active
	FilterImage->AddLayer(TAttribute<const FSlateBrush*>::CreateLambda([this]() -> const FSlateBrush*
		{
			if (AreFiltersMuted() || !HasAnyFilterEnabled())
			{
				return nullptr;
			}

			if (HasAnyFilterActive(false, false))
			{
				return FAppStyle::Get().GetBrush(TEXT("Icons.BadgeModified"));
			}

			return FAppStyle::Get().GetBrush(TEXT("Icons.Badge"));
		}));

	const TSharedRef<SComboButton> ComboButton = SNew(SComboButton)
		.ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>(TEXT("SimpleComboButtonWithIcon")))
		.ForegroundColor(FSlateColor::UseStyle())
		.ToolTipText_Lambda([this]()
			{
				return LOCTEXT("AddFilterToolTip", "Open the Add Filter Menu to add or manage filters\n\n"
					"Shift + Click to temporarily mute all active filters");
			})
		.OnComboBoxOpened_Lambda([this]()
			{
				// Don't allow opening the menu if filters are muted or we are toggling the filter mute state
				if (AreFiltersMuted() || FSlateApplication::Get().GetModifierKeys().IsShiftDown())
				{
					FSlateApplication::Get().DismissAllMenus();
				}
			})
		.OnGetMenuContent_Lambda([this]() -> TSharedRef<SWidget>
			{
				if (FSlateApplication::Get().GetModifierKeys().IsShiftDown())
				{
					MuteFilters(!AreFiltersMuted());
					FSlateApplication::Get().DismissAllMenus();
					return SNullWidget::NullWidget;
				}
				return FilterMenu->CreateMenu(SharedThis(this));
			})
		.ContentPadding(FMargin(1, 0))
		.ButtonContent()
		[
			FilterImage.ToSharedRef()
		];
	ComboButton->AddMetadata(MakeShared<FTagMetaData>(TEXT("NavigationToolFiltersCombo")));

	return ComboButton;
}

bool FNavigationToolFilterBar::ShouldShowFilterBarWidget() const
{
	if (const UNavigationToolSettings* const ToolSettings = GetDefault<UNavigationToolSettings>())
	{
		return ToolSettings->IsFilterBarVisible() && HasAnyFiltersEnabled();
	}
	return false;
}

bool FNavigationToolFilterBar::IsFilterBarVisible() const
{
	if (const UNavigationToolSettings* const ToolSettings = GetDefault<UNavigationToolSettings>())
	{
		return ToolSettings->IsFilterBarVisible();
	}
	return false;
}

void FNavigationToolFilterBar::ToggleFilterBarVisibility()
{
	UNavigationToolSettings* const ToolSettings = GetMutableDefault<UNavigationToolSettings>();
	if (!ToolSettings)
	{
		return;
	}

	const bool bNewVisibility = !ToolSettings->IsFilterBarVisible();
	ToolSettings->SetFilterBarVisible(bNewVisibility);
	VisibilityChangedEvent.Broadcast(bNewVisibility);
}

bool FNavigationToolFilterBar::IsFilterBarLayout(const EFilterBarLayout InLayout) const
{
	if (const UNavigationToolSettings* const ToolSettings = GetDefault<UNavigationToolSettings>())
	{
		return (ToolSettings->GetFilterBarLayout() == InLayout);
	}
	return false;
}

void FNavigationToolFilterBar::SetToVerticalLayout()
{
	UNavigationToolSettings* const ToolSettings = GetMutableDefault<UNavigationToolSettings>();
	if (!ToolSettings)
	{
		return;
	}

	ToolSettings->SetFilterBarLayout(EFilterBarLayout::Vertical);
	LayoutChangedEvent.Broadcast(ToolSettings->GetFilterBarLayout());
}

void FNavigationToolFilterBar::SetToHorizontalLayout()
{
	UNavigationToolSettings* const ToolSettings = GetMutableDefault<UNavigationToolSettings>();
	if (!ToolSettings)
	{
		return;
	}

	ToolSettings->SetFilterBarLayout(EFilterBarLayout::Horizontal);
	LayoutChangedEvent.Broadcast(ToolSettings->GetFilterBarLayout());
}

void FNavigationToolFilterBar::ToggleFilterBarLayout()
{
	if (IsFilterBarLayout(EFilterBarLayout::Horizontal))
	{
		SetToVerticalLayout();
	}
	else if (IsFilterBarLayout(EFilterBarLayout::Vertical))
	{
		SetToHorizontalLayout();
	}
}

void FNavigationToolFilterBar::CreateWindow_AddCustomTextFilter(const FCustomTextFilterData& InCustomTextFilterData)
{
	if (SNavigationToolCustomTextFilterDialog::DialogInstance.IsValid()
		&& SNavigationToolCustomTextFilterDialog::DialogInstance->IsVisible())
	{
		SNavigationToolCustomTextFilterDialog::DialogInstance->BringToFront();
		return;
	}

	SNavigationToolCustomTextFilterDialog::DialogInstance = SNew(SNavigationToolCustomTextFilterDialog)
		.CustomTextFilterData(InCustomTextFilterData)
		.OnTryCreateFilter(this, &FNavigationToolFilterBar::TryCreateCustomTextFilter);

	SNavigationToolCustomTextFilterDialog::ShowWindow(SNavigationToolCustomTextFilterDialog::DialogInstance.ToSharedRef(), true);
}

void FNavigationToolFilterBar::CreateWindow_EditCustomTextFilter(const TSharedPtr<FNavigationToolFilter_CustomText>& InCustomTextFilter)
{
	if (SNavigationToolCustomTextFilterDialog::DialogInstance.IsValid()
		&& SNavigationToolCustomTextFilterDialog::DialogInstance->IsVisible())
	{
		SNavigationToolCustomTextFilterDialog::DialogInstance->BringToFront();
		return;
	}

	SNavigationToolCustomTextFilterDialog::DialogInstance = SNew(SNavigationToolCustomTextFilterDialog)
		.CustomTextFilterData(InCustomTextFilter->CreateCustomTextFilterData())
		.OnTryModifyFilter(this, &FNavigationToolFilterBar::TryModifyCustomTextFilter)
		.OnTryDeleteFilter(this, &FNavigationToolFilterBar::TryDeleteCustomTextFilter);

	SNavigationToolCustomTextFilterDialog::ShowWindow(SNavigationToolCustomTextFilterDialog::DialogInstance.ToSharedRef(), true);
}

bool FNavigationToolFilterBar::CheckFilterNameValidity(const FString& InNewFilterName, const FString& InOldFilterName, const bool bInIsEdit, FText& OutErrorText) const
{
	UNavigationToolSettings* const ToolSettings = GetMutableDefault<UNavigationToolSettings>();
	if (!ToolSettings)
	{
		return false;
	}

	if (InNewFilterName.IsEmpty())
	{
		OutErrorText = LOCTEXT("EmptyFilterLabelError", "Filter Label cannot be empty");
		return false;
	}

	FSequencerFilterBarConfig& Config = ToolSettings->FindOrAddFilterBar(GetIdentifier(), true);

	const TArray<FCustomTextFilterData> CustomTextFilterDatas = Config.GetCustomTextFilters();

	// Check for duplicate filter labels
	for (const FCustomTextFilterData& Data : CustomTextFilterDatas)
	{
		/* Special Case: If we are editing a filter and don't change the filter label, it will be considered a duplicate of itself!
		 * To prevent this we check against the original filter label if we are in edit mode */
		if (Data.FilterLabel.ToString().Equals(InNewFilterName)
			&& !(bInIsEdit && Data.FilterLabel.ToString().Equals(InOldFilterName)))
		{
			OutErrorText = LOCTEXT("DuplicateFilterLabelError", "A filter with this label already exists!");
			return false;
		}
	}

	return true;
}

bool FNavigationToolFilterBar::TryCreateCustomTextFilter(const FCustomTextFilterData& InNewFilterData, const FString& InOldFilterName, const bool bInApply, FText& OutErrorText)
{
	UNavigationToolSettings* const ToolSettings = GetMutableDefault<UNavigationToolSettings>();
	if (!ToolSettings)
	{
		return false;
	}

	if (!CheckFilterNameValidity(InNewFilterData.FilterLabel.ToString(), InOldFilterName, false, OutErrorText))
	{
		return false;
	}

	const TSharedRef<FNavigationToolFilter_CustomText> NewCustomTextFilter = MakeShared<FNavigationToolFilter_CustomText>(*this);
	NewCustomTextFilter->SetFromCustomTextFilterData(InNewFilterData);

	const TSharedPtr<FNavigationToolFilter> NewFilter = StaticCastSharedPtr<FNavigationToolFilter>(NewCustomTextFilter->GetFilter());
	if (!NewFilter.IsValid())
	{
		return false;
	}

	FSequencerFilterBarConfig& Config = ToolSettings->FindOrAddFilterBar(GetIdentifier(), /*bInSaveConfig=*/false);
	Config.AddCustomTextFilter(InNewFilterData);

	AddCustomTextFilter(NewCustomTextFilter, false);

	if (bInApply)
	{
		SetTextFilterString(TEXT(""));
		SetFilterActive(NewFilter.ToSharedRef(), true, true);
	}
	else
	{
		SetFilterEnabled(NewFilter.ToSharedRef(), true, true);
	}

	return true;
}

bool FNavigationToolFilterBar::TryDeleteCustomTextFilter(const FString& InFilterName, FText& OutErrorText)
{
	UNavigationToolSettings* const ToolSettings = GetMutableDefault<UNavigationToolSettings>();
	if (!ToolSettings)
	{
		return false;
	}

	FSequencerFilterBarConfig& Config = ToolSettings->FindOrAddFilterBar(GetIdentifier(), /*bInSaveConfig=*/false);
	Config.RemoveCustomTextFilter(InFilterName);

	if (const TSharedPtr<FNavigationToolFilter_CustomText> CustomTextFilter = FindCustomTextFilterByDisplayName(InFilterName))
	{
		RemoveCustomTextFilter(CustomTextFilter.ToSharedRef(), true);
	}

	return true;
}

void FNavigationToolFilterBar::HandleTextFilterChanged()
{
	TextFilterTextChangedEvent.Broadcast();
	RequestFilterUpdate();
}

bool FNavigationToolFilterBar::TryModifyCustomTextFilter(const FCustomTextFilterData& InFilterData, const FString& InOldFilterName, FText& OutErrorText)
{
	UNavigationToolSettings* const ToolSettings = GetMutableDefault<UNavigationToolSettings>();
	if (!ToolSettings)
	{
		return false;
	}

	const FString NewFilterName = InFilterData.FilterLabel.ToString();

	if (!CheckFilterNameValidity(NewFilterName, InOldFilterName, true, OutErrorText))
	{
		return false;
	}

	const TSharedPtr<FNavigationToolFilter_CustomText> CustomTextFilter = FindCustomTextFilterByDisplayName(InOldFilterName);
	if (!CustomTextFilter.IsValid())
	{
		return false;
	}

	const TSharedRef<FNavigationToolFilter_CustomText> CustomTextFilterRef = CustomTextFilter.ToSharedRef();
	const bool bWasFilterEnabled = IsFilterEnabled(CustomTextFilterRef);
	const bool bWasFilterActive = IsFilterActive(CustomTextFilterRef);

	FSequencerFilterBarConfig& Config = ToolSettings->FindOrAddFilterBar(GetIdentifier(), /*bInSaveConfig=*/false);
	Config.RemoveCustomTextFilter(InOldFilterName);

	if (CustomTextFilter.IsValid())
	{
		CustomTextFilter->SetFromCustomTextFilterData(InFilterData);
	}

	Config.AddCustomTextFilter(InFilterData);

	RemoveCustomTextFilter(CustomTextFilterRef, false);
	AddCustomTextFilter(CustomTextFilterRef, false);

	if (bWasFilterActive)
	{
		SetFilterActiveByDisplayName(NewFilterName, true, true);
	}
	else if (bWasFilterEnabled)
	{
		SetFilterEnabledByDisplayName(NewFilterName, true, true);
	}
	else
	{
		ToolSettings->SaveConfig();
	}

	return true;
}

FCustomTextFilterData FNavigationToolFilterBar::DefaultNewCustomTextFilterData(const FText& InFilterString)
{
	FCustomTextFilterData CustomTextFilterData;
	CustomTextFilterData.FilterLabel = LOCTEXT("NewFilterName", "New Filter Name");
	CustomTextFilterData.FilterString = InFilterString;
	return MoveTemp(CustomTextFilterData);
}

void FNavigationToolFilterBar::OnSearchTextChanged(const FText& InFilterText)
{
	TextFilter->SetRawFilterText(InFilterText);

	RequestFilterUpdate();
}

void FNavigationToolFilterBar::OnSearchTextCommitted(const FText& InFilterText, const ETextCommit::Type InCommitType)
{
	TextFilter->SetRawFilterText(InFilterText);

	RequestFilterUpdate();
}

void FNavigationToolFilterBar::OnSearchTextSaved(const FText& InFilterText)
{
	FCustomTextFilterData CustomTextFilterData;
	CustomTextFilterData.FilterString = InFilterText;
	CreateWindow_AddCustomTextFilter(CustomTextFilterData);
}

void FNavigationToolFilterBar::BroadcastFiltersChanged(const TSharedRef<FNavigationToolFilter>& InFilter, const ENavigationToolFilterChange InChangeType)
{
	FiltersChangedEvent.Broadcast(InChangeType, InFilter);
}

bool FNavigationToolFilterBar::IsFilterSupported(const TSharedRef<FNavigationToolFilter>& InFilter) const
{
	const TSharedPtr<ISequencer> Sequencer = Tool.GetSequencer();
	if (!Sequencer.IsValid())
	{
		return false;
	}

	UMovieSceneSequence* const RootMovieSceneSequence = Sequencer->GetRootMovieSceneSequence();
	if (!RootMovieSceneSequence)
	{
		return false;
	}

	const FString FilterName = InFilter->GetName();
	const bool bFilterSupportsSequence = InFilter->SupportsSequence(RootMovieSceneSequence);
	const bool bSequenceSupportsFilter = RootMovieSceneSequence->IsFilterSupported(FilterName);
	return bFilterSupportsSequence || bSequenceSupportsFilter;
}

bool FNavigationToolFilterBar::IsFilterSupported(const FString& InFilterName) const
{
	const TArray<TSharedRef<FNavigationToolFilter>> FilterList = GetFilterList();
	const TSharedRef<FNavigationToolFilter>* const FoundFilter = FilterList.FindByPredicate(
		[InFilterName](const TSharedRef<FNavigationToolFilter>& InFilter)
		{
			return InFilter->GetName().Equals(InFilterName, ESearchCase::IgnoreCase);
		});
	return FoundFilter ? IsFilterSupported(*FoundFilter) : false;
}

void FNavigationToolFilterBar::OpenTextExpressionHelp()
{
	FFilterExpressionHelpDialogConfig Config;
	Config.IdentifierName = TEXT("NavigationToolCustomTextFilterHelp");
	Config.DialogTitle = LOCTEXT("CustomTextFilterHelp", "Sequence Navigator Custom Text Filter Help");
	Config.AddCommonFilterHelpEntries<FNavigationToolFilter>(SharedThis(this));

	SFilterExpressionHelpDialog::Open(MoveTemp(Config));
}

void FNavigationToolFilterBar::SaveCurrentFilterSetAsCustomTextFilter()
{
	const FText NewFilterString = FText::FromString(GenerateTextFilterStringFromEnabledFilters());
	CreateWindow_AddCustomTextFilter(DefaultNewCustomTextFilterData(NewFilterString));
}

void FNavigationToolFilterBar::CreateNewTextFilter()
{
	const FText NewFilterString = FText::FromString(GetTextFilterString());
	CreateWindow_AddCustomTextFilter(DefaultNewCustomTextFilterData(NewFilterString));
}

bool FNavigationToolFilterBar::DoesCommonFilterPass(const TViewModelPtr<FViewModel>& InViewModel, const FString& InFilterName) const
{
	bool bAnyPassed = false;

	CommonFilters->ForEachFilter([this, &bAnyPassed, &InViewModel, &InFilterName]
		(const TSharedRef<FNavigationToolFilter>& InFilter)
		{
			if (InFilter->GetName().Equals(InFilterName, ESearchCase::IgnoreCase)
				&& InFilter->PassesFilter(InViewModel.ImplicitCast()))
			{
				bAnyPassed = true;
				return false; // Stop looping
			}
			return true;
		});

	return bAnyPassed;
}

void FNavigationToolFilterBar::GetCommonFilterKeySuggestions(TArray<FSequencerFilterSuggestion>& OutSuggestions) const
{
	for (const TSharedRef<FNavigationToolFilter>& CommonFilter : CommonFilters->GetAllFilters())
	{
		FSequencerFilterSuggestion NewSuggestion;
		NewSuggestion.Suggestion = CommonFilter->GetName();
		NewSuggestion.DisplayName = CommonFilter->GetDisplayName();
		NewSuggestion.Description = CommonFilter->GetDefaultToolTipText();

		OutSuggestions.AddUnique(NewSuggestion);
	}
}

void FNavigationToolFilterBar::GetCommonFilterValueSuggestions(const FString& InKey, TArray<FSequencerFilterSuggestion>& OutSuggestions) const
{
	for (const TSharedRef<FNavigationToolFilter>& CommonFilter : CommonFilters->GetAllFilters())
	{
		if (InKey.Equals(CommonFilter->GetName(), ESearchCase::IgnoreCase))
		{
			// Suggestions for common filters text expressions will always be boolean for turning on/off
			FSequencerFilterSuggestion::MakeBooleanSuggestions(OutSuggestions);
			break;
		}
	}
}

bool FNavigationToolFilterBar::IsCommonFilterName(const FString& InFilterName) const
{
	bool bFoundFilter = false;

	CommonFilters->ForEachFilter([&InFilterName, &bFoundFilter](const TSharedRef<FNavigationToolFilter>& InFilter)
		{
			bFoundFilter = InFilter->GetName().Equals(InFilterName, ESearchCase::IgnoreCase);
			const bool bContinue = !bFoundFilter;
			return bContinue;
		});

	return bFoundFilter;
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
