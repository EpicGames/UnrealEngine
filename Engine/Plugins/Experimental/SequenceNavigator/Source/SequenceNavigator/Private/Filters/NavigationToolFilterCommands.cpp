// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavigationToolFilterCommands.h"

#define LOCTEXT_NAMESPACE "NavigationToolFilterCommands"

void FNavigationToolFilterCommands::RegisterCommands()
{
	UI_COMMAND(ToggleFilterBarVisibility
		, "Filter Bar Visibility"
		, "Toggle the visibility of the filter bar. The filter bar will only be displayed when there are enabled filters"
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(SetToVerticalLayout
		, "Vertical"
		, "Swap to a vertical layout for the filter bar"
		, EUserInterfaceActionType::RadioButton
		, FInputChord());

	UI_COMMAND(SetToHorizontalLayout
		, "Horizontal"
		, "Swap to a horizontal layout for the filter bar"
		, EUserInterfaceActionType::RadioButton
		, FInputChord());

	FUICommandInfo::MakeCommandInfo(AsShared()
		, ResetFilters
		, TEXT("ResetFilters")
		, LOCTEXT("ResetFilters", "Reset Filters")
		, LOCTEXT("ResetFiltersTooltip", "Reset all enabled filters")
		, FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("PropertyWindow.DiffersFromDefault"))
		, EUserInterfaceActionType::Button
		, FInputChord());

	FUICommandInfo::MakeCommandInfo(AsShared()
		, ToggleMuteFilters
		, TEXT("ToggleMuteFilters")
		, LOCTEXT("ToggleMuteFilters", "Mute Filters")
		, LOCTEXT("ToggleMuteFiltersTooltip", "Mute or Unmute all active filters")
		, FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("Denied"))
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	FUICommandInfo::MakeCommandInfo(AsShared()
		, DisableAllFilters
		, TEXT("DisableAllFilters")
		, LOCTEXT("DisableAllFilters", "Remove All Filters")
		, LOCTEXT("DisableAllFiltersTooltip", "Disables all filters and removes them from the filter bar")
		, FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("Icons.Minus"))
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(ToggleActivateEnabledFilters
		, "Toggle Active State"
		, "Activates or deactivates all enabled filters"
		, EUserInterfaceActionType::Button
		, FInputChord());

	FUICommandInfo::MakeCommandInfo(AsShared()
		, ActivateAllFilters
		, TEXT("ActivateAllFilters")
		, LOCTEXT("ActivateAllFilters", "Activate All Filters")
		, LOCTEXT("ActivateAllFiltersTooltip", "Activates all enabled filters.")
		, FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.Plus"))
		, EUserInterfaceActionType::Button
		, FInputChord());

	FUICommandInfo::MakeCommandInfo(AsShared()
		, DeactivateAllFilters
		, TEXT("DeactivateAllFilters")
		, LOCTEXT("DeactivateAllFilters", "Deactivate All Filters")
		, LOCTEXT("DeactivateAllFiltersTooltip", "Deactivates all enabled filters.")
		, FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.Minus"))
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(ToggleFilter_Sequence
		, "Toggle Sequence Filter"
		, "Toggle the filter for Sequence items"
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(ToggleFilter_Track
		, "Toggle Track Filter"
		, "Toggle the filter for Track items"
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(ToggleFilter_Binding
		, "Toggle Binding Filter"
		, "Toggle the filter for Binding items"
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(ToggleFilter_Marker
		, "Toggle Marker Filter"
		, "Toggle the filter for Marker items"
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(ToggleFilter_Unbound
		, "Toggle Unbound Filter"
		, "Toggle the filter for displaying items that contain unbound tracks"
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(ToggleFilter_Marks
		, "Toggle Marks Filter"
		, "Toggle the filter for displaying items that contain marked frames"
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(ToggleFilter_Playhead
		, "Toggle Playhead Filter"
		, "Toggle the filter for displaying items whose range contains the current playhead location"
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(ToggleFilter_Dirty
		, "Toggle Dirty Filter"
		, "Toggle the filter for displaying items whose package is marked dirty"
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());
}

#undef LOCTEXT_NAMESPACE
