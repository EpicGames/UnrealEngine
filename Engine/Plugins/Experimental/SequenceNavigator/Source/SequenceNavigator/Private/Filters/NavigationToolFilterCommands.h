// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "NavigationToolFilterCommands"

class FNavigationToolFilterCommands
	: public TCommands<FNavigationToolFilterCommands>
{
public:
	FNavigationToolFilterCommands()
		: TCommands<FNavigationToolFilterCommands>(TEXT("SequenceNavigationToolFilters")
		, LOCTEXT("SequenceNavigatorFilters", "Sequence Navigator Filters")
		, NAME_None
		, FAppStyle::GetAppStyleSetName())
	{}

	virtual void RegisterCommands() override;

	TSharedPtr<FUICommandInfo> ToggleFilterBarVisibility;

	TSharedPtr<FUICommandInfo> SetToVerticalLayout;
	TSharedPtr<FUICommandInfo> SetToHorizontalLayout;

	TSharedPtr<FUICommandInfo> ResetFilters;

	TSharedPtr<FUICommandInfo> ToggleMuteFilters;

	TSharedPtr<FUICommandInfo> DisableAllFilters;

	TSharedPtr<FUICommandInfo> ToggleActivateEnabledFilters;

	TSharedPtr<FUICommandInfo> ActivateAllFilters;
	TSharedPtr<FUICommandInfo> DeactivateAllFilters;

	// Global Filters
	TSharedPtr<FUICommandInfo> ToggleFilter_Sequence;
	TSharedPtr<FUICommandInfo> ToggleFilter_Track;
	TSharedPtr<FUICommandInfo> ToggleFilter_Binding;
	TSharedPtr<FUICommandInfo> ToggleFilter_Marker;

	// Normal Filters
	TSharedPtr<FUICommandInfo> ToggleFilter_Unbound;
	TSharedPtr<FUICommandInfo> ToggleFilter_Marks;
	TSharedPtr<FUICommandInfo> ToggleFilter_Playhead;
	TSharedPtr<FUICommandInfo> ToggleFilter_Dirty;
};

#undef LOCTEXT_NAMESPACE
