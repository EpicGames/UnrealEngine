// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/NavigationToolFilterData.h"
#include "Items/NavigationToolItem.h"

namespace UE::SequenceNavigator
{

using namespace Sequencer;

FNavigationToolFilterData::FNavigationToolFilterData(const FString& InRawFilterText)
	: RawFilterText(InRawFilterText)
{
}

bool FNavigationToolFilterData::operator==(const FNavigationToolFilterData& InRhs) const
{
	return GetTotalNodeCount() == InRhs.GetTotalNodeCount()
		&& GetDisplayNodeCount() == InRhs.GetDisplayNodeCount()
		&& ContainsFilterInNodes(InRhs);
}

bool FNavigationToolFilterData::operator!=(const FNavigationToolFilterData& InRhs) const
{
	return !(*this == InRhs);
}

void FNavigationToolFilterData::Reset()
{
	RawFilterText.Reset();
	TotalNodeCount = 0;
	FilterInNodes.Reset();
}

FString FNavigationToolFilterData::GetRawFilterText() const
{
	return RawFilterText;
}

uint32 FNavigationToolFilterData::GetDisplayNodeCount() const
{
	return FilterInNodes.Num();
}

uint32 FNavigationToolFilterData::GetTotalNodeCount() const
{
	return TotalNodeCount;
}

uint32 FNavigationToolFilterData::GetFilterInCount() const
{
	return FilterInNodes.Num();
}

uint32 FNavigationToolFilterData::GetFilterOutCount() const
{
	return GetTotalNodeCount() - GetFilterInCount();
}

void FNavigationToolFilterData::IncrementTotalNodeCount()
{
	++TotalNodeCount;
}

void FNavigationToolFilterData::FilterInNode(const FNavigationToolViewModelWeakPtr& InWeakNode)
{
	FilterInNodes.Add(InWeakNode.Pin());
}

void FNavigationToolFilterData::FilterOutNode(const FNavigationToolViewModelWeakPtr& InWeakNode)
{
	const FSetElementId ElementId = FilterInNodes.FindId(InWeakNode.Pin());
	if (ElementId.IsValidId())
	{
		FilterInNodes.Remove(ElementId);
	}
}

void FNavigationToolFilterData::FilterInParentChildNodes(const FNavigationToolViewModelWeakPtr& InWeakNode
	, const bool bInIncludeSelf, const bool bInIncludeParents, const bool bInIncludeChildren)
{
	const FNavigationToolViewModelPtr Item = InWeakNode.Pin();
	if (!Item.IsValid())
	{
		return;
	}

	if (bInIncludeParents)
	{
		FNavigationToolViewModelPtr CurrentParent = Item->GetParent();
		while (CurrentParent.IsValid())
		{
			FilterInNode(CurrentParent);

			CurrentParent = CurrentParent->GetParent();
			if (CurrentParent.IsValid() && CurrentParent->GetItemId() == FNavigationToolItemId::RootId)
			{
				CurrentParent = nullptr;
			}
		}
	}

	if (bInIncludeSelf)
	{
		FilterInNode(InWeakNode);
	}

	if (bInIncludeChildren)
	{
		auto AddAllChildren = [this](const FNavigationToolViewModelPtr& InNode)
		{
			for (const FNavigationToolViewModelWeakPtr& WeakChild : InNode->GetChildren())
			{
				FilterInNode(WeakChild);
			}
		};

		for (const FNavigationToolViewModelWeakPtr& WeakChild : Item->GetChildren())
		{
			AddAllChildren(WeakChild.Pin());
		}
	}
}

void FNavigationToolFilterData::FilterInNodeWithAncestors(const FNavigationToolViewModelWeakPtr& InWeakNode)
{
	FilterInParentChildNodes(InWeakNode.Pin(), true, true, false);
}

bool FNavigationToolFilterData::ContainsFilterInNodes(const FNavigationToolFilterData& InOtherData) const
{
	return FilterInNodes.Includes(InOtherData.FilterInNodes);
}

bool FNavigationToolFilterData::IsFilteredIn(const FNavigationToolViewModelWeakPtr& InWeakNode) const
{
	return FilterInNodes.Contains(InWeakNode.Pin());
}

bool FNavigationToolFilterData::IsFilteredOut(const FNavigationToolViewModelWeakPtr& InWeakNode) const
{
	return !FilterInNodes.Contains(InWeakNode.Pin());
}

} // namespace UE::SequenceNavigator
