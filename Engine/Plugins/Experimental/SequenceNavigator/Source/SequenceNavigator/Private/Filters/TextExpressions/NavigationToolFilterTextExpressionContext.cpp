// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/TextExpressions/NavigationToolFilterTextExpressionContext.h"
#include "Filters/INavigationToolFilterBar.h"
#include "Items/NavigationToolItem.h"

namespace UE::SequenceNavigator
{

using namespace Sequencer;

FNavigationToolFilterTextExpressionContext::FNavigationToolFilterTextExpressionContext(INavigationToolFilterBar& InFilterInterface)
	: FilterInterface(InFilterInterface)
{
}

void FNavigationToolFilterTextExpressionContext::SetFilterItem(const FNavigationToolViewModelPtr& InFilterItem)
{
	WeakFilterItem = InFilterItem;
}

bool FNavigationToolFilterTextExpressionContext::TestBasicStringExpression(const FTextFilterString& InValue
	, const ETextFilterTextComparisonMode InTextComparisonMode) const
{
	const FNavigationToolViewModelPtr FilterItem = WeakFilterItem.Pin();
	if (!FilterItem.IsValid())
	{
		return false;
	}

	constexpr ETextFilterTextComparisonMode TextComparisonMode = ETextFilterTextComparisonMode::Partial;

	FString Label = FilterItem->GetDisplayName().ToString();

	if (TextFilterUtils::TestBasicStringExpression(Label, InValue, TextComparisonMode))
	{
		return true;
	}

	for (const FNavigationToolViewModelPtr& ParentItem : FilterItem->GetParents())
	{
		Label = ParentItem->GetDisplayName().ToString();
		if (TextFilterUtils::TestBasicStringExpression(Label, InValue, TextComparisonMode))
		{
			return true;
		}
	}

	return false;
}

bool FNavigationToolFilterTextExpressionContext::TestComplexExpression(const FName& InKey
	, const FTextFilterString& InValue
	, const ETextFilterComparisonOperation InComparisonOperation
	, const ETextFilterTextComparisonMode InTextComparisonMode) const
{
	const TSet<FName> Keys = GetKeys();
	if (!Keys.IsEmpty() && !Keys.Contains(InKey))
	{
		return false;
	}

	return !InValue.IsEmpty();
}

UMovieSceneSequence* FNavigationToolFilterTextExpressionContext::GetFocusedMovieSceneSequence() const
{
	return FilterInterface.GetSequencer().GetFocusedMovieSceneSequence();
}

UMovieScene* FNavigationToolFilterTextExpressionContext::GetFocusedGetMovieScene() const
{
	const UMovieSceneSequence* const FocusedMovieSceneSequence = GetFocusedMovieSceneSequence();
	return FocusedMovieSceneSequence ? FocusedMovieSceneSequence->GetMovieScene() : nullptr;
}

} // namespace UE::SequenceNavigator
