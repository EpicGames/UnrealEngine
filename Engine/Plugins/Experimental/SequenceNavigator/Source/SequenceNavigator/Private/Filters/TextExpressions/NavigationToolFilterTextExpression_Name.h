// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/ISequencerTextFilterExpressionContext.h"
#include "Filters/TextExpressions/NavigationToolFilterTextExpressionContext.h"

namespace UE::SequenceNavigator
{

class INavigationToolFilterBar;

class FNavigationToolFilterTextExpression_Name : public FNavigationToolFilterTextExpressionContext
{
public:
	FNavigationToolFilterTextExpression_Name(INavigationToolFilterBar& InFilterInterface);

	//~ Begin ISequencerTextFilterExpressionContext
	virtual TSet<FName> GetKeys() const override;
	virtual ESequencerTextFilterValueType GetValueType() const override;
	virtual FText GetDescription() const override;
	//~ End ISequencerTextFilterExpressionContext

	//~ Begin ITextFilterExpressionContext
	virtual bool TestComplexExpression(const FName& InKey
		, const FTextFilterString& InValue
		, const ETextFilterComparisonOperation InComparisonOperation
		, const ETextFilterTextComparisonMode InTextComparisonMode) const override;
	//~ End ITextFilterExpressionContext
};

} // namespace UE::SequenceNavigator
