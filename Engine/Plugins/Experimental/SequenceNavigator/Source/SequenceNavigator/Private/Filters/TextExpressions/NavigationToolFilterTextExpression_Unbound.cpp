// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavigationToolFilterTextExpression_Unbound.h"

#define LOCTEXT_NAMESPACE "NavigationToolFilterTextExpression_Unbound"

namespace UE::SequenceNavigator
{

using namespace Sequencer;

FNavigationToolFilterTextExpression_Unbound::FNavigationToolFilterTextExpression_Unbound(INavigationToolFilterBar& InFilterInterface)
	: FNavigationToolFilterTextExpressionContext(InFilterInterface)
{
}

TSet<FName> FNavigationToolFilterTextExpression_Unbound::GetKeys() const
{
	return { TEXT("Unbound") };
}

ESequencerTextFilterValueType FNavigationToolFilterTextExpression_Unbound::GetValueType() const
{
	return ESequencerTextFilterValueType::String;
}

FText FNavigationToolFilterTextExpression_Unbound::GetDescription() const
{
	return LOCTEXT("ExpressionDescription", "Filter by Sequences that contain unbound tracks");
}

bool FNavigationToolFilterTextExpression_Unbound::TestComplexExpression(const FName& InKey
	, const FTextFilterString& InValue
	, const ETextFilterComparisonOperation InComparisonOperation
	, const ETextFilterTextComparisonMode InTextComparisonMode) const
{
	if (!FNavigationToolFilterTextExpressionContext::TestComplexExpression(InKey, InValue, InComparisonOperation, InTextComparisonMode))
	{
		return true;
	}

	/*if (const FNavigationToolViewModelPtr FilterItem = WeakFilterItem.Pin())
	{
		const FString Label = FilterItem->GetDisplayName().ToString();
		const bool bPassed = TextFilterUtils::TestComplexExpression(Label, InValue, InComparisonOperation, InTextComparisonMode);
		return bPassed;
	}*/

	return false;
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
