// Copyright Epic Games, Inc. All Rights Reserved.

#include "ItemActions/NavigationToolAddItem.h"
#include "Items/INavigationToolItem.h"
#include "Items/NavigationToolTreeRoot.h"
#include "NavigationTool.h"
#include "NavigationToolScopedSelection.h"
#include "Widgets/Views/STableRow.h"

namespace UE::SequenceNavigator
{

using namespace Sequencer;

UE_SEQUENCER_DEFINE_CASTABLE(FNavigationToolAddItem)

FNavigationToolAddItem::FNavigationToolAddItem(const FNavigationToolAddItemParams& InAddItemParams)
	: AddParams(InAddItemParams)
{
}

bool FNavigationToolAddItem::ShouldTransact() const
{
	return EnumHasAnyFlags(AddParams.Flags, ENavigationToolAddItemFlags::Transact);
}

void FNavigationToolAddItem::Execute(FNavigationTool& InTool)
{
	const FNavigationToolViewModelPtr TreeRoot = InTool.GetTreeRoot().Pin();
	if (!TreeRoot.IsValid())
	{
		return;
	}

	const FNavigationToolViewModelPtr ItemToAdd = AddParams.WeakItem.Pin();
	if (!ItemToAdd.IsValid())
	{
		return;
	}

	// Try to Create Children on Find
	if (EnumHasAnyFlags(AddParams.Flags, ENavigationToolAddItemFlags::AddChildren))
	{
		TArray<FNavigationToolViewModelWeakPtr> WeakChildren;
		ItemToAdd->FindValidChildren(WeakChildren, /*bInRecursive=*/true);
	}

	const FNavigationToolViewModelPtr ParentItem = ItemToAdd->GetParent();

	// If this array has elements in it, then we need to stop a circular dependency from forming
	const TArray<FNavigationToolViewModelPtr> PathToRelativeItem = ItemToAdd->FindPath({ AddParams.WeakRelativeItem });
	if (PathToRelativeItem.Num() > 0 && ParentItem.IsValid())
	{
		FNavigationToolAddItemParams CircularSolverParams;
		CircularSolverParams.WeakItem         = PathToRelativeItem[0];
		CircularSolverParams.WeakRelativeItem = AddParams.WeakItem;
		CircularSolverParams.RelativeDropZone = EItemDropZone::AboveItem;
		CircularSolverParams.Flags            = AddParams.Flags;

		ParentItem->AddChild(MoveTemp(CircularSolverParams));
	}

	if (const FNavigationToolViewModelPtr RelativeItem = AddParams.WeakRelativeItem.Pin())
	{
		const FNavigationToolViewModelPtr RelativeItemParent = RelativeItem->GetParent();

		// If it's onto item, the relative item is going to be the parent
		if (!AddParams.RelativeDropZone.IsSet() || AddParams.RelativeDropZone == EItemDropZone::OntoItem)
		{
			// If the relative item is onto and it's the same as the current parent, shift Item up in the hierarchy
			// (as long as the parent is valid)
			if (RelativeItem == ParentItem && RelativeItemParent)
			{
				AddParams.RelativeDropZone = EItemDropZone::BelowItem;
				RelativeItemParent->AddChild(AddParams);
			}
			else
			{
				RelativeItem->AddChild(AddParams);
			}
		}
		// Else we place it as a Sibling to the Relative Item
		else if (RelativeItemParent)
		{
			RelativeItemParent->AddChild(AddParams);
		}
		// If no parent, then add it to the tree root
		else
		{
			TreeRoot->AddChild(AddParams);
		}
	}
	else
	{
		// If no relative item, add to tree root
		TreeRoot->AddChild(AddParams);
	}

	const FNavigationToolScopedSelection ScopedSelection(*InTool.GetSequencer(), ENavigationToolScopedSelectionPurpose::Read);

	// Automatically select the item if it's selected
	if (ItemToAdd->IsSelected(ScopedSelection))
	{
		// Select in Navigation Tool but don't signal selection as we already have it selected in mode tools
		AddParams.Flags = ENavigationToolAddItemFlags::Select;
		AddParams.SelectionFlags &= ~ENavigationToolItemSelectionFlags::SignalSelectionChange;
	}
	// Signal selection change when we attempt to select this item in the Navigation Tool but it isn't selected in Sequencer
	else if (EnumHasAnyFlags(AddParams.Flags, ENavigationToolAddItemFlags::Select))
	{
		AddParams.SelectionFlags |= ENavigationToolItemSelectionFlags::SignalSelectionChange;
	}

	if (EnumHasAnyFlags(AddParams.Flags, ENavigationToolAddItemFlags::Select))
	{
		InTool.SelectItems({ ItemToAdd }, AddParams.SelectionFlags);
	}

	InTool.SetToolModified();
}

void FNavigationToolAddItem::OnObjectsReplaced(const TMap<UObject*, UObject*>& InReplacementMap, bool bRecursive)
{
	using namespace Sequencer;

	if (const FNavigationToolViewModelPtr ItemToAdd = AddParams.WeakItem.Pin())
	{
		ItemToAdd->OnObjectsReplaced(InReplacementMap, bRecursive);
	}

	if (const FNavigationToolViewModelPtr RelativeItem = AddParams.WeakRelativeItem.Pin())
	{
		RelativeItem->OnObjectsReplaced(InReplacementMap, bRecursive);
	}
}

} // namespace UE::SequenceNavigator
