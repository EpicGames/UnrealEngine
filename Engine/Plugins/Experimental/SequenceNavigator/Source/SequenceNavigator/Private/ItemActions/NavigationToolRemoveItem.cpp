// Copyright Epic Games, Inc. All Rights Reserved.

#include "ItemActions/NavigationToolRemoveItem.h"
#include "ItemActions/NavigationToolAddItem.h"
#include "Items/INavigationToolItem.h"
#include "NavigationTool.h"
#include "Widgets/Views/STableRow.h"

namespace UE::SequenceNavigator
{

using namespace Sequencer;

UE_SEQUENCER_DEFINE_CASTABLE(FNavigationToolRemoveItem)

FNavigationToolRemoveItem::FNavigationToolRemoveItem(const FNavigationToolRemoveItemParams& InRemoveItemParams)
	: RemoveParams(InRemoveItemParams)
{
	if (const FNavigationToolViewModelPtr Item = RemoveParams.WeakItem.Pin())
	{
		Item->AddFlags(ENavigationToolItemFlags::PendingRemoval);
	}
}

void FNavigationToolRemoveItem::Execute(FNavigationTool& InTool)
{
	using namespace Sequencer;

	const FNavigationToolViewModelPtr Item = RemoveParams.WeakItem.Pin();
	if (!Item.IsValid())
	{
		return;
	}

	const FNavigationToolItemFlagGuard Guard(Item, ENavigationToolItemFlags::IgnorePendingKill);

	// Copy the array since we may modify it below on add/remove child
	TArray<FNavigationToolViewModelWeakPtr> WeakChildren = Item->GetChildren();

	FNavigationToolViewModelPtr Parent = Item->GetParent();
	FNavigationToolViewModelPtr RelativeItem = Item;

	// Search the lowest parent that is not pending removal
	while (Parent.IsValid() && Parent->HasAnyFlags(ENavigationToolItemFlags::PendingRemoval))
	{
		RelativeItem = Parent;
		Parent = Parent->GetParent();
	}

	// Reparent the item's children to the valid parent found above
	if (Parent.IsValid())
	{
		FNavigationToolAddItemParams ReparentParams;
		ReparentParams.WeakRelativeItem = RelativeItem;
		ReparentParams.RelativeDropZone = EItemDropZone::BelowItem;
		ReparentParams.Flags = ENavigationToolAddItemFlags::Select;

		TArray<FNavigationToolViewModelWeakPtr> ItemsToReparent;

		ItemsToReparent.Append(WeakChildren);
		while (ItemsToReparent.Num() > 0)
		{
			ItemsToReparent.Pop();
			ReparentParams.WeakItem = ItemsToReparent.Pop();

			// Then parent it to the item's parent.
			// If we couldn't add the Child, then it means either the child item itself is invalid
			// or the underlying info (e.g. object) is invalid (e.g. actor pending kill).
			if (!Parent->AddChild(ReparentParams))
			{
				// So try reparenting the children of this invalid child (since this invalid child will be removed)
				if (const FNavigationToolViewModelPtr ChildItem = ReparentParams.WeakItem.Pin())
				{
					ItemsToReparent.Append(ChildItem->GetChildren());
				}
			}
		}

		// In case the parent is still the same, remove child item, else the parent is going to be removed anyways
		if (Parent == Item->GetParent())
		{
			Parent->RemoveChild(Item);
		}
	}
	else
	{
		for (const FNavigationToolViewModelWeakPtr& WeakChild : WeakChildren)
		{
			if (const FNavigationToolViewModelPtr ChildItem = WeakChild.Pin())
			{
				Item->RemoveChild(ChildItem);
			}
		}
	}

	InTool.UnregisterItem(Item->GetItemId());
	InTool.SetToolModified();
}

void FNavigationToolRemoveItem::OnObjectsReplaced(const TMap<UObject*, UObject*>& InReplacementMap, const bool bInRecursive)
{
	if (const FNavigationToolViewModelPtr Item = RemoveParams.WeakItem.Pin())
	{
		Item->OnObjectsReplaced(InReplacementMap, bInRecursive);
	}
}

} // namespace UE::SequenceNavigator
