// Copyright Epic Games, Inc. All Rights Reserved.

#include "ItemProxies/NavigationToolComponentProxy.h"
#include "NavigationTool.h"
#include "Styling/SlateIconFinder.h"

#define LOCTEXT_NAMESPACE "NavigationToolComponentProxy"

namespace UE::SequenceNavigator
{

UE_SEQUENCER_DEFINE_CASTABLE(FNavigationToolComponentProxy)

FNavigationToolComponentProxy::FNavigationToolComponentProxy(INavigationTool& InTool, const FNavigationToolViewModelPtr& InParentItem)
	: FNavigationToolItemProxy(InTool, InParentItem)
{
}

FText FNavigationToolComponentProxy::GetDisplayName() const
{
	return LOCTEXT("DisplayName", "Components");
}

FSlateIcon FNavigationToolComponentProxy::GetIcon() const
{
	return FSlateIconFinder::FindIconForClass(USceneComponent::StaticClass());
}

FText FNavigationToolComponentProxy::GetIconTooltipText() const
{
	return LOCTEXT("Tooltip", "Shows the Components in an Actor. Visualization, non-editable and UCS Components are excluded");
}

ENavigationToolItemViewMode FNavigationToolComponentProxy::GetSupportedViewModes(const INavigationToolView& InToolView) const
{
	// Components should only be visualized in Navigation Tool View and not appear in the Item Column List
	// Support any other type of View Mode
	return ~ENavigationToolItemViewMode::ItemTree | ENavigationToolItemViewMode::HorizontalItemList;
}

void FNavigationToolComponentProxy::GetProxiedItems(const FNavigationToolViewModelPtr& InParent
	, TArray<FNavigationToolViewModelWeakPtr>& OutWeakChildren, const bool bInRecursive)
{
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
