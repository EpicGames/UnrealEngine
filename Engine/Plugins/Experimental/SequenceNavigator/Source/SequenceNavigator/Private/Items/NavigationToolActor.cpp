// Copyright Epic Games, Inc. All Rights Reserved.

#include "Items/NavigationToolActor.h"
#include "Editor/EditorEngine.h"
#include "GameFramework/Actor.h"
#include "ISequencer.h"
#include "Items/NavigationToolComponent.h"
#include "Items/NavigationToolSequence.h"
#include "MovieScene.h"
#include "MovieSceneCommonHelpers.h"
#include "MovieSceneSequence.h"
#include "NavigationTool.h"
#include "NavigationToolSettings.h"
#include "ScopedTransaction.h"
#include "Utils/NavigationToolMiscUtils.h"

#define LOCTEXT_NAMESPACE "NavigationToolActor"

namespace UE::SequenceNavigator
{

using namespace Sequencer;

UE_SEQUENCER_DEFINE_CASTABLE(FNavigationToolActor)

FNavigationToolActor::FNavigationToolActor(INavigationTool& InTool
	, const FNavigationToolViewModelPtr& InParentItem
	, const TSharedPtr<FNavigationToolSequence>& InParentSequenceItem
	, const FMovieSceneBinding& InBinding)
	: FNavigationToolBinding(InTool, InParentItem, InParentSequenceItem, InBinding)
{
}

void FNavigationToolActor::FindChildren(TArray<FNavigationToolViewModelWeakPtr>& OutWeakChildren, const bool bInRecursive)
{
	FNavigationToolBinding::FindChildren(OutWeakChildren, bInRecursive);

	const AActor* const ThisActor = GetActor();
	if (!ThisActor)
	{
		return;
	}

	const TViewModelPtr<FNavigationToolSequence> ParentSequenceItem = WeakParentSequenceItem.Pin();
	if (!ParentSequenceItem.IsValid())
	{
		return;
	}

	UMovieSceneSequence* const Sequence = ParentSequenceItem->GetSequence();
	if (!Sequence)
	{
		return;
	}

	UMovieScene* const MovieScene = Sequence->GetMovieScene();
	if (!MovieScene)
	{
		return;
	}

	const TSharedPtr<ISequencer> Sequencer = Tool.GetSequencer();
	if (!Sequencer.IsValid())
	{
		return;
	}

	const TSharedPtr<FNavigationToolProvider> Provider = GetProvider();
	if (!Provider.IsValid())
	{
		return;
	}

	const TSharedRef<FNavigationToolProvider> ProviderRef = Provider.ToSharedRef();
	const FNavigationToolViewModelPtr ParentItem = AsItemViewModel();
	const TArray<FMovieSceneBinding> Bindings = ParentSequenceItem->GetSortedBindings();

	for (const FMovieSceneBinding& CurrentBinding : Bindings)
	{
		const FGuid& CurrentBindingGuid = CurrentBinding.GetObjectGuid();
		const UClass* const CurrentObjectClass = MovieSceneHelpers::GetBoundObjectClass(Sequence, CurrentBindingGuid);
		if (CurrentObjectClass && CurrentObjectClass->IsChildOf<UActorComponent>())
		{
			const TArrayView<TWeakObjectPtr<>> WeakBoundObjects = ResolveBoundObjects(*Sequencer, Sequence, CurrentBindingGuid);
			const TWeakObjectPtr<> CurrentBoundObject = WeakBoundObjects.IsEmpty() ? nullptr : WeakBoundObjects[0];
			const UActorComponent* const CurrentBoundActorComponent = Cast<UActorComponent>(CurrentBoundObject);

			if (CurrentBoundActorComponent && CurrentBoundActorComponent->GetOwner() == ThisActor)
			{
				const FNavigationToolViewModelPtr NewItem = Tool.FindOrAdd<FNavigationToolComponent>(ProviderRef
					, ParentItem, ParentSequenceItem, CurrentBinding);
				OutWeakChildren.Add(NewItem);
				if (bInRecursive)
				{
					NewItem->FindChildren(OutWeakChildren, bInRecursive);
				}
			}
		}
	}
}

bool FNavigationToolActor::IsAllowedInTool() const
{
	FNavigationTool& ToolPrivate = static_cast<FNavigationTool&>(Tool);

	const AActor* const UnderlyingActor = GetActor();
	if (!UnderlyingActor)
	{
		// Always allow unbound binding items
		return true;
	}

	const bool bActorAllowed = ToolPrivate.IsObjectAllowedInTool(UnderlyingActor);

	return bActorAllowed;
}

ENavigationToolItemViewMode FNavigationToolActor::GetSupportedViewModes(const INavigationToolView& InToolView) const
{
	return ENavigationToolItemViewMode::ItemTree | ENavigationToolItemViewMode::HorizontalItemList;
}

TArray<FName> FNavigationToolActor::GetTags() const
{
	if (const AActor* const UnderlyingActor = GetActor())
	{
		return UnderlyingActor->Tags;
	}
	return FNavigationToolBinding::GetTags();
}

bool FNavigationToolActor::GetVisibility() const
{
	if (const AActor* const UnderlyingActor = GetActor())
	{
		return !UnderlyingActor->IsTemporarilyHiddenInEditor(true);
	}
	return false;
}

void FNavigationToolActor::OnVisibilityChanged(const bool bInNewVisibility)
{
	if (AActor* const UnderlyingActor = GetActor())
	{
		UnderlyingActor->SetIsTemporarilyHiddenInEditor(!bInNewVisibility);
	}
}

bool FNavigationToolActor::CanRename() const
{
	return FNavigationToolBinding::CanRename() && !GetActor();
}

void FNavigationToolActor::Rename(const FText& InNewName)
{
	AActor* const UnderlyingActor = GetActor();
	if (!UnderlyingActor || !UnderlyingActor->IsActorLabelEditable())
	{
		return;
	}

	const FString NewNameString = InNewName.ToString();
	if (NewNameString.Equals(UnderlyingActor->GetActorLabel(), ESearchCase::CaseSensitive))
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("NavigationToolRenameActor", "Rename Actor"));

	FActorLabelUtilities::RenameExistingActor(UnderlyingActor, NewNameString);

	FNavigationToolBinding::Rename(InNewName);
}

AActor* FNavigationToolActor::GetActor() const
{
	return WeakBoundObject.IsValid() ? Cast<AActor>(GetCachedBoundObject()) : nullptr;
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
