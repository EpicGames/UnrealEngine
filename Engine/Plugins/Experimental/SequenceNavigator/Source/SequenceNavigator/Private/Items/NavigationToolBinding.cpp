// Copyright Epic Games, Inc. All Rights Reserved.

#include "Items/NavigationToolBinding.h"
#include "INavigationTool.h"
#include "ISequencer.h"
#include "Items/NavigationToolItemUtils.h"
#include "Items/NavigationToolSequence.h"
#include "MovieScene.h"
#include "MVVM/ObjectBindingModelStorageExtension.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/TrackModel.h"
#include "NavigationToolScopedSelection.h"
#include "ScopedTransaction.h"
#include "Sections/MovieSceneSubSection.h"
#include "Styling/SlateIconFinder.h"
#include "Styling/StyleColors.h"
#include "Utils/NavigationToolMiscUtils.h"
#include "Widgets/Views/STableRow.h"

#define LOCTEXT_NAMESPACE "NavigationToolBinding"

namespace UE::SequenceNavigator
{

using namespace Sequencer;

UE_SEQUENCER_DEFINE_CASTABLE(FNavigationToolBinding)

FNavigationToolBinding::FNavigationToolBinding(INavigationTool& InTool
	, const FNavigationToolViewModelPtr& InParentItem
	, const TSharedPtr<FNavigationToolSequence>& InParentSequenceItem
	, const FMovieSceneBinding& InBinding)
	: FNavigationToolItem(InTool, InParentItem)
	, WeakParentSequenceItem(InParentSequenceItem)
	, Binding(InBinding)
{
	Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Sequencer.InvalidSpawnableIcon"));

	CacheBoundObject();
}

bool FNavigationToolBinding::IsItemValid() const
{
	return Binding.GetObjectGuid().IsValid();
}

UObject* FNavigationToolBinding::GetItemObject() const
{
	return GetCachedBoundObject();
}

bool FNavigationToolBinding::IsAllowedInTool() const
{
	return IsItemValid();
}

void FNavigationToolBinding::FindChildren(TArray<FNavigationToolViewModelWeakPtr>& OutWeakChildren, const bool bInRecursive)
{
	CacheBoundObject();

	FNavigationToolItem::FindChildren(OutWeakChildren, bInRecursive);
}

FText FNavigationToolBinding::GetDisplayName() const
{
	if (UMovieScene* const MovieScene = GetMovieScene())
	{
		return MovieScene->GetObjectDisplayName(Binding.GetObjectGuid());
	}
	return FText::GetEmpty();
}

FText FNavigationToolBinding::GetClassName() const
{
	if (WeakBoundObjectClass.IsValid())
	{
		return WeakBoundObjectClass->GetDisplayNameText();
	}
	return FText::GetEmpty();
}

FSlateIcon FNavigationToolBinding::GetIcon() const
{
	return Icon;
}

FText FNavigationToolBinding::GetIconTooltipText() const
{
	if (const UObject* const UnderlyingObject = GetCachedBoundObject())
	{
		return FText::Format(LOCTEXT("BoundObjectToolTip", " Class: %s (BindingID: %s)")
			, UnderlyingObject->GetClass()->GetDisplayNameText(), FText::FromString(LexToString(Binding.GetObjectGuid())));
	}
	return FText::Format(LOCTEXT("InvalidBoundObjectToolTip", "The object bound to this track is missing (BindingID: {0}).")
		, FText::FromString(LexToString(Binding.GetObjectGuid())));
}

FSlateColor FNavigationToolBinding::GetIconColor() const
{
	return IconColor;
}

bool FNavigationToolBinding::IsSelected(const FNavigationToolScopedSelection& InSelection) const
{
	return InSelection.IsSelected(Binding.GetObjectGuid());
}

void FNavigationToolBinding::Select(FNavigationToolScopedSelection& InSelection) const
{
	const FGuid ObjectGuid = Binding.GetObjectGuid();
	if (ObjectGuid.IsValid())
	{
		InSelection.Select(ObjectGuid);
	}
}
	
void FNavigationToolBinding::OnSelect()
{
	FNavigationToolItem::OnSelect();
}

void FNavigationToolBinding::OnDoubleClick()
{
	if (UMovieSceneSequence* const Sequence = GetSequence())
	{
		FocusSequence(Tool, *Sequence, *this);
	}
}

bool FNavigationToolBinding::CanDelete() const
{
	// Disable for now
	return false;
	//return WeakBoundObject.IsValid();
}

bool FNavigationToolBinding::Delete()
{
	return !WeakBoundObject.IsValid();
}

FNavigationToolItemId FNavigationToolBinding::CalculateItemId() const
{
	const TSharedPtr<FNavigationToolSequence> ParentSequenceItem = WeakParentSequenceItem.Pin();
	check(ParentSequenceItem.IsValid());

	return FNavigationToolItemId(GetParent()
		, ParentSequenceItem->GetSequence()
		, ParentSequenceItem->GetSubSection()
		, ParentSequenceItem->GetSubSectionIndex()
		, Binding.GetObjectGuid().ToString());
}

bool FNavigationToolBinding::CanRename() const
{
	// Disable for now
	return false;
	//return WeakBoundObject.IsValid();
}

void FNavigationToolBinding::Rename(const FText& InNewName)
{
	UMovieSceneSequence* const Sequence = GetSequence();
	if (!Sequence)
	{
		return;
	}

	UMovieScene* const MovieScene = Sequence->GetMovieScene();
	if (!MovieScene)
	{
		return;
	}

	const TSharedPtr<ISequencer> Sequencer = Tool.GetSequencer();
	if (!Sequencer.IsValid())
	{
		return;
	}

	const FGuid ObjectBindingID = Binding.GetObjectGuid();
	const FString NewNameString = InNewName.ToString();

	const FScopedTransaction Transaction(LOCTEXT("SetTrackName", "Set Track Name"));

	// Modify the movie scene so that it gets marked dirty and renames are saved consistently.
	MovieScene->Modify();

	// If there is only one binding, set the name of the bound actor
	const TArrayView<TWeakObjectPtr<>> Objects = Sequencer->FindObjectsInCurrentSequence(ObjectBindingID);
	if (Objects.Num() == 1)
	{
		if (AActor* const Actor = Cast<AActor>(Objects[0].Get()))
		{
			Actor->SetActorLabel(NewNameString);
		}
	}

	if (FMovieSceneSpawnable* const Spawnable = MovieScene->FindSpawnable(ObjectBindingID))
	{
		Spawnable->SetName(NewNameString);
	}
	else if (FMovieScenePossessable* const Possessable = MovieScene->FindPossessable(ObjectBindingID))
	{
		Possessable->SetName(NewNameString);
	}
	else
	{
		MovieScene->SetObjectDisplayName(ObjectBindingID, InNewName);
	}
}
/*
EItemSequenceLockState FNavigationToolBinding::GetLockState() const
{
	TArray<UMovieSceneSection*> Sections;

	for (UMovieSceneTrack* const Track : Binding.GetTracks())
	{
		for (UMovieSceneSection* const Section : Track->GetAllSections())
		{
			Sections.Add(Section);
		}
	}

	const ENavigationToolCompareState State = CompareArrayState<UMovieSceneSection>(Sections,
		[](const UMovieSceneSection* const InSection)
			{
				return InSection->IsLocked();
			},
		[](const UMovieSceneSection* const InSection)
			{
				return !InSection->IsLocked();
			}); 

	return static_cast<EItemSequenceLockState>(State);
}

void FNavigationToolBinding::SetIsLocked(const bool bInIsLocked)
{
	for (UMovieSceneTrack* const Track : Binding.GetTracks())
	{
		for (UMovieSceneSection* const Section : Track->GetAllSections())
		{
			if (Section->IsLocked() != bInIsLocked)
			{
				Section->Modify();
				Section->SetIsLocked(bInIsLocked);
			}
		}
	}

	for (ISequenceLockableExtension* const LockableItem : GetChildrenOfType<ISequenceLockableExtension>())
	{
		LockableItem->SetIsLocked(bInIsLocked);
	}
}*/

FText FNavigationToolBinding::GetId() const
{
	return FText::FromString(Binding.GetObjectGuid().ToString());
}

EItemContainsPlayhead FNavigationToolBinding::ContainsPlayhead() const
{
	for (const FNavigationToolViewModelWeakPtr& WeakItem : GetChildren())
	{
		if (const TViewModelPtr<IPlayheadExtension> PlayheadExtension = WeakItem.ImplicitPin())
		{
			return PlayheadExtension->ContainsPlayhead();
		}
	}

	const TSharedPtr<ISequencer> Sequencer = Tool.GetSequencer();
	if (!Sequencer.IsValid())
	{
		return EItemContainsPlayhead::None;
	}

	const FQualifiedFrameTime PlayheadTime = Sequencer->GetLocalTime();
	const FFrameNumber PlayheadFrame = PlayheadTime.ConvertTo(Sequencer->GetFocusedTickResolution()).FloorToFrame();

	for (const UMovieSceneTrack* const Track : Binding.GetTracks())
	{
		const TArray<UMovieSceneSection*> Sections = Track->GetAllSections();
		for (const UMovieSceneSection* const Section : Sections)
		{
			const TRange<FFrameNumber> Range = Section->ComputeEffectiveRange();
			if (Range.Contains(PlayheadFrame))
			{
				return EItemContainsPlayhead::ContainsPlayhead;
			}
		}
	}

	return EItemContainsPlayhead::None;
}

UObject* FNavigationToolBinding::GetCachedBoundObject() const
{
	return WeakBoundObject.Get();
}

UObject* FNavigationToolBinding::CacheBoundObject()
{
	using namespace ItemUtils;

	const TSharedPtr<ISequencer> Sequencer = Tool.GetSequencer();
	if (!Sequencer.IsValid())
	{
		return nullptr;
	}

	UMovieSceneSequence* const Sequence = GetSequence();
	if (!Sequence)
	{
		return nullptr;
	}

	const TArrayView<TWeakObjectPtr<>> WeakBoundObjects = ResolveBoundObjects(*Sequencer, Sequence, Binding.GetObjectGuid());
	WeakBoundObject = WeakBoundObjects.IsEmpty() ? nullptr : WeakBoundObjects[0];

	WeakBoundObjectClass = MovieSceneHelpers::GetBoundObjectClass(Sequence, Binding.GetObjectGuid());

	Icon = FSlateIconFinder::FindIconForClass(WeakBoundObjectClass.Get());

	const FSlateColor DefaultColor = FNavigationToolItem::GetItemLabelColor();
	IconColor = GetItemBindingColor(*Sequencer, *Sequence, Binding.GetObjectGuid(), DefaultColor);

	return WeakBoundObject.Get();
}

const FMovieSceneBinding& FNavigationToolBinding::GetBinding() const
{
	return Binding;
}

UMovieSceneSequence* FNavigationToolBinding::GetSequence() const
{
	if (const TSharedPtr<FNavigationToolSequence> ParentSequenceItem = WeakParentSequenceItem.Pin())
	{
		return ParentSequenceItem->GetSequence();
	}
	return nullptr;
}

UMovieScene* FNavigationToolBinding::GetMovieScene() const
{
	if (UMovieSceneSequence* const Sequence = GetSequence())
	{
		return Sequence->GetMovieScene();
	}
	return nullptr;
}

TViewModelPtr<FObjectBindingModel> FNavigationToolBinding::GetViewModel() const
{
	const FGuid ObjectGuid = Binding.GetObjectGuid();
	if (!ObjectGuid.IsValid())
	{
		return nullptr;
	}

	const TSharedPtr<ISequencer> Sequencer = Tool.GetSequencer();
	if (!Sequencer.IsValid())
	{
		return nullptr;
	}

	const TSharedPtr<FSequencerEditorViewModel> ViewModel = Sequencer->GetViewModel();
	if (!ViewModel.IsValid())
	{
		return nullptr;
	}

	const FViewModelPtr RootViewModel = ViewModel->GetRootModel();
	if (!RootViewModel.IsValid())
	{
		return nullptr;
	}

	const FObjectBindingModelStorageExtension* const StorageExtension = RootViewModel->CastDynamic<FObjectBindingModelStorageExtension>();
	if (!StorageExtension)
	{
		return nullptr;
	}

	return StorageExtension->FindModelForObjectBinding(ObjectGuid);
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
