// Copyright Epic Games, Inc. All Rights Reserved.

#include "Items/NavigationToolComponent.h"
#include "Components/PrimitiveComponent.h"
#include "Components/SceneComponent.h"
#include "Items/NavigationToolItemProxy.h"
#include "NavigationTool.h"
#include "NavigationToolSettings.h"
#include "ScopedTransaction.h"
#include "Styling/StyleColors.h"
#include "Widgets/Columns/SNavigationToolLabelComponent.h"

#define LOCTEXT_NAMESPACE "NavigationToolComponent"

namespace UE::SequenceNavigator
{

UE_SEQUENCER_DEFINE_CASTABLE(FNavigationToolComponent)

FNavigationToolComponent::FNavigationToolComponent(INavigationTool& InTool
	, const FNavigationToolViewModelPtr& InParentItem
	, const TSharedPtr<FNavigationToolSequence>& InParentSequenceItem
	, const FMovieSceneBinding& InBinding)
	: FNavigationToolBinding(InTool, InParentItem, InParentSequenceItem, InBinding)
{
}

void FNavigationToolComponent::FindChildren(TArray<FNavigationToolViewModelWeakPtr>& OutWeakChildren, const bool bInRecursive)
{
	FNavigationToolBinding::FindChildren(OutWeakChildren, bInRecursive);
}

void FNavigationToolComponent::GetItemProxies(TArray<TSharedPtr<FNavigationToolItemProxy>>& OutItemProxies)
{
	FNavigationToolBinding::GetItemProxies(OutItemProxies);

	if (UPrimitiveComponent* const PrimitiveComponent = Cast<UPrimitiveComponent>(GetComponent()))
	{
		if (const TSharedPtr<FNavigationToolItemProxy> MaterialItemProxy = Tool.GetOrCreateItemProxy<FNavigationToolItemProxy>(AsItemViewModel()))
		{
			OutItemProxies.Add(MaterialItemProxy);
		}
	}
}

bool FNavigationToolComponent::IsAllowedInTool() const
{
	FNavigationTool& ToolPrivate = static_cast<FNavigationTool&>(Tool);

	const UActorComponent* const UnderlyingComponent = GetComponent();
	if (!UnderlyingComponent)
	{
		// Always allow unbound binding items
		return true;
	}

	const bool bOwnerAllowed = ToolPrivate.IsObjectAllowedInTool(UnderlyingComponent->GetOwner());
	const bool bComponentAllowed = ToolPrivate.IsObjectAllowedInTool(UnderlyingComponent);

	return bOwnerAllowed && bComponentAllowed;
}

ENavigationToolItemViewMode FNavigationToolComponent::GetSupportedViewModes(const INavigationToolView& InToolView) const
{
	return ENavigationToolItemViewMode::ItemTree | ENavigationToolItemViewMode::HorizontalItemList;
}

TSharedRef<SWidget> FNavigationToolComponent::GenerateLabelWidget(const TSharedRef<SNavigationToolTreeRow>& InRow)
{
	return SNew(SNavigationToolLabelComponent, AsItemViewModel(), InRow);
}

FLinearColor FNavigationToolComponent::GetItemTintColor() const
{
	return FStyleColors::White25.GetSpecifiedColor();
}

TArray<FName> FNavigationToolComponent::GetTags() const
{
	if (const UActorComponent* const UnderlyingComponent = GetComponent())
	{
		return UnderlyingComponent->ComponentTags;
	}
	return FNavigationToolBinding::GetTags();
}

bool FNavigationToolComponent::GetVisibility() const
{
	if (const USceneComponent* const UnderlyingComponent = Cast<USceneComponent>(GetComponent()))
	{
		return UnderlyingComponent->IsVisibleInEditor();
	}
	return false;
}

void FNavigationToolComponent::OnVisibilityChanged(const bool bInNewVisibility)
{
	if (USceneComponent* const UnderlyingComponent = Cast<USceneComponent>(GetComponent()))
	{
		UnderlyingComponent->SetVisibility(bInNewVisibility);
	}
}

bool FNavigationToolComponent::CanRename() const
{
	return FNavigationToolBinding::CanRename() && !GetComponent();
}

void FNavigationToolComponent::Rename(const FText& InNewName)
{
	UActorComponent* const UnderlyingComponent = GetComponent();
	if (!UnderlyingComponent)
	{
		return;
	}

	const FString NewNameString = InNewName.ToString();

	if (NewNameString.Equals(UnderlyingComponent->GetName(), ESearchCase::CaseSensitive))
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("RenameComponent", "Rename Component"));

	UnderlyingComponent->Modify();
	UnderlyingComponent->Rename(*NewNameString);

	FNavigationToolBinding::Rename(InNewName);
}

UActorComponent* FNavigationToolComponent::GetComponent() const
{
	return WeakBoundObject.IsValid() ? Cast<UActorComponent>(GetCachedBoundObject()) : nullptr;
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
