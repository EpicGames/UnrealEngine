// Copyright Epic Games, Inc. All Rights Reserved.

#include "Items/NavigationToolItem.h"
#include "DragDropOps/NavigationToolItemDragDropOp.h"
#include "Editor.h"
#include "Framework/Application/SlateApplication.h"
#include "ItemActions/NavigationToolAddItem.h"
#include "ItemActions/NavigationToolRemoveItem.h"
#include "Items/NavigationToolItemUtils.h"
#include "Misc/Optional.h"
#include "NavigationTool.h"
#include "NavigationToolSettings.h"
#include "NavigationToolView.h"
#include "Providers/NavigationToolProvider.h"
#include "Styling/StyleColors.h"
#include "Textures/SlateIcon.h"
#include "Widgets/Columns/SNavigationToolLabelItem.h"
#include "Widgets/Views/STableRow.h"

#define LOCTEXT_NAMESPACE "NavigationToolItem"

namespace UE::SequenceNavigator
{

using namespace Sequencer;

UE_SEQUENCER_DEFINE_CASTABLE(FNavigationToolItem)

FNavigationToolItem::FNavigationToolItem(INavigationTool& InTool, const FNavigationToolViewModelPtr& InParentItem)
	: Tool(InTool)
	, WeakParent(InParentItem)
{
}

INavigationTool& FNavigationToolItem::GetOwnerTool() const
{
	return Tool;
}

TSharedPtr<FNavigationToolProvider> FNavigationToolItem::GetProvider() const
{
	return WeakProvider.Pin();
}

bool FNavigationToolItem::IsItemValid() const
{
	return true;
}

void FNavigationToolItem::RefreshChildren()
{
	using namespace ItemUtils;

	TArray<FNavigationToolViewModelWeakPtr> WeakFoundChildren;
	FindValidChildren(WeakFoundChildren, /*bRecursiveFind=*/false);

	TArray<FNavigationToolViewModelWeakPtr> Sortable;
	TArray<FNavigationToolViewModelWeakPtr> Unsortable;
	SplitSortableAndUnsortableItems(WeakFoundChildren, Sortable, Unsortable);

	// Start with all sortable/unsortable items, and remove every item seen by iterating children
	TSet<FNavigationToolViewModelWeakPtr> NewSortableChildren(Sortable);
	TSet<FNavigationToolViewModelWeakPtr> NewUnsortableChildren(Unsortable);
	
	// Remove items from "WeakChildren" that were not present in the sortable found children (we'll add non-sortable later)
	// Result is have WeakChildren only contain Items that existed previously
	TSet<FNavigationToolViewModelWeakPtr> ItemsToRemove;

	for (const FNavigationToolViewModelWeakPtr& WeakChildItem : WeakChildren)
	{
		const FNavigationToolViewModelPtr Item = WeakChildItem.Pin();

		if (!Item.IsValid() || NewUnsortableChildren.Contains(Item))
		{
			ItemsToRemove.Add(WeakChildItem);
		}
		else if (!NewSortableChildren.Contains(Item) || !Item->IsItemValid())
		{
			Item->SetParent(nullptr);
			ItemsToRemove.Add(WeakChildItem);
		}

		NewSortableChildren.Remove(Item);
		NewUnsortableChildren.Remove(Item);
	}

	WeakChildren.RemoveAll([&ItemsToRemove](const FNavigationToolViewModelWeakPtr& InWeakItem)
		{
			return ItemsToRemove.Contains(InWeakItem);
		});

	// Find children for new children in case these new children have grand children.
	// Note: This does not affect any of the current containers. It's just called for discovery.
	auto FindGrandChildren = [](const TSet<FNavigationToolViewModelWeakPtr>& InWeakChildren)
		{
			for (const FNavigationToolViewModelWeakPtr& WeakChild : InWeakChildren)
			{
				TArray<FNavigationToolViewModelWeakPtr> WeakGrandChildren;
				WeakChild.Pin()->FindValidChildren(WeakGrandChildren, /*bRecursiveFind=*/true);
			}
		};

	FindGrandChildren(NewUnsortableChildren);
	FindGrandChildren(NewSortableChildren);

	// After removing Children not present in Sortable
	// Children should either be equal in size with Sortable (which means no new sortable children were added)
	// or Sortable has more entries which means there are new items to add
	if (Sortable.Num() > WeakChildren.Num())
	{
		check(!NewSortableChildren.IsEmpty());
		HandleNewSortableChildren(NewSortableChildren.Array());
	}

	// Rearrange so that children are arranged like so:
	// [Unsortable Children][Sortable Children]
	Unsortable.Append(MoveTemp(WeakChildren));
	WeakChildren = MoveTemp(Unsortable);

	// Update the parents of every child in the list
	const FNavigationToolViewModelPtr ThisRef = SharedThis(this);
	for (const FNavigationToolViewModelWeakPtr& WeakChild : WeakChildren)
	{
		WeakChild.Pin()->SetParent(ThisRef);
	}
}

void FNavigationToolItem::ResetChildren()
{
	using namespace Sequencer;

	for (const FNavigationToolViewModelWeakPtr& WeakItem : GetChildren())
	{
		if (const FNavigationToolViewModelPtr Item = WeakItem.Pin())
		{
			Item->SetParent(nullptr);
		}
	}

	GetChildrenMutable().Reset();	
};

void FNavigationToolItem::FindChildren(TArray<FNavigationToolViewModelWeakPtr>& OutWeakChildren
	, const bool bInRecursive)
{
	FNavigationTool& ToolPrivate = static_cast<FNavigationTool&>(Tool);

	const FNavigationToolViewModelPtr ThisRef = SharedThis(this);

	TArray<TSharedPtr<FNavigationToolItemProxy>> ItemProxies;
	ToolPrivate.GetItemProxiesForItem(ThisRef, ItemProxies);
	OutWeakChildren.Reserve(OutWeakChildren.Num() + ItemProxies.Num());

	for (const TSharedPtr<FNavigationToolItemProxy>& ItemProxy : ItemProxies)
	{
		OutWeakChildren.Add(ItemProxy);
		if (bInRecursive)
		{
			ItemProxy->FindChildren(OutWeakChildren, bInRecursive);
		}
	}
}

void FNavigationToolItem::FindValidChildren(TArray<FNavigationToolViewModelWeakPtr>& OutWeakChildren
	, const bool bInRecursive)
{
	FindChildren(OutWeakChildren, bInRecursive);

	OutWeakChildren.RemoveAll([](const FNavigationToolViewModelWeakPtr& InWeakItem)
		{
			const FNavigationToolViewModelPtr Item = InWeakItem.Pin();
			return !Item.IsValid() || !Item->IsAllowedInTool();
		});
}

TArray<FNavigationToolViewModelPtr> FNavigationToolItem::FindPath(const TArray<FNavigationToolViewModelWeakPtr>& InWeakItems) const
{
	using namespace Sequencer;

	TArray<FNavigationToolViewModelPtr> Path;

	for (const FNavigationToolViewModelWeakPtr& WeakItem : InWeakItems)
	{
		Path.Reset();
		FNavigationToolViewModelPtr CurrentItem = WeakItem.Pin();

		while (CurrentItem.IsValid())
		{
			if (this == CurrentItem.Get())
			{
				Algo::Reverse(Path);
				return Path;
			}

			Path.Add(CurrentItem);
			CurrentItem = CurrentItem->GetParent();
		}
	}

	return TArray<FNavigationToolViewModelPtr>();
}

FString FNavigationToolItem::GetFullPath() const
{
	FString OutPath;

	FNavigationToolViewModelPtr CurrentParent = GetParent();
	while (CurrentParent.IsValid())
	{
		if (OutPath.IsEmpty())
		{
			OutPath = CurrentParent->GetItemId().GetStringId();
		}
		else
		{
			OutPath = CurrentParent->GetItemId().GetStringId() + TEXT(",") + OutPath;
		}

		CurrentParent = CurrentParent->GetParent();
	}

	return OutPath;
}

FNavigationToolItem::IndexType FNavigationToolItem::GetChildIndex(const FNavigationToolViewModelWeakPtr& InWeakChildItem) const
{
	return GetChildren().Find(InWeakChildItem);
}

FNavigationToolViewModelWeakPtr FNavigationToolItem::GetChildAt(const IndexType InIndex) const
{
	const TArray<FNavigationToolViewModelWeakPtr>& ChildItems = GetChildren();
	if (ChildItems.IsValidIndex(InIndex))
	{
		return ChildItems[InIndex].Pin();
	}
	return nullptr;
}

FNavigationToolViewModelPtr FNavigationToolItem::GetParent() const
{
	return FindAncestorOfType<FNavigationToolItem>();
}

TSet<FNavigationToolViewModelPtr> FNavigationToolItem::GetParents(const bool bInIncludeRoot) const
{
	TSet<FNavigationToolViewModelPtr> OutParents;

	FNavigationToolViewModelPtr CurrentParent = GetParent();
	while (CurrentParent.IsValid()
		&& (bInIncludeRoot || CurrentParent->GetItemId() != FNavigationToolItemId::RootId))
	{
		OutParents.Add(CurrentParent);

		CurrentParent = CurrentParent->GetParent();
	}

	return OutParents;
}

bool FNavigationToolItem::CanAddChild(const FNavigationToolViewModelPtr& InChild) const
{
	return InChild.IsValid();
}

bool FNavigationToolItem::AddChild(const FNavigationToolAddItemParams& InAddItemParams)
{
	const FNavigationToolViewModelPtr Item = InAddItemParams.WeakItem.Pin();
	if (CanAddChild(Item))
	{
		AddChildChecked(InAddItemParams);
		return true;
	}
	return false;
}

bool FNavigationToolItem::RemoveChild(const FNavigationToolRemoveItemParams& InRemoveItemParams)
{
	return RemoveChildChecked(InRemoveItemParams);
}

void FNavigationToolItem::SetParent(FNavigationToolViewModelPtr InParent)
{
	//check that one of the parent's children is this
	WeakParent = InParent;
}

ENavigationToolItemViewMode FNavigationToolItem::GetSupportedViewModes(const INavigationToolView& InToolView) const
{
	return InToolView.GetItemDefaultViewMode();
}
	
bool FNavigationToolItem::IsViewModeSupported(const ENavigationToolItemViewMode InViewMode, const INavigationToolView& InToolView) const
{
	return EnumHasAnyFlags(InViewMode, GetSupportedViewModes(InToolView));
}

FNavigationToolItemId FNavigationToolItem::GetItemId() const
{
	if (ItemId.IsValidId())
	{
		return ItemId;
	}
	const_cast<FNavigationToolItem*>(this)->RecalculateItemId();
	return ItemId;
}

FSlateColor FNavigationToolItem::GetItemLabelColor() const
{
	return FStyleColors::Foreground;
}

const FSlateBrush* FNavigationToolItem::GetIconBrush() const
{
	const FSlateIcon Icon = FNavigationToolExtender::FindOverrideIcon(AsItemViewModelConst());
	if (Icon.IsSet())
	{
		return Icon.GetIcon();
	}

	if (const FSlateBrush* const DefaultIconBrush = GetDefaultIconBrush())
	{
		return DefaultIconBrush;
	}

	return GetIcon().GetIcon();
}

FSlateColor FNavigationToolItem::GetIconColor() const
{
	return FStyleColors::Foreground;
}

TSharedRef<SWidget> FNavigationToolItem::GenerateLabelWidget(const TSharedRef<SNavigationToolTreeRow>& InRow)
{
	return SNew(SNavigationToolLabelItem, SharedThis(this), InRow);
}

bool FNavigationToolItem::Delete()
{
	GetOwnerTool().NotifyToolItemDeleted(SharedThis(this));
	return true;
}

void FNavigationToolItem::AddFlags(ENavigationToolItemFlags Flags)
{
	EnumAddFlags(ItemFlags, Flags);
}

void FNavigationToolItem::RemoveFlags(ENavigationToolItemFlags Flags)
{
	EnumRemoveFlags(ItemFlags, Flags);
}

bool FNavigationToolItem::HasAnyFlags(ENavigationToolItemFlags Flags) const
{
	return EnumHasAnyFlags(ItemFlags, Flags);
}

bool FNavigationToolItem::HasAllFlags(ENavigationToolItemFlags Flags) const
{
	return EnumHasAllFlags(ItemFlags, Flags);
}

TOptional<EItemDropZone> FNavigationToolItem::CanAcceptDrop(const FDragDropEvent& InDragDropEvent, const EItemDropZone InDropZone)
{
	if (const TSharedPtr<FNavigationToolItemDragDropOp> ItemDragDropOp = InDragDropEvent.GetOperationAs<FNavigationToolItemDragDropOp>())
	{
		const TOptional<EItemDropZone> DropZone = ItemDragDropOp->CanDrop(InDropZone, SharedThis(this));
		if (DropZone.IsSet())
		{
			if (GetItemId() != FNavigationToolItemId::RootId)
			{
				ItemDragDropOp->CurrentIconBrush = GetIconBrush();
			}
			return *DropZone;
		}

		ItemDragDropOp->CurrentIconBrush = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));
	}

	TOptional<EItemDropZone> OutDropZone;

	GetOwnerTool().ForEachProvider([this, &InDragDropEvent, InDropZone, &OutDropZone]
		(const TSharedRef<FNavigationToolProvider>& InProvider)
		{
			OutDropZone = InProvider->OnToolItemCanAcceptDrop(InDragDropEvent, InDropZone, SharedThis(this));
			if (OutDropZone.IsSet())
			{
				return false;
			}
			return true;
		});

	return OutDropZone;
}

FReply FNavigationToolItem::AcceptDrop(const FDragDropEvent& InDragDropEvent, const EItemDropZone InDropZone)
{
	if (const TSharedPtr<FNavigationToolItemDragDropOp> ItemDragDropOp = InDragDropEvent.GetOperationAs<FNavigationToolItemDragDropOp>())
	{
		const FReply Reply = ItemDragDropOp->Drop(InDropZone, SharedThis(this));
		if (Reply.IsEventHandled())
		{
			return Reply;
		}
	}

	FReply OutReply = FReply::Unhandled();

	GetOwnerTool().ForEachProvider([this, &InDragDropEvent, InDropZone, &OutReply]
		(const TSharedRef<FNavigationToolProvider>& InProvider)
		{
			OutReply = InProvider->OnToolItemAcceptDrop(InDragDropEvent, InDropZone, SharedThis(this));
			if (OutReply.IsEventHandled())
			{
				return false;
			}
			return true;
		});

	return OutReply;
}

bool FNavigationToolItem::IsIgnoringPendingKill() const
{
	return HasAllFlags(ENavigationToolItemFlags::IgnorePendingKill);
}

void FNavigationToolItem::OnObjectsReplaced(const TMap<UObject*, UObject*>& InReplacementMap, const bool bInRecursive)
{
	using namespace Sequencer;

	if (bInRecursive)
	{
		for (const FNavigationToolViewModelWeakPtr& WeakChildItem : GetChildren())
		{
			if (const FNavigationToolViewModelPtr ChildItem = WeakChildItem.Pin())
			{
				ChildItem->OnObjectsReplaced(InReplacementMap, bInRecursive);
			}
		}
	}
}

FLinearColor FNavigationToolItem::GetItemTintColor() const
{
	return FStyleColors::White.GetSpecifiedColor();
}

bool FNavigationToolItem::IsExpanded() const
{
	if (const TSharedPtr<INavigationToolView> ToolView = Tool.GetMostRecentToolView())
	{
		return ToolView->IsItemExpanded(SharedThis(const_cast<FNavigationToolItem*>(this)));
	}
	return false;
}

void FNavigationToolItem::SetExpansion(const bool bInExpand)
{
	if (const TSharedPtr<INavigationToolView> ToolView = Tool.GetMostRecentToolView())
	{
		return ToolView->SetItemExpansion(SharedThis(this), bInExpand);
	}
}

TOptional<FColor> FNavigationToolItem::GetColor() const
{
	return Tool.FindItemColor(SharedThis(const_cast<FNavigationToolItem*>(this)));
}

void FNavigationToolItem::SetColor(const TOptional<FColor>& InColor)
{
	Tool.SetItemColor(SharedThis(this), InColor.Get(FColor()));
}

FNavigationToolSerializedItem FNavigationToolItem::MakeSerializedItem()
{
	return FNavigationToolSerializedItem(GetItemId().GetStringId());
}

FOutlinerSizing FNavigationToolItem::GetOutlinerSizing() const
{
	int32 Height = 0;
	FNavigationToolViewModelPtr TopParent = GetParent();
	while (TopParent.IsValid())
	{
		TopParent = TopParent->GetParent();
		++Height;
	}
	return Height;
}

void FNavigationToolItem::SetProvider(const TWeakPtr<FNavigationToolProvider>& InWeakProvider)
{
	WeakProvider = InWeakProvider;
}

void FNavigationToolItem::RecalculateItemId()
{
	const FNavigationToolItemId OldItemId = ItemId;
	ItemId = CalculateItemId();

	FNavigationTool& ToolPrivate = static_cast<FNavigationTool&>(Tool);
	ToolPrivate.NotifyItemIdChanged(OldItemId, SharedThis(this));
}

void FNavigationToolItem::AddChildChecked(const FNavigationToolAddItemParams& InAddItemParams)
{
	using namespace Sequencer;

	const FNavigationToolViewModelPtr ItemToAdd = InAddItemParams.WeakItem.Pin();
	if (!ItemToAdd.IsValid())
	{
		return;
	}

	if (const FNavigationToolViewModelPtr OldParent = ItemToAdd->GetParent())
	{
		// If we are adding the child and the old parent is this,
		// then it means we're just rearranging, only remove from array
		if (OldParent.Get() == this)
		{
			WeakChildren.Remove(InAddItemParams.WeakItem);
		}
		else
		{
			FNavigationToolRemoveItemParams RemoveParams(ItemToAdd);
			OldParent->RemoveChild(RemoveParams);
		}
	}

	if (ItemToAdd.IsValid()
		&& InAddItemParams.RelativeDropZone.IsSet()
		&& InAddItemParams.RelativeDropZone != EItemDropZone::OntoItem)
	{
		const int32 RelativeItemIndex = WeakChildren.Find(InAddItemParams.WeakRelativeItem);
		if (RelativeItemIndex != INDEX_NONE)
		{
			const int32 TargetIndex = InAddItemParams.RelativeDropZone == EItemDropZone::BelowItem
				? RelativeItemIndex + 1
				: RelativeItemIndex;

			WeakChildren.EmplaceAt(TargetIndex, InAddItemParams.WeakItem);
		}
		else
		{
			WeakChildren.EmplaceAt(0, InAddItemParams.WeakItem);
		}
	}
	else
	{
		WeakChildren.EmplaceAt(0, InAddItemParams.WeakItem);
	}

	ItemToAdd->SetParent(SharedThis(this));
}

bool FNavigationToolItem::RemoveChildChecked(const FNavigationToolRemoveItemParams& InRemoveItemParams)
{
	if (const FNavigationToolViewModelPtr Item = InRemoveItemParams.WeakItem.Pin())
	{
		Item->SetParent(nullptr);
	}
	return WeakChildren.Remove(InRemoveItemParams.WeakItem) > 0;
}

void FNavigationToolItem::HandleNewSortableChildren(TArray<FNavigationToolViewModelWeakPtr> InWeakSortableChildren)
{
	InWeakSortableChildren.Sort([this](const FNavigationToolViewModelWeakPtr& InWeakItemA, const FNavigationToolViewModelWeakPtr& InWeakItemB)
		{
			const TViewModelPtr<INavigationToolItem> ItemA = InWeakItemA.Pin();
			const FNavigationToolSaveState* const SaveStateA = ItemA->GetProviderSaveState();
			if (!SaveStateA)
			{
				return false;
			}

			const TViewModelPtr<INavigationToolItem> ItemB = InWeakItemB.Pin();
			const FNavigationToolSaveState* const SaveStateB = ItemB->GetProviderSaveState();
			if (!SaveStateB)
			{
				return false;
			}

			const FNavigationToolSerializedTreeNode* const NodeA = SaveStateA->SerializedTree.FindTreeNode(ItemA->MakeSerializedItem());
			const FNavigationToolSerializedTreeNode* const NodeB = SaveStateB->SerializedTree.FindTreeNode(ItemB->MakeSerializedItem());

			return FNavigationToolSerializedTree::CompareTreeItemOrder(NodeA, NodeB);
		});

	FNavigationToolAddItemParams AddItemParams;
	for (const FNavigationToolViewModelWeakPtr& WeakNewChild : InWeakSortableChildren)
	{
		const FNavigationToolViewModelPtr NewChild = WeakNewChild.Pin();
		if (!NewChild.IsValid())
		{
			continue;
		}

		const FNavigationToolSaveState* const SaveState = NewChild->GetProviderSaveState();
		if (!SaveState)
		{
			continue;
		}

		AddItemParams.WeakItem = WeakNewChild;

		const FNavigationToolSerializedTreeNode* const TreeNode = SaveState->SerializedTree.FindTreeNode(NewChild->MakeSerializedItem());
		if (TreeNode && WeakChildren.IsValidIndex(TreeNode->GetLocalIndex()))
		{
			// Add Before the Child at Index, so this Item is at the specific Index
			AddItemParams.WeakRelativeItem = WeakChildren[TreeNode->GetLocalIndex()];
			AddItemParams.RelativeDropZone = EItemDropZone::AboveItem;
		}
		else
		{
			// Add After Last, so this Item is the last item in the List
			AddItemParams.WeakRelativeItem = WeakChildren.IsEmpty() ? nullptr : WeakChildren.Last();
			AddItemParams.RelativeDropZone = EItemDropZone::BelowItem;
		}

		AddChild(AddItemParams);
	}
}

FNavigationToolSaveState* FNavigationToolItem::GetProviderSaveState() const
{
	if (const TSharedPtr<FNavigationToolProvider> Provider = GetProvider())
	{
		return Provider->GetSaveState(Tool);
	}
	return nullptr;
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
