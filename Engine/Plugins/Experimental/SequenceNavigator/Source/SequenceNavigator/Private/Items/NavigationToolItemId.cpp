// Copyright Epic Games, Inc. All Rights Reserved.

#include "Items/NavigationToolItemId.h"
#include "ItemProxies/INavigationToolItemProxyFactory.h"
#include "Items/NavigationToolItem.h"
#include "MovieScene.h"
#include "MovieSceneSection.h"
#include "MovieSceneSequence.h"
#include "MVVM/Extensions/IObjectBindingExtension.h"
#include "MVVM/Extensions/ITrackExtension.h"
#include "MVVM/ViewModels/SectionModel.h"
#include "MVVM/ViewModels/SequenceModel.h"
#include "UObject/SoftObjectPath.h"
#include "MVVM/ViewModels/ViewModelIterators.h"

namespace UE::SequenceNavigator
{

FNavigationToolItemId FNavigationToolItemId::RootId = FNavigationToolItemId(TEXT("Root"));

FNavigationToolItemId::FNavigationToolItemId(const FString& InUniqueId)
{
	Id = InUniqueId;
	CalculateTypeHash();
}

FNavigationToolItemId::FNavigationToolItemId(const UObject* InObject
	, const FNavigationToolViewModelPtr& InReferencingItem
	, const FString& InReferencingId)
	: FNavigationToolItemId(InReferencingItem->GetItemId())
{
	AddSeparatedSegment(Id, GetObjectPath(InObject));
	AddSeparatedSegment(Id, InReferencingId);
	CalculateTypeHash();
}

FNavigationToolItemId::FNavigationToolItemId(const FNavigationToolViewModelPtr& InParentItem
	, const INavigationToolItemProxyFactory& InItemProxyFactory)
	: FNavigationToolItemId(InParentItem->GetItemId())
{
	AddSeparatedSegment(Id, InItemProxyFactory.GetItemProxyTypeName().ToString());
	CalculateTypeHash();
}

FNavigationToolItemId::FNavigationToolItemId(const FNavigationToolViewModelPtr& InParentItem
	, const FNavigationToolViewModelPtr& InItemProxy)
	: FNavigationToolItemId(InParentItem->GetItemId())
{
	AddSeparatedSegment(Id, InItemProxy.AsModel()->GetTypeTable().GetTypeName().ToString());
	CalculateTypeHash();
}

FNavigationToolItemId::FNavigationToolItemId(const FNavigationToolViewModelPtr& InParentItem
	, const UMovieSceneSequence* const InSequence
	, const UMovieSceneSection* const InSection
	, const int32 InSectionIndex
	, const FString& InReferenceId)
{
	ConstructId(InParentItem, InSequence, InSection, InSectionIndex, InReferenceId);
	CalculateTypeHash();
}

FNavigationToolItemId::FNavigationToolItemId(const UE::Sequencer::FViewModelPtr& InViewModel)
{
	using namespace Sequencer;

	check(InViewModel.IsValid());

	const TViewModelPtr<FSectionModel> SectionModel = InViewModel->FindAncestorOfType<FSectionModel>(true);
	const TViewModelPtr<FSequenceModel> SequenceModel = InViewModel->FindAncestorOfType<FSequenceModel>();

	UMovieSceneSequence* const Sequence = SequenceModel->GetSequence();
	UMovieSceneSection* const Section = SectionModel.IsValid() ? SectionModel->GetSection() : nullptr;

	int32 SectionIndex = 0;
	FString ReferenceId;

	const TArray<UMovieSceneSection*> AllSections = SequenceModel->GetMovieScene()->GetAllSections();
	for (int32 Index = 0; Index < AllSections.Num(); ++Index)
	{
		if (AllSections[Index] == Section)
		{
			SectionIndex = Index;
			break;
		}
	}

	if (const TViewModelPtr<ITrackExtension> TrackExtension = InViewModel.ImplicitCast())
	{
		if (const UMovieSceneTrack* const Track = TrackExtension->GetTrack())
		{
			ReferenceId = GetObjectPath(Track);
		}
	}
	else if (const TViewModelPtr<IObjectBindingExtension> ObjectBindingExtension = InViewModel.ImplicitCast())
	{
		ReferenceId = ObjectBindingExtension->GetObjectGuid().ToString();
	}

	ConstructId(nullptr, Sequence, Section, SectionIndex, ReferenceId);

	CalculateTypeHash();
}

void FNavigationToolItemId::ConstructId(const FNavigationToolViewModelPtr& InParentItem
	, const UMovieSceneSequence* const InSequence
	, const UMovieSceneSection* const InSection
	, const int32 InSectionIndex
	, const FString& InReferenceId)
{
	check(InSequence);

	if (InParentItem.IsValid())
	{
		AddSeparatedSegment(Id, InParentItem->GetFullPath());
	}

	AddSeparatedSegment(Id, GetObjectPath(InSequence));

	if (InSection)
	{
		AddSeparatedSegment(Id, GetObjectPath(InSection));
		AddSeparatedSegment(Id, FString::FromInt(InSectionIndex));
	}

	if (!InReferenceId.IsEmpty())
	{
		AddSeparatedSegment(Id, InReferenceId);
	}
}

FNavigationToolItemId::FNavigationToolItemId(const FNavigationToolItemId& Other)
{
	*this = Other;
}

FNavigationToolItemId::FNavigationToolItemId(FNavigationToolItemId&& Other) noexcept
{
	*this = MoveTemp(Other);
}

FNavigationToolItemId& FNavigationToolItemId::operator=(const FNavigationToolItemId& Other)
{
	Id = Other.Id;
	CalculateTypeHash();
	return *this;
}

FNavigationToolItemId& FNavigationToolItemId::operator=(FNavigationToolItemId&& Other) noexcept
{
	Swap(*this, Other);
	CalculateTypeHash();
	return *this;
}

bool FNavigationToolItemId::operator==(const FNavigationToolItemId& Other) const
{
	return Id == Other.Id && CachedHash == Other.CachedHash;
}

bool FNavigationToolItemId::IsValidId() const
{
	return bHasCachedHash;
}

FString FNavigationToolItemId::GetStringId() const
{
	return Id;
}

FString FNavigationToolItemId::GetObjectPath(const UObject* const InObject)
{
	return FSoftObjectPath(InObject).ToString();
}

void FNavigationToolItemId::AddSeparatedSegment(FString& OutString, const FString& InSegment)
{
	if (OutString.IsEmpty())
	{
		OutString = InSegment;
		return;
	}

	OutString += Separator + InSegment;
}

void FNavigationToolItemId::CalculateTypeHash()
{
	CachedHash = GetTypeHash(Id);
	bHasCachedHash = true;
}

} // namespace UE::SequenceNavigator
