// Copyright Epic Games, Inc. All Rights Reserved.

#include "Items/NavigationToolItemProxy.h"
#include "Input/Reply.h"
#include "NavigationTool.h"
#include "NavigationToolView.h"

namespace UE::SequenceNavigator
{

using namespace Sequencer;

UE_SEQUENCER_DEFINE_CASTABLE(FNavigationToolItemProxy)

FNavigationToolItemProxy::FNavigationToolItemProxy(INavigationTool& InTool, const FNavigationToolViewModelPtr& InParentItem)
	: FNavigationToolItem(InTool, InParentItem)
{
	WeakParent = InParentItem;
}

bool FNavigationToolItemProxy::IsItemValid() const
{
	return FNavigationToolItem::IsItemValid()
		&& WeakParent.Pin().IsValid()
		&& Tool.FindItem(WeakParent.Pin()->GetItemId()).IsValid();
}

void FNavigationToolItemProxy::FindChildren(TArray<FNavigationToolViewModelWeakPtr>& OutWeakChildren, const bool bInRecursive)
{
	const FNavigationToolViewModelPtr Parent = GetParent();
	if (!Parent.IsValid() || !Parent->IsAllowedInTool())
	{
		return;
	}

	FNavigationToolItem::FindChildren(OutWeakChildren, bInRecursive);

	GetProxiedItems(Parent, OutWeakChildren, bInRecursive);
}

void FNavigationToolItemProxy::SetParent(FNavigationToolViewModelPtr InParent)
{
	FNavigationToolItem::SetParent(InParent);

	// Recalculate our item Id because we rely on what our parent is for our Id
	RecalculateItemId();
}

ENavigationToolItemViewMode FNavigationToolItemProxy::GetSupportedViewModes(const INavigationToolView& InToolView) const
{
	// Hide proxies if it has no children
	if (WeakChildren.IsEmpty())
	{
		return ENavigationToolItemViewMode::None;
	}
	return InToolView.GetItemProxyViewMode();
}

bool FNavigationToolItemProxy::CanAutoExpand() const
{
	return false;
}

FNavigationToolItemId FNavigationToolItemProxy::CalculateItemId() const
{
	if (const FNavigationToolViewModelPtr Parent = WeakParent.Pin())
	{
		const FNavigationToolViewModelPtr ViewModel = AsItemViewModelConst();
		return FNavigationToolItemId(Parent, ViewModel);	
	}
	return FNavigationToolItemId();
}

} // namespace UE::SequenceNavigator
