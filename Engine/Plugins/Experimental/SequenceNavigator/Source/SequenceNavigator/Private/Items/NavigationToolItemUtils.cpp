// Copyright Epic Games, Inc. All Rights Reserved.

#include "Items/NavigationToolItemUtils.h"
#include "Containers/Array.h"
#include "Items/INavigationToolItem.h"
#include "Items/NavigationToolSequence.h"
#include "Items/NavigationToolTreeRoot.h"
#include "MovieSceneSequence.h"
#include "NavigationTool.h"
#include "ScopedTransaction.h"
#include "Sections/MovieSceneCinematicShotSection.h"
#include "Sections/MovieSceneSubSection.h"
#include "TrackEditors/SubTrackEditorBase.h"
#include "Utils/NavigationToolMiscUtils.h"

#define LOCTEXT_NAMESPACE "NavigationToolItemUtils"

namespace UE::SequenceNavigator::ItemUtils
{

using namespace Sequencer;

bool CompareToolItemOrder(const FNavigationToolViewModelPtr& InA, const FNavigationToolViewModelPtr& InB)
{
	if (!InA.IsValid())
	{
		return false;
	}

	if (!InB.IsValid())
	{
		return false;
	}

	if (const FNavigationToolViewModelPtr LowestCommonAncestor = FNavigationTool::FindLowestCommonAncestor({ InA, InB }))
	{
		const TArray<FNavigationToolViewModelPtr> PathToA = LowestCommonAncestor->FindPath({ InA });
		const TArray<FNavigationToolViewModelPtr> PathToB = LowestCommonAncestor->FindPath({ InB });

		int32 Index = 0;
	
		int32 PathAIndex = -1;
		int32 PathBIndex = -1;

		while (PathAIndex == PathBIndex)
		{
			if (!PathToA.IsValidIndex(Index))
			{
				return true;
			}
			if (!PathToB.IsValidIndex(Index))
			{
				return false;
			}

			PathAIndex = LowestCommonAncestor->GetChildIndex(PathToA[Index]);
			PathBIndex = LowestCommonAncestor->GetChildIndex(PathToB[Index]);
			++Index;
		}

		return PathAIndex < PathBIndex;
	}

	return false;
}

void SplitSortableAndUnsortableItems(const TArray<FNavigationToolViewModelWeakPtr>& InWeakItems
	, TArray<FNavigationToolViewModelWeakPtr>& OutWeakSortable
	, TArray<FNavigationToolViewModelWeakPtr>& OutWeakUnsortable)
{
	// Allocate both for worst case scenarios
	OutWeakSortable.Reserve(InWeakItems.Num());
	OutWeakUnsortable.Reserve(InWeakItems.Num());

	for (const FNavigationToolViewModelWeakPtr& WeakItem : InWeakItems)
	{
		const FNavigationToolViewModelPtr Item = WeakItem.Pin();
		if (!Item.IsValid())
		{
			continue;
		}

		if (Item->ShouldSort())
		{
			OutWeakSortable.Add(WeakItem);
		}
		else
		{
			OutWeakUnsortable.Add(WeakItem);
		}
	}
}

UMovieSceneSubSection* GetSequenceItemSubSection(const FNavigationToolViewModelPtr& InItem)
{
	if (const TViewModelPtr<FNavigationToolSequence> SequenceItem = InItem.ImplicitCast())
	{
		return SequenceItem->GetSubSection();
	}
	return nullptr;
}

UMovieSceneMetaData* GetSequenceItemMetaData(const FNavigationToolViewModelPtr& InItem)
{
	if (const TViewModelPtr<FNavigationToolSequence> SequenceItem = InItem.ImplicitCast())
	{
		return FSubTrackEditorUtil::FindOrAddMetaData(SequenceItem->GetSequence());
	}
	return nullptr;
}

void RemoveSequenceDisplayNameParentPrefix(FText& InOutDisplayName
	, const TViewModelPtr<FNavigationToolSequence>& InSequenceItem)
{
	if (!InSequenceItem.IsValid())
	{
		return;
	}

	UMovieSceneSequence* const Sequence = InSequenceItem->GetSequence();
	if (!Sequence)
	{
		return;
	}

	const FNavigationToolViewModelPtr Parent = InSequenceItem->GetParent();
	if (!Parent.IsValid() || Parent.AsModel()->IsA<FNavigationToolTreeRoot>())
	{
		return;
	}

	// Go up the hierarchy to find the parent sequence
	const TViewModelPtr<FNavigationToolSequence> ParentSequenceItem = InSequenceItem->FindAncestorOfType<FNavigationToolSequence>();
	if (!ParentSequenceItem)
	{
		return;
	}

	UMovieSceneSequence* const ParentSequence = ParentSequenceItem->GetSequence();
	if (!ParentSequence)
	{
		return;
	}

	const UMovieSceneCinematicShotSection* const ShotSection = Cast<UMovieSceneCinematicShotSection>(InSequenceItem->GetSubSection());

	FText NewSequenceName = ShotSection ? FText::FromString(ShotSection->GetShotDisplayName()) : Sequence->GetDisplayName();
	if (NewSequenceName.IsEmpty())
	{
		return;
	}

	// Remove the parent prefix from the child name
	static FString SeparatorString = TEXT("_");
	const FString ParentSequenceNameStr = ParentSequence->GetDisplayName().ToString();

	FString Prefix;
	FString BaseName;
	if (ParentSequenceNameStr.Split(SeparatorString, &Prefix, &BaseName))
	{
		Prefix += SeparatorString;

		FString NewSequenceNameStr = NewSequenceName.ToString();

		if (NewSequenceNameStr.StartsWith(Prefix)
			&& NewSequenceNameStr.Len() > Prefix.Len())
		{
			NewSequenceNameStr.RemoveFromStart(Prefix);
			NewSequenceName = FText::FromString(NewSequenceNameStr);
		}
	}

	InOutDisplayName = NewSequenceName;
}

void AppendSequenceDisplayNameDirtyStatus(FText& InOutDisplayName
	, const UMovieSceneSequence& InSequence)
{
	UPackage* const Package = InSequence.GetPackage();
	if (Package && Package->IsDirty())
	{
		InOutDisplayName = FText::Format(LOCTEXT("DirtySymbol", "{0}*"), InOutDisplayName);
	}
}

FSlateColor GetItemBindingColor(const ISequencer& InSequencer
	, UMovieSceneSequence& InSequence
	, const FGuid& InObjectGuid
	, const FSlateColor& DefaultColor)
{
	const TArrayView<TWeakObjectPtr<>> BoundObjects = ResolveBoundObjects(InSequencer, &InSequence, InObjectGuid);
	if (BoundObjects.Num() > 0)
	{
		int32 NumValidObjects = 0;

		for (const TWeakObjectPtr<>& BoundObject : BoundObjects)
		{
			if (BoundObject.IsValid())
			{
				++NumValidObjects;
			}
		}

		if (NumValidObjects == BoundObjects.Num())
		{
			return DefaultColor;
		}

		if (NumValidObjects > 0)
		{
			return FStyleColors::Warning;
		}
	}

	return FStyleColors::Error;
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
