// Copyright Epic Games, Inc. All Rights Reserved.

#include "Items/NavigationToolMarker.h"
#include "ISequencer.h"
#include "Items/NavigationToolSequence.h"
#include "MovieScene.h"
#include "NavigationTool.h"
#include "NavigationToolScopedSelection.h"
#include "NavigationToolStyle.h"
#include "ScopedTransaction.h"
#include "Sections/MovieSceneSubSection.h"
#include "SequencerSettings.h"
#include "Styling/StyleColors.h"
#include "Utils/NavigationToolMiscUtils.h"

#define LOCTEXT_NAMESPACE "NavigationToolMarker"

namespace UE::SequenceNavigator
{

using namespace Sequencer;

UE_SEQUENCER_DEFINE_CASTABLE(FNavigationToolMarker)

FNavigationToolMarker::FNavigationToolMarker(INavigationTool& InTool
	, const FNavigationToolViewModelPtr& InParentItem
	, const TSharedPtr<FNavigationToolSequence>& InParentSequenceItem
	, const int32 InMarkedFrameIndex)
	: FNavigationToolItem(InTool, InParentItem)
	, WeakParentSequenceItem(InParentSequenceItem)
	, MarkedFrameIndex(InMarkedFrameIndex)
{
}

bool FNavigationToolMarker::IsItemValid() const
{
	return WeakParentSequenceItem.Pin().IsValid() && MarkedFrameIndex != INDEX_NONE;
}

bool FNavigationToolMarker::IsAllowedInTool() const
{
	return IsItemValid();
}

ENavigationToolItemViewMode FNavigationToolMarker::GetSupportedViewModes(const INavigationToolView& InToolView) const
{
	return ENavigationToolItemViewMode::ItemTree | ENavigationToolItemViewMode::HorizontalItemList;
}

FText FNavigationToolMarker::GetDisplayName() const
{
	if (const FMovieSceneMarkedFrame* const MarkedFrame = GetMarkedFrame())
	{
		return FText::FromString(MarkedFrame->Label);
	}
	return FText::GetEmpty();
}

FText FNavigationToolMarker::GetClassName() const
{
	return FMovieSceneMarkedFrame::StaticStruct()->GetDisplayNameText();
}

FSlateIcon FNavigationToolMarker::GetIcon() const
{
	return FSlateIcon(FNavigationToolStyle::Get().GetStyleSetName(), TEXT("Item.Marker.Icon"));
}

const FSlateBrush* FNavigationToolMarker::GetIconBrush() const
{
	return FNavigationToolStyle::Get().GetBrush(TEXT("Item.Marker.Icon"));
}

FSlateColor FNavigationToolMarker::GetIconColor() const
{
	const TSharedPtr<ISequencer> Sequencer = Tool.GetSequencer();
	if (!Sequencer.IsValid())
	{
		return FStyleColors::Foreground;
	}

	USequencerSettings* const SequencerSettings = Sequencer->GetSequencerSettings();
	if (!SequencerSettings)
	{
		return FStyleColors::Foreground;
	}

	return SequencerSettings->GetMarkedFrameColor();
}

FText FNavigationToolMarker::GetIconTooltipText() const
{
	if (const FMovieSceneMarkedFrame* const MarkedFrame = GetMarkedFrame())
	{
		return FText::FromString(MarkedFrame->Label);
	}
	return FText::GetEmpty();
}

bool FNavigationToolMarker::IsSelected(const FNavigationToolScopedSelection& InSelection) const
{
	return InSelection.IsSelected(GetParentSequence(), MarkedFrameIndex);
}

void FNavigationToolMarker::Select(FNavigationToolScopedSelection& InSelection) const
{
	InSelection.Select(GetParentSequence(), MarkedFrameIndex);
}

void FNavigationToolMarker::OnSelect()
{
	FNavigationToolItem::OnSelect();
}

void FNavigationToolMarker::OnDoubleClick()
{
	UMovieSceneSequence* const Sequence = GetParentSequence();
	if (!Sequence)
	{
		return;
	}

	const FMovieSceneMarkedFrame* const MarkedFrame = GetMarkedFrame();
	if (!MarkedFrame)
	{
		return;
	}

	FocusSequence(Tool, *Sequence, *MarkedFrame);
}

FNavigationToolItemId FNavigationToolMarker::CalculateItemId() const
{
	const TSharedPtr<FNavigationToolSequence> ParentSequenceItem = WeakParentSequenceItem.Pin();
	check(ParentSequenceItem.IsValid());

	FString MarkId = TEXT("Mark");
	FNavigationToolItemId::AddSeparatedSegment(MarkId, FString::FromInt(MarkedFrameIndex));

	return FNavigationToolItemId(GetParent()
		, ParentSequenceItem->GetSequence()
		, ParentSequenceItem->GetSubSection()
		, ParentSequenceItem->GetSubSectionIndex()
		, MarkId);
}

bool FNavigationToolMarker::CanDelete() const
{
	return IsItemValid();
}

bool FNavigationToolMarker::Delete()
{
	if (!IsItemValid())
	{
		return false;
	}

	UMovieScene* const MovieScene = GetParentMovieScene();
	if (!MovieScene)
	{
		return false;
	}

	const FScopedTransaction Transaction(LOCTEXT("NavigationToolDeleteMarker", "Delete Marker"));

	MovieScene->Modify();
	MovieScene->DeleteMarkedFrame(MarkedFrameIndex);

	return true;
}
	
bool FNavigationToolMarker::CanRename() const
{
	if (!IsItemValid())
	{
		return false;
	}

	const TSharedPtr<FNavigationToolSequence> ParentSequenceItem = WeakParentSequenceItem.Pin();
	if (!ParentSequenceItem.IsValid())
	{
		return false;
	}

	const UMovieScene* const MovieScene = ParentSequenceItem->GetSequenceMovieScene();
	if (!MovieScene)
	{
		return false;
	}

	return MovieScene->CanModify();
}

void FNavigationToolMarker::Rename(const FText& InNewName)
{
	const TViewModelPtr<FNavigationToolSequence> ParentSequenceItem = WeakParentSequenceItem.ImplicitPin();
	if (!ParentSequenceItem.IsValid() || MarkedFrameIndex == INDEX_NONE)
	{
		return;
	}

	UMovieScene* const MovieScene = WeakParentSequenceItem.Pin()->GetSequenceMovieScene();
	if (!MovieScene)
	{
		return;
	}

	const TArray<FMovieSceneMarkedFrame>& MarkedFrames = MovieScene->GetMarkedFrames();
	if (!MarkedFrames.IsValidIndex(MarkedFrameIndex))
	{
		return;
	}

	FMovieSceneMarkedFrame& MarkedFrame = const_cast<FMovieSceneMarkedFrame&>(MarkedFrames[MarkedFrameIndex]);

	const FString NewNameString = InNewName.ToString();

	if (NewNameString.Equals(MarkedFrame.Label, ESearchCase::CaseSensitive))
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("RenameTransaction", "Rename Marker"));

	MovieScene->Modify();

	MarkedFrame.Label = NewNameString;
}

FFrameNumber FNavigationToolMarker::GetInTime() const
{
	const TViewModelPtr<FNavigationToolSequence> ParentSequenceItem = WeakParentSequenceItem.ImplicitPin();
	if (!ParentSequenceItem.IsValid() || MarkedFrameIndex == INDEX_NONE)
	{
		return 0;
	}

	UMovieSceneSequence* const Sequence = WeakParentSequenceItem.Pin()->GetSequence();
	if (!Sequence)
	{
		return 0;
	}

	UMovieScene* const MovieScene = Sequence->GetMovieScene();
	if (!MovieScene)
	{
		return 0;
	}

	const TArray<FMovieSceneMarkedFrame>& MarkedFrames = MovieScene->GetMarkedFrames();
	if (!MarkedFrames.IsValidIndex(MarkedFrameIndex))
	{
		return 0;
	}

	FMovieSceneMarkedFrame& MarkedFrame = const_cast<FMovieSceneMarkedFrame&>(MarkedFrames[MarkedFrameIndex]);

	return MarkedFrame.FrameNumber;
}

void FNavigationToolMarker::SetInTime(const FFrameNumber& InTime)
{
	const TViewModelPtr<FNavigationToolSequence> ParentSequenceItem = WeakParentSequenceItem.ImplicitPin();
	if (!ParentSequenceItem.IsValid() || MarkedFrameIndex == INDEX_NONE)
	{
		return;
	}

	UMovieSceneSequence* const Sequence = WeakParentSequenceItem.Pin()->GetSequence();
	if (!Sequence)
	{
		return;
	}

	UMovieScene* const MovieScene = Sequence->GetMovieScene();
	if (!MovieScene)
	{
		return;
	}

	const TArray<FMovieSceneMarkedFrame>& MarkedFrames = MovieScene->GetMarkedFrames();
	if (!MarkedFrames.IsValidIndex(MarkedFrameIndex))
	{
		return;
	}

	FMovieSceneMarkedFrame& MarkedFrame = const_cast<FMovieSceneMarkedFrame&>(MarkedFrames[MarkedFrameIndex]);

	if (InTime == MarkedFrame.FrameNumber)
	{
		return;
	}

	MovieScene->Modify();
	MarkedFrame.FrameNumber = InTime;

	if (const TSharedPtr<ISequencer> Sequencer = GetOwnerTool().GetSequencer())
	{
		//Sequencer->InvalidateGlobalMarkedFramesCache();
	}
}

int32 FNavigationToolMarker::GetMarkedFrameIndex() const
{
	return MarkedFrameIndex;
}

FMovieSceneMarkedFrame* FNavigationToolMarker::GetMarkedFrame() const
{
	if (!IsItemValid())
	{
		return nullptr;
	}

	UMovieScene* const MovieScene = GetParentMovieScene();
	if (!MovieScene)
	{
		return nullptr;
	}

	const TArray<FMovieSceneMarkedFrame>& MarkedFrames = MovieScene->GetMarkedFrames();
	if (!MarkedFrames.IsValidIndex(MarkedFrameIndex))
	{
		return nullptr;
	}

	return const_cast<FMovieSceneMarkedFrame*>(&MarkedFrames[MarkedFrameIndex]);
}

UMovieSceneSequence* FNavigationToolMarker::GetParentSequence() const
{
	if (const TSharedPtr<FNavigationToolSequence> ParentSequenceItem = WeakParentSequenceItem.Pin())
	{
		return ParentSequenceItem->GetSequence();
	}
	return nullptr;
}

UMovieScene* FNavigationToolMarker::GetParentMovieScene() const
{
	if (const TSharedPtr<FNavigationToolSequence> ParentSequenceItem = WeakParentSequenceItem.Pin())
	{
		return ParentSequenceItem->GetSequenceMovieScene();
	}
	return nullptr;
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
