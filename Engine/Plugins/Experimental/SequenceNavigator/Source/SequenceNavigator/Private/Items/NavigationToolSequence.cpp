// Copyright Epic Games, Inc. All Rights Reserved.

#include "Items/NavigationToolSequence.h"
#include "Compilation/MovieSceneCompiledDataManager.h"
#include "GameFramework/Actor.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "Evaluation/MovieSceneSequenceHierarchy.h"
#include "INavigationTool.h"
#include "Items/NavigationToolActor.h"
#include "Items/NavigationToolItemUtils.h"
#include "Items/NavigationToolMarker.h"
#include "Items/NavigationToolSubTrack.h"
#include "Items/NavigationToolTrack.h"
#include "LevelSequenceActor.h"
#include "Misc/QualifiedFrameTime.h"
#include "MovieScene.h"
#include "MVVM/SectionModelStorageExtension.h"
#include "MVVM/ViewModels/SectionModel.h"
#include "NavigationToolScopedSelection.h"
#include "NavigationToolSettings.h"
#include "ScopedTransaction.h"
#include "Sections/MovieSceneCinematicShotSection.h"
#include "Sections/MovieSceneSubSection.h"
#include "SequencerChannelInterface.h"
#include "SequencerSettings.h"
#include "Styling/SlateIconFinder.h"
#include "Tracks/MovieSceneCinematicShotTrack.h"
#include "Utils/NavigationToolMiscUtils.h"
#include "Utils/NavigationToolMovieSceneUtils.h"

#define LOCTEXT_NAMESPACE "NavigationToolSequence"

namespace UE::SequenceNavigator
{

using namespace Sequencer;

UE_SEQUENCER_DEFINE_CASTABLE(FNavigationToolSequence)

FNavigationToolSequence::FNavigationToolSequence(INavigationTool& InTool
	, const FNavigationToolViewModelPtr& InParentItem
	, UMovieSceneSequence* const InSequence
	, UMovieSceneSubSection* const InSubSection
	, const int32 InSubSectionIndex)
	: FNavigationToolItem(InTool, InParentItem)
	, IRevisionControlExtension(InSequence)
	, WeakSubSection(InSubSection)
	, SubSectionIndex(InSubSectionIndex)
	, WeakSequence(InSequence)
{
}

bool FNavigationToolSequence::IsItemValid() const
{
	return WeakSequence.IsValid();
}

UObject* FNavigationToolSequence::GetItemObject() const
{
	return GetSequence();
}

bool FNavigationToolSequence::IsAllowedInTool() const
{
	return WeakSequence.IsValid();
}

void FNavigationToolSequence::FindChildren(TArray<FNavigationToolViewModelWeakPtr>& OutWeakChildren, const bool bInRecursive)
{
	FNavigationToolItem::FindChildren(OutWeakChildren, bInRecursive);

	UMovieSceneSequence* const Sequence = GetSequence();
	if (!Sequence)
	{
		return;
	}

	UMovieScene* const MovieScene = Sequence->GetMovieScene();
	if (!MovieScene)
	{
		return;
	}

	const TSharedPtr<FNavigationToolProvider> Provider = GetProvider();
	if (!Provider.IsValid())
	{
		return;
	}

	TArray<UMovieSceneTrack*> MovieSceneTracks = MovieScene->GetTracks();
	MovieSceneTracks.Sort([](const UMovieSceneTrack& InA, const UMovieSceneTrack& InB)
		{
			return InA.GetSortingOrder() < InB.GetSortingOrder();
		});

	const TSharedRef<FNavigationToolProvider> ProviderRef = Provider.ToSharedRef();
	const FNavigationToolViewModelPtr ParentItem = AsItemViewModel();

	for (UMovieSceneTrack* const Track : MovieSceneTracks)
	{
		FNavigationToolViewModelPtr NewItem;

		if (UMovieSceneSubTrack* const SubTrack = Cast<UMovieSceneSubTrack>(Track))
		{
			NewItem = Tool.FindOrAdd<FNavigationToolSubTrack>(ProviderRef
				, ParentItem, SubTrack, Sequence, WeakSubSection, SubSectionIndex);
		}
		else if (Track)
		{
			NewItem = Tool.FindOrAdd<FNavigationToolTrack>(ProviderRef
				, ParentItem, Track, Sequence, WeakSubSection, SubSectionIndex);
		}

		if (NewItem.IsValid())
		{
			OutWeakChildren.Add(NewItem);
			if (bInRecursive)
			{
				NewItem->FindChildren(OutWeakChildren, bInRecursive);
			}
		}
	}

	const TSharedRef<FNavigationToolSequence> ThisRef = SharedThis(this);

	// Only show actor bindings for a Sequence
	const TArray<FMovieSceneBinding> Bindings = GetSortedBindings();
	for (const FMovieSceneBinding& Binding : Bindings)
	{
		const FGuid& BindingObjectGuid = Binding.GetObjectGuid();
		const UClass* const BoundObjectClass = MovieSceneHelpers::GetBoundObjectClass(Sequence, BindingObjectGuid);
		if (BoundObjectClass && BoundObjectClass->IsChildOf<AActor>())
		{
			const FNavigationToolViewModelPtr NewItem = Tool.FindOrAdd<FNavigationToolActor>(ProviderRef
				, ThisRef, ThisRef, Binding);
			OutWeakChildren.Add(NewItem);
			if (bInRecursive)
			{
				NewItem->FindChildren(OutWeakChildren, bInRecursive);
			}
		}
	}

	// Add sequence marker items
	const TArray<FMovieSceneMarkedFrame>& MarkedFrames = MovieScene->GetMarkedFrames();
	for (int32 Index = 0; Index < MarkedFrames.Num(); ++Index)
	{
		const FNavigationToolViewModelPtr NewItem = Tool.FindOrAdd<FNavigationToolMarker>(ProviderRef
			, ThisRef, ThisRef, Index);
		OutWeakChildren.Add(NewItem);
		if (bInRecursive)
		{
			NewItem->FindChildren(OutWeakChildren, bInRecursive);
		}
	}
}

void FNavigationToolSequence::GetItemProxies(TArray<TSharedPtr<FNavigationToolItemProxy>>& OutItemProxies)
{
	FNavigationToolItem::GetItemProxies(OutItemProxies);
}

bool FNavigationToolSequence::AddChild(const FNavigationToolAddItemParams& InAddItemParams)
{
	// @TODO: handle sequence being moved to this item
	return FNavigationToolItem::AddChild(InAddItemParams);
}

bool FNavigationToolSequence::RemoveChild(const FNavigationToolRemoveItemParams& InRemoveItemParams)
{
	/// @TODO: handle sequence being moved to this item
	return FNavigationToolItem::RemoveChild(InRemoveItemParams);
}

ENavigationToolItemViewMode FNavigationToolSequence::GetSupportedViewModes(const INavigationToolView& InToolView) const
{
	// Sequences should only be visualized in Navigation Tool View and not appear in the Item Column List
	// Support any other type of View Mode
	return ENavigationToolItemViewMode::ItemTree | ~ENavigationToolItemViewMode::HorizontalItemList;
}

FText FNavigationToolSequence::GetDisplayName() const
{
	using namespace ItemUtils;

	FText NewDisplayName = FText::GetEmpty();

	if (const UMovieSceneCinematicShotSection* const ShotSection = Cast<UMovieSceneCinematicShotSection>(WeakSubSection.Get()))
	{
		NewDisplayName = FText::FromString(ShotSection->GetShotDisplayName());
	}
	else
	{
		if (const UMovieSceneSequence* const Sequence = GetSequence())
		{
			NewDisplayName = Sequence->GetDisplayName();
		}
	}

	if (NewDisplayName.IsEmpty())
	{
		return NewDisplayName;
	}

	// Apply additional options to the display name
	const UNavigationToolSettings* const ToolSettings = GetDefault<UNavigationToolSettings>();

	if (ToolSettings->ShouldUseShortNames())
	{
		RemoveSequenceDisplayNameParentPrefix(NewDisplayName, AsItemViewModelConst().ImplicitCast());
	}

	if (UMovieSceneSequence* const Sequence = GetSequence())
	{
		AppendSequenceDisplayNameDirtyStatus(NewDisplayName, *Sequence);
	}

	return NewDisplayName;
}

FText FNavigationToolSequence::GetClassName() const
{
	if (const UMovieSceneSequence* const Sequence = GetSequence())
	{
		return FText::FromString(Sequence->GetClass()->GetName());
	}
	return FText::FromString(UMovieSceneSequence::StaticClass()->GetName());
}

FSlateIcon FNavigationToolSequence::GetIcon() const
{
	return FSlateIconFinder::FindIconForClass(ALevelSequenceActor::StaticClass());
}

FText FNavigationToolSequence::GetIconTooltipText() const
{
	return ALevelSequenceActor::StaticClass()->GetDisplayNameText();
}

bool FNavigationToolSequence::IsSelected(const FNavigationToolScopedSelection& InSelection) const
{
	return WeakSubSection.IsValid() && InSelection.IsSelected(WeakSubSection.Get());
}

void FNavigationToolSequence::Select(FNavigationToolScopedSelection& InSelection) const
{
	if (WeakSubSection.IsValid())
	{
		InSelection.Select(WeakSubSection.Get());
	}
}

void FNavigationToolSequence::OnSelect()
{
	FNavigationToolItem::OnSelect();
}

void FNavigationToolSequence::OnDoubleClick()
{
	UMovieSceneSequence* const Sequence = GetSequence();
	if (!Sequence)
	{
		return;
	}

	const FModifierKeysState ModifierKeys = FSlateApplication::Get().GetModifierKeys();

	if (ModifierKeys.IsAltDown())
	{
		// Instead of focusing the sequence in the sequencer, focus the parent sequence and select the sequence
		if (const TViewModelPtr<FNavigationToolSequence> ParentSequenceItem = FindAncestorOfType<FNavigationToolSequence>())
		{
			if (UMovieSceneSequence* const ParentSequence = ParentSequenceItem->GetSequence())
			{
				FocusSequence(Tool, *ParentSequence, *this);
			}
		}
	}

	FocusSequence(Tool, *Sequence);
}

void FNavigationToolSequence::OnObjectsReplaced(const TMap<UObject*, UObject*>& InReplacementMap, bool bInRecursive)
{
	// Get the Object even if it's Pending Kill (most likely it is)
	const UMovieSceneSequence* const ObjectPendingKill = WeakSequence.Get(true);
	if (ObjectPendingKill && InReplacementMap.Contains(ObjectPendingKill))
	{
		WeakSequence = Cast<UMovieSceneSequence>(InReplacementMap[ObjectPendingKill]);
	}

	// This handles calling OnObjectsReplaced for every child item
	FNavigationToolItem::OnObjectsReplaced(InReplacementMap, bInRecursive);
}

FNavigationToolItemId FNavigationToolSequence::CalculateItemId() const
{
	return FNavigationToolItemId(GetParent(), GetSequence(), WeakSubSection.Get(), SubSectionIndex);
}

bool FNavigationToolSequence::CanRename() const
{
	// Disable for now
	return false;
	/*const UMovieSceneSequence* const Sequence = GetSequence();
	if (!Sequence)
	{
		return false;
	}

	const UMovieScene* const MovieScene = Sequence->GetMovieScene();
	if (!MovieScene)
	{
		return false;
	}

	return !MovieScene->IsReadOnly();*/
}

void FNavigationToolSequence::Rename(const FText& InNewName)
{
	UMovieSceneSequence* const Sequence = GetSequence();
	if (!Sequence)
	{
		return;
	}

	const FString NewNameString = InNewName.ToString();

	if (NewNameString.Equals(Sequence->GetName(), ESearchCase::CaseSensitive))
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("SequenceRename", "Rename Sequence"));

	if (UMovieSceneCinematicShotSection* const ShotSection = Cast<UMovieSceneCinematicShotSection>(WeakSubSection.Get()))
	{
		ShotSection->Modify();
		ShotSection->SetShotDisplayName(NewNameString);

		Tool.NotifyToolItemRenamed(SharedThis(this));
	}
	else
	{
		Sequence->Modify();

		if (Sequence->Rename(*NewNameString))
		{
			Tool.NotifyToolItemRenamed(SharedThis(this));
		}
	}
}

bool FNavigationToolSequence::IsDeactivated() const
{
	using namespace ItemUtils;

	if (WeakSubSection.IsValid())
	{
		return !WeakSubSection->IsActive();
	}

	const ENavigationToolCompareState State = CompareChildrenItemState<IDeactivatableExtension>(AsItemViewModelConst(),
		[](const TViewModelPtr<IDeactivatableExtension>& InItem)
			{
				return InItem->IsDeactivated();
			},
		[](const TViewModelPtr<IDeactivatableExtension>& InItem)
			{
				return !InItem->IsDeactivated();
			});

	return State == ENavigationToolCompareState::AllTrue;
}

void FNavigationToolSequence::SetIsDeactivated(const bool bInIsDeactivated)
{
	if (WeakSubSection.IsValid())
	{
		if (!WeakSubSection->IsReadOnly() && WeakSubSection->IsActive() == bInIsDeactivated)
		{
			WeakSubSection->Modify();
			WeakSubSection->SetIsActive(!bInIsDeactivated);
		}
	}

	for (const TViewModelPtr<IDeactivatableExtension>& InactivableItem : GetChildrenOfType<IDeactivatableExtension>())
	{
		InactivableItem->SetIsDeactivated(bInIsDeactivated);
	}
}

ELockableLockState FNavigationToolSequence::GetLockState() const
{
	using namespace ItemUtils;

	if (WeakSubSection.IsValid())
	{
		return WeakSubSection->IsLocked() ? ELockableLockState::Locked : ELockableLockState::None;
	}

	const ENavigationToolCompareState State = CompareChildrenItemState<ILockableExtension>(AsItemViewModelConst(),
		[](const TViewModelPtr<ILockableExtension>& InItem)
			{
				return InItem->GetLockState() == ELockableLockState::Locked;
			},
		[](const TViewModelPtr<ILockableExtension>& InItem)
			{
				return InItem->GetLockState() != ELockableLockState::Locked;
			});

	return static_cast<ELockableLockState>(State);
}

void FNavigationToolSequence::SetIsLocked(const bool bInIsLocked)
{
	if (WeakSubSection.IsValid())
	{
		if (WeakSubSection->IsLocked() != bInIsLocked)
		{
			WeakSubSection->Modify();
			WeakSubSection->SetIsLocked(bInIsLocked);
		}

		return;
	}

	for (const TViewModelPtr<ILockableExtension>& LockableItem : GetChildrenOfType<ILockableExtension>())
	{
		LockableItem->SetIsLocked(bInIsLocked);
	}
}

EItemMarkerVisibility FNavigationToolSequence::GetMarkerVisibility() const
{
	return IsGloballyMarkedFramesForSequence(GetSequence())
		? EItemMarkerVisibility::Visible : EItemMarkerVisibility::None;
}

void FNavigationToolSequence::SetMarkerVisibility(const bool bInVisible)
{
	const TSharedPtr<ISequencer> Sequencer = Tool.GetSequencer();
	if (!Sequencer.IsValid())
	{
		return;
	}

	const bool bVisible = (GetMarkerVisibility() == EItemMarkerVisibility::Visible);
	if (bVisible == bInVisible)
	{
		return;
	}

	UMovieSceneSequence* const Sequence = GetSequence();
	if (Sequence == Sequencer->GetRootMovieSceneSequence())
	{
		Sequencer->GetSequencerSettings()->SetShowMarkedFrames(bInVisible);
	}

	ShowGloballyMarkedFramesForSequence(*Sequencer, Sequence, bInVisible);

	for (const TViewModelPtr<IMarkerVisibilityExtension>& MarkerVisibilityItem : GetChildrenOfType<IMarkerVisibilityExtension>())
	{
		MarkerVisibilityItem->SetMarkerVisibility(bInVisible);
	}
}

TOptional<FColor> FNavigationToolSequence::GetColor() const
{
	if (!WeakSubSection.IsValid())
	{
		return FColor::Transparent;
	}

	const TSharedPtr<ISequencer> Sequencer = Tool.GetSequencer();
	if (!Sequencer.IsValid())
	{
		return FColor::Transparent;
	}

	const FColor TrackColor = WeakSubSection->GetColorTint();
	return (TrackColor == FColor()) ? TOptional<FColor>() : WeakSubSection->GetColorTint();
}

void FNavigationToolSequence::SetColor(const TOptional<FColor>& InColor)
{
	if (!WeakSubSection.IsValid())
	{
		return;
	}

	const TSharedPtr<ISequencer> Sequencer = Tool.GetSequencer();
	if (!Sequencer.IsValid())
	{
		return;
	}

	WeakSubSection->Modify();
	WeakSubSection->SetColorTint(InColor.Get(FColor()));
}

FText FNavigationToolSequence::GetId() const
{
	UMovieSceneSequence* const Sequence = GetSequence();
	if (!Sequence)
	{
		return FText();
	}

	const TSharedPtr<ISequencer> Sequencer = Tool.GetSequencer();
	if (!Sequencer.IsValid())
	{
		return FText();
	}

	FMovieSceneRootEvaluationTemplateInstance& RootInstance = Sequencer->GetEvaluationTemplate();
	const FMovieSceneSequenceHierarchy* Hierarchy = RootInstance.GetCompiledDataManager()->FindHierarchy(RootInstance.GetCompiledDataID());
	if (Hierarchy)
	{
		for (const TTuple<FMovieSceneSequenceID, FMovieSceneSubSequenceData>& Pair : Hierarchy->AllSubSequenceData())
		{
			if (Sequence == Pair.Value.GetSequence())
			{
				return FText::FromString(FString::Printf(TEXT("%d"), Pair.Key.GetInternalValue()));
			}
		}
	}

	return FText();
}

EItemRevisionControlState FNavigationToolSequence::GetRevisionControlState() const
{
	UMovieSceneSequence* const Sequence = GetSequence();
	if (!Sequence)
	{
		return EItemRevisionControlState::None;
	}

	const FSourceControlStatePtr RevisionControlState = FindSourceControlState(Sequence->GetPackage());
	if (!RevisionControlState.IsValid())
	{
		return EItemRevisionControlState::None;
	}

	return EItemRevisionControlState::SourceControlled;
}

const FSlateBrush* FNavigationToolSequence::GetRevisionControlStatusIcon() const
{
	if (const UMovieSceneSequence* const Sequence = GetSequence())
	{
		return FindSourceControlStatusBrush(Sequence->GetPackage());
	}
	return nullptr;
}

FText FNavigationToolSequence::GetRevisionControlStatusText() const
{
	if (const UMovieSceneSequence* const Sequence = GetSequence())
	{
		return FindSourceControlStatusText(Sequence->GetPackage());
	}
	return FText::GetEmpty();
}

EItemContainsPlayhead FNavigationToolSequence::ContainsPlayhead() const
{
	const TSharedPtr<ISequencer> Sequencer = Tool.GetSequencer();
	if (!Sequencer.IsValid())
	{
		return EItemContainsPlayhead::None;
	}

	const UMovieSceneSequence* const Sequence = WeakSequence.Get();
	if (!Sequence)
	{
		return EItemContainsPlayhead::None;
	}

	UMovieSceneSequence* const FocusedSequence = Sequencer->GetFocusedMovieSceneSequence();
	if (!FocusedSequence)
	{
		return EItemContainsPlayhead::None;
	}

	const FQualifiedFrameTime PlayheadTime = Sequencer->GetLocalTime();
	const FFrameNumber PlayheadFrame = PlayheadTime.ConvertTo(Sequencer->GetFocusedTickResolution()).FloorToFrame();

	if (Sequence == FocusedSequence)
	{
		if (UMovieScene* const SequenceMovieScene = Sequence->GetMovieScene())
		{
			return SequenceMovieScene->GetPlaybackRange().Contains(PlayheadFrame)
				? EItemContainsPlayhead::ContainsPlayhead : EItemContainsPlayhead::None;
		}
	}
	else
	{
		if (UMovieSceneSubSection* const SubSection = GetSubSection())
		{
			return SubSection->GetRange().Contains(PlayheadTime.Time.FrameNumber)
				? EItemContainsPlayhead::ContainsPlayhead : EItemContainsPlayhead::None;
		}
	}

	return EItemContainsPlayhead::None;
}

FFrameNumber FNavigationToolSequence::GetInTime() const
{
	const UMovieSceneSequence* const Sequence = WeakSequence.Get();
	if (!Sequence)
	{
		return 0;
	}

	const UMovieSceneSubSection* const SubSection = WeakSubSection.Get();
	if (!SubSection)
	{
		return 0;
	}

	const FFrameTime LowerBoundValue = SubSection->SectionRange.Value.GetLowerBoundValue();
	return LowerBoundValue.FrameNumber;
}

void FNavigationToolSequence::SetInTime(const FFrameNumber& InTime)
{
	const UMovieSceneSequence* const Sequence = WeakSequence.Get();
	if (!Sequence)
	{
		return;
	}

	UMovieSceneSubSection* const SubSection = WeakSubSection.Get();
	if (!SubSection)
	{
		return;
	}

	if (SubSection->SectionRange.Value.GetLowerBoundValue() == InTime)
	{
		return;
	}

	SubSection->Modify();
	SubSection->SectionRange.Value.SetLowerBoundValue(InTime);
}

FFrameNumber FNavigationToolSequence::GetOutTime() const
{
	const UMovieSceneSequence* const Sequence = WeakSequence.Get();
	if (!Sequence)
	{
		return 0;
	}

	const UMovieSceneSubSection* const SubSection = WeakSubSection.Get();
	if (!SubSection)
	{
		return 0;
	}

	const FFrameTime UpperBoundValue = SubSection->SectionRange.Value.GetUpperBoundValue();
	return UpperBoundValue.FrameNumber;
}

void FNavigationToolSequence::SetOutTime(const FFrameNumber& InTime)
{
	const UMovieSceneSequence* const Sequence = WeakSequence.Get();
	if (!Sequence)
	{
		return;
	}

	UMovieSceneSubSection* const SubSection = WeakSubSection.Get();
	if (!SubSection)
	{
		return;
	}

	if (SubSection->SectionRange.Value.GetUpperBoundValue() == InTime)
	{
		return;
	}

	SubSection->Modify();
	SubSection->SectionRange.Value.SetUpperBoundValue(InTime);
}

UMovieSceneSequence* FNavigationToolSequence::GetSequence() const
{
	return WeakSequence.Get();
}

UMovieSceneSubSection* FNavigationToolSequence::GetSubSection() const
{
	return WeakSubSection.IsValid() ? WeakSubSection.Get() : nullptr;
}

int32 FNavigationToolSequence::GetSubSectionIndex() const
{
	return SubSectionIndex;
}

UMovieScene* FNavigationToolSequence::GetSequenceMovieScene() const
{
	if (const UMovieSceneSequence* const Sequence = GetSequence())
	{
		return Sequence->GetMovieScene();
	}
	return nullptr;
}

TArray<FMovieSceneBinding> FNavigationToolSequence::GetSortedBindings() const
{
	const UMovieScene* MovieScene = GetSequenceMovieScene();
	if (!MovieScene)
	{
		return {};
	}

	TArray<FMovieSceneBinding> Bindings = MovieScene->GetBindings();

	Bindings.Sort([MovieScene](const FMovieSceneBinding& InA, const FMovieSceneBinding& InB)
		{
			const int32 SortingOrderA = InA.GetSortingOrder();
			const int32 SortingOrderB = InB.GetSortingOrder();
			if (SortingOrderA == SortingOrderB)
			{
				return MovieScene->GetObjectDisplayName(InA.GetObjectGuid()).ToString() < MovieScene->GetObjectDisplayName(InB.GetObjectGuid()).ToString();
			}
			return SortingOrderA < SortingOrderB;
		});

	return MoveTemp(Bindings);
}

TViewModelPtr<FSectionModel> FNavigationToolSequence::GetViewModel() const
{
	if (!WeakSubSection.IsValid())
	{
		return nullptr;
	}

	const TSharedPtr<ISequencer> Sequencer = Tool.GetSequencer();
	if (!Sequencer.IsValid())
	{
		return nullptr;
	}

	const TSharedPtr<FSequencerEditorViewModel> ViewModel = Sequencer->GetViewModel();
	if (!ViewModel.IsValid())
	{
		return nullptr;
	}

	const FViewModelPtr RootViewModel = ViewModel->GetRootModel();
	if (!RootViewModel.IsValid())
	{
		return nullptr;
	}

	const FSectionModelStorageExtension* const StorageExtension = RootViewModel->CastDynamic<FSectionModelStorageExtension>();
	if (!StorageExtension)
	{
		return nullptr;
	}

	return StorageExtension->FindModelForSection(WeakSubSection.Get());
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
