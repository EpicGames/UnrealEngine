// Copyright Epic Games, Inc. All Rights Reserved.

#include "Items/NavigationToolSubTrack.h"
#include "INavigationTool.h"
#include "Items/NavigationToolItemUtils.h"
#include "Items/NavigationToolSequence.h"
#include "MovieSceneSequence.h"
#include "MovieSceneTrack.h"
#include "Sections/MovieSceneSubSection.h"
#include "Tracks/MovieSceneSubTrack.h"
#include "Utils/NavigationToolMovieSceneUtils.h"
#include "Widgets/Views/STableRow.h"

#define LOCTEXT_NAMESPACE "NavigationToolSubTrack"

namespace UE::SequenceNavigator
{

using namespace Sequencer;

UE_SEQUENCER_DEFINE_CASTABLE(FNavigationToolSubTrack)

FNavigationToolSubTrack::FNavigationToolSubTrack(INavigationTool& InTool
	, const FNavigationToolViewModelPtr& InParentItem
	, UMovieSceneSubTrack* const InSubTrack
	, const TWeakObjectPtr<UMovieSceneSequence>& InSequence
	, const TWeakObjectPtr<UMovieSceneSection>& InSection
	, const int32 InSubSectionIndex)
	: FNavigationToolTrack(InTool
	, InParentItem
	, InSubTrack
	, InSequence
	, InSection
	, InSubSectionIndex)
{
	OnTrackObjectChanged();
}

void FNavigationToolSubTrack::FindChildren(TArray<FNavigationToolViewModelWeakPtr>& OutWeakChildren, const bool bInRecursive)
{
	FNavigationToolTrack::FindChildren(OutWeakChildren, bInRecursive);

	UMovieSceneSubTrack* const SubTrack = GetSubTrack();
	if (!SubTrack)
	{
		return;
	}

	const TSharedPtr<FNavigationToolProvider> Provider = GetProvider();
	if (!Provider.IsValid())
	{
		return;
	}

	const TSharedRef<FNavigationToolProvider> ProviderRef = Provider.ToSharedRef();
	const FNavigationToolViewModelPtr ParentItem = AsItemViewModel();
	const TArray<UMovieSceneSection*>& AllSections = SubTrack->GetAllSections();

	for (int32 Index = 0; Index < AllSections.Num(); ++Index)
	{
		if (UMovieSceneSubSection* const SubSection = Cast<UMovieSceneSubSection>(AllSections[Index]))
		{
			if (UMovieSceneSequence* const Sequence = SubSection->GetSequence())
			{
				const FNavigationToolViewModelPtr NewItem = Tool.FindOrAdd<FNavigationToolSequence>(ProviderRef
					, ParentItem, Sequence, SubSection, Index);
				OutWeakChildren.Add(NewItem);
				if (bInRecursive)
				{
					NewItem->FindChildren(OutWeakChildren, bInRecursive);
				}
			}
		}
	}
}

TOptional<EItemDropZone> FNavigationToolSubTrack::CanAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone)
{
	return FNavigationToolTrack::CanAcceptDrop(InDragDropEvent, InDropZone);
}

FReply FNavigationToolSubTrack::AcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone)
{
	return FNavigationToolTrack::AcceptDrop(InDragDropEvent, InDropZone);
}

UMovieSceneSubTrack* FNavigationToolSubTrack::GetSubTrack() const
{
	return Cast<UMovieSceneSubTrack>(GetTrack());
}

bool FNavigationToolSubTrack::IsDeactivated() const
{
	using namespace ItemUtils;

	if (UMovieSceneSubSection* const ThisSubSection = Cast<UMovieSceneSubSection>(WeakSection.Get()))
	{
		return !ThisSubSection->IsActive();
	}

	const ENavigationToolCompareState State = CompareChildrenItemState<IDeactivatableExtension>(AsItemViewModelConst(),
		[](const TViewModelPtr<IDeactivatableExtension>& InItem)
			{
				return InItem->IsDeactivated();
			},
		[](const TViewModelPtr<IDeactivatableExtension>& InItem)
			{
				return !InItem->IsDeactivated();
			});

	return State == ENavigationToolCompareState::AllTrue;
}

void FNavigationToolSubTrack::SetIsDeactivated(const bool bInIsDeactivated)
{
	const bool bNewActiveState = !bInIsDeactivated;

	if (UMovieSceneSubSection* const ThisSubSection = Cast<UMovieSceneSubSection>(WeakSection.Get()))
	{
		if (ThisSubSection->IsActive() != bNewActiveState)
		{
			ThisSubSection->Modify();
			ThisSubSection->SetIsActive(bNewActiveState);
		}
	}

	for (const TViewModelPtr<IDeactivatableExtension>& InactivableItem : GetChildrenOfType<IDeactivatableExtension>())
	{
		InactivableItem->SetIsDeactivated(bInIsDeactivated);
	}
}

EItemMarkerVisibility FNavigationToolSubTrack::GetMarkerVisibility() const
{
	using namespace ItemUtils;

	const ENavigationToolCompareState State = CompareChildrenItemState<IMarkerVisibilityExtension>(AsItemViewModelConst(),
		[](const TViewModelPtr<IMarkerVisibilityExtension>& InItem)
			{
				return InItem->GetMarkerVisibility() == EItemMarkerVisibility::Visible;
			},
		[](const TViewModelPtr<IMarkerVisibilityExtension>& InItem)
			{
				return InItem->GetMarkerVisibility() == EItemMarkerVisibility::None;
			});

	return static_cast<EItemMarkerVisibility>(State);
}

void FNavigationToolSubTrack::SetMarkerVisibility(const bool bInVisible)
{
	const TSharedPtr<ISequencer> Sequencer = Tool.GetSequencer();
	if (!Sequencer.IsValid())
	{
		return;
	}

	if (UMovieSceneSubSection* const ThisSubSection = Cast<UMovieSceneSubSection>(WeakSection.Get()))
	{
		UMovieSceneSequence* const Sequence = ThisSubSection->GetSequence();
		const bool bIsVisible = IsGloballyMarkedFramesForSequence(Sequence);
		if (bIsVisible != bInVisible)
		{
			ModifySequenceAndMovieScene(Sequence);
			ShowGloballyMarkedFramesForSequence(*Sequencer, Sequence, bInVisible);
		}
	}

	for (const TViewModelPtr<IMarkerVisibilityExtension>& MarkerVisibilityItem : GetChildrenOfType<IMarkerVisibilityExtension>())
	{
		MarkerVisibilityItem->SetMarkerVisibility(bInVisible);
	}
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
