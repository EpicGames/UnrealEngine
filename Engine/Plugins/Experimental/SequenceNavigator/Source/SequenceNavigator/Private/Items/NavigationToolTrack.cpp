// Copyright Epic Games, Inc. All Rights Reserved.

#include "Items/NavigationToolTrack.h"
#include "INavigationTool.h"
#include "ISequencer.h"
#include "ISequencerTrackEditor.h"
#include "Items/NavigationToolItemUtils.h"
#include "MovieScene.h"
#include "MovieSceneSequence.h"
#include "MovieSceneTrack.h"
#include "MVVM/TrackModelStorageExtension.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/TrackModel.h"
#include "NavigationToolScopedSelection.h"
#include "ScopedTransaction.h"
#include "Sections/MovieSceneSubSection.h"
#include "Styling/SlateIconFinder.h"
#include "Utils/NavigationToolMiscUtils.h"
#include "Widgets/Views/STableRow.h"

#define LOCTEXT_NAMESPACE "NavigationToolTrack"

namespace UE::SequenceNavigator
{

UE_SEQUENCER_DEFINE_CASTABLE(FNavigationToolTrack)

FNavigationToolTrack::FNavigationToolTrack(INavigationTool& InTool
	, const FNavigationToolViewModelPtr& InParentItem
	, UMovieSceneTrack* const InTrack
	, const TWeakObjectPtr<UMovieSceneSequence>& InSequence
	, const TWeakObjectPtr<UMovieSceneSection>& InSection
	, const int32 InSubSectionIndex)
	: FNavigationToolItem(InTool, InParentItem)
	, WeakSequence(InSequence)
	, WeakSection(InSection)
	, SectionIndex(InSubSectionIndex)
	, WeakTrack(InTrack)
{
	OnTrackObjectChanged();
}

bool FNavigationToolTrack::IsItemValid() const
{
	return WeakTrack.IsValid();
}

UObject* FNavigationToolTrack::GetItemObject() const
{
	return GetTrack();
}

bool FNavigationToolTrack::IsAllowedInTool() const
{
	return IsItemValid();
}

FNavigationToolItemId FNavigationToolTrack::CalculateItemId() const
{
	return FNavigationToolItemId(GetParent()
		, WeakSequence.Get()
		, WeakSection.Get()
		, SectionIndex
		, FNavigationToolItemId::GetObjectPath(WeakTrack.Get()));
}

void FNavigationToolTrack::FindChildren(TArray<FNavigationToolViewModelWeakPtr>& OutWeakChildren, const bool bInRecursive)
{
	FNavigationToolItem::FindChildren(OutWeakChildren, bInRecursive);
}

FText FNavigationToolTrack::GetDisplayName() const
{
	if (UMovieSceneTrack* const Track = GetTrack())
	{
		return Track->GetDisplayName();
	}
	return FNavigationToolItem::GetDisplayName();
}

FText FNavigationToolTrack::GetClassName() const
{
	if (UMovieSceneTrack* const Track = GetTrack())
	{
		return FText::FromString(Track->GetClass()->GetName());
	}
	return FText::FromString(UMovieSceneTrack::StaticClass()->GetName());
}

const FSlateBrush* FNavigationToolTrack::GetDefaultIconBrush() const
{
	UMovieSceneTrack* const Track = GetTrack();
	if (!Track)
	{
		return nullptr;
	}

	if (const TSharedPtr<ISequencer> Sequencer = Tool.GetSequencer())
	{
		if (const TSharedPtr<ISequencerTrackEditor> TrackEditor = Sequencer->GetTrackEditor(Track))
		{
			if (const FSlateBrush* const IconBrush = TrackEditor->GetIconBrush())
			{
				return IconBrush;
			}
		}
	}

	return FSlateIconFinder::FindIconForClass(UMovieSceneTrack::StaticClass()).GetIcon();
}

FSlateIcon FNavigationToolTrack::GetIcon() const
{
	return Icon;
}

FText FNavigationToolTrack::GetIconTooltipText() const
{
	if (const UMovieSceneTrack* const Track = GetTrack())
	{
		return Track->GetClass()->GetDisplayNameText();
	}
	return FText::GetEmpty();
}

bool FNavigationToolTrack::IsSelected(const FNavigationToolScopedSelection& InSelection) const
{
	UMovieSceneTrack* const Track = GetTrack();
	return Track && InSelection.IsSelected(Track);
}

void FNavigationToolTrack::Select(FNavigationToolScopedSelection& InSelection) const
{
	if (UMovieSceneTrack* const Track = GetTrack())
	{
		InSelection.Select(Track);
	}
}

void FNavigationToolTrack::OnSelect()
{
	FNavigationToolItem::OnSelect();
}

void FNavigationToolTrack::OnDoubleClick()
{
	if (!WeakSequence.IsValid())
	{
		return;
	}

	UMovieSceneTrack* const Track = GetTrack();
	if (!Track)
	{
		return;
	}

	FocusSequence(Tool, *WeakSequence.Get(), *this);
}

void FNavigationToolTrack::OnObjectsReplaced(const TMap<UObject*, UObject*>& InReplacementMap, bool bInRecursive)
{
	// Get the Object even if it's Pending Kill (most likely it is)
	const UMovieSceneTrack* const ObjectPendingKill = WeakTrack.Get(true);
	if (ObjectPendingKill && InReplacementMap.Contains(ObjectPendingKill))
	{
		WeakTrack = Cast<UMovieSceneTrack>(InReplacementMap[ObjectPendingKill]);
	}

	// This handles calling OnObjectsReplaced for every child item
	FNavigationToolItem::OnObjectsReplaced(InReplacementMap, bInRecursive);

	OnTrackObjectChanged();
}

Sequencer::ELockableLockState FNavigationToolTrack::GetLockState() const
{
	using namespace Sequencer;
	using namespace ItemUtils;

	if (WeakSection.IsValid())
	{
		return WeakSection->IsLocked() ? ELockableLockState::Locked : ELockableLockState::None;
	}

	const ENavigationToolCompareState State = CompareChildrenItemState<ILockableExtension>(AsItemViewModelConst(),
		[](const TViewModelPtr<ILockableExtension>& InItem)
			{
				return InItem->GetLockState() == ELockableLockState::Locked;
			},
		[](const TViewModelPtr<ILockableExtension>& InItem)
			{
				return InItem->GetLockState() != ELockableLockState::Locked;
			});

	return static_cast<ELockableLockState>(State);
}

void FNavigationToolTrack::SetIsLocked(const bool bInIsLocked)
{
	using namespace Sequencer;

	if (WeakSection.IsValid())
	{
		if (WeakSection->IsLocked() != bInIsLocked)
		{
			WeakSection->Modify();
			WeakSection->SetIsLocked(bInIsLocked);
		}
	}

	for (const TViewModelPtr<ILockableExtension>& LockableItem : GetChildrenOfType<ILockableExtension>())
	{
		LockableItem->SetIsLocked(bInIsLocked);
	}
}

EItemContainsPlayhead FNavigationToolTrack::ContainsPlayhead() const
{
	const TSharedPtr<ISequencer> Sequencer = Tool.GetSequencer();
	if (!Sequencer.IsValid())
	{
		return EItemContainsPlayhead::None;
	}

	const FQualifiedFrameTime PlayheadTime = Sequencer->GetLocalTime();
	const FFrameNumber PlayheadFrame = PlayheadTime.ConvertTo(Sequencer->GetFocusedTickResolution()).FloorToFrame();

	if (WeakSection.IsValid())
	{
		return WeakSection.Pin()->ComputeEffectiveRange().Contains(PlayheadFrame)
			? EItemContainsPlayhead::ContainsPlayhead : EItemContainsPlayhead::None;
	}

	UMovieSceneTrack* const Track = GetTrack();
	if (!Track)
	{
		return EItemContainsPlayhead::None;
	}

	const TArray<UMovieSceneSection*> Sections = Track->GetAllSections();
	for (const UMovieSceneSection* const Section : Sections)
	{
		if (Section->ComputeEffectiveRange().Contains(PlayheadFrame))
		{
			return EItemContainsPlayhead::ContainsPlayhead;
		}
	}

	return EItemContainsPlayhead::None;
}

void FNavigationToolTrack::OnTrackObjectChanged()
{
	UMovieSceneTrack* const Track = GetTrack();
	if (!Track)
	{
		return;
	}

	const TSharedPtr<ISequencer> Sequencer = Tool.GetSequencer();
	if (!Sequencer.IsValid())
	{
		return;
	}

	// Refresh icon from the track editor
	const TSharedPtr<ISequencerTrackEditor> TrackEditor = Sequencer->GetTrackEditor(Track);
	if (TrackEditor.IsValid())
	{
		if (const FSlateBrush* const IconBrush = TrackEditor->GetIconBrush())
		{
			Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), IconBrush->GetResourceName());
		}
	}
}

UMovieSceneTrack* FNavigationToolTrack::GetTrack() const
{
	return WeakTrack.Get();
}

UMovieSceneSequence* FNavigationToolTrack::GetSequence() const
{
	return WeakSequence.Get();
}

UMovieSceneSection* FNavigationToolTrack::GetSection() const
{
	return WeakSection.Get();
}

int32 FNavigationToolTrack::GetSectionIndex() const
{
	return SectionIndex;
}

UE::Sequencer::TViewModelPtr<UE::Sequencer::FTrackModel> FNavigationToolTrack::GetViewModel() const
{
	using namespace Sequencer;

	UMovieSceneTrack* const Track = GetTrack();
	if (!Track)
	{
		return nullptr;
	}

	const TSharedPtr<ISequencer> Sequencer = Tool.GetSequencer();
	if (!Sequencer.IsValid())
	{
		return nullptr;
	}

	const TSharedPtr<FSequencerEditorViewModel> ViewModel = Sequencer->GetViewModel();
	if (!ViewModel.IsValid())
	{
		return nullptr;
	}

	const FViewModelPtr RootViewModel = ViewModel->GetRootModel();
	if (!RootViewModel.IsValid())
	{
		return nullptr;
	}

	const FTrackModelStorageExtension* const StorageExtension = RootViewModel->CastDynamic<FTrackModelStorageExtension>();
	if (!StorageExtension)
	{
		return nullptr;
	}

	return StorageExtension->FindModelForTrack(Track);
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
