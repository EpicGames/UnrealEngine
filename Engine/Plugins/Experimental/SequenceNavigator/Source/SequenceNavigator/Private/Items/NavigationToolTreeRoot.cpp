// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavigationToolTreeRoot.h"
#include "NavigationTool.h"
#include "Textures/SlateIcon.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Views/STableRow.h"

namespace UE::SequenceNavigator
{

UE_SEQUENCER_DEFINE_CASTABLE(FNavigationToolTreeRoot)
	
FNavigationToolTreeRoot::FNavigationToolTreeRoot(INavigationTool& InTool)
	: FNavigationToolItem(InTool, nullptr)
{
}

void FNavigationToolTreeRoot::FindChildren(TArray<FNavigationToolViewModelWeakPtr>& OutWeakChildren, const bool bInRecursive)
{
	FNavigationToolItem::FindChildren(OutWeakChildren, bInRecursive);

	// Get items to add from providers
	GetOwnerTool().ForEachProvider([this, &OutWeakChildren, bInRecursive]
		(const TSharedRef<FNavigationToolProvider>& InToolProvider)
		{
			TArray<FNavigationToolViewModelWeakPtr> WeakProviderItems;
			InToolProvider->OnExtendItemChildren(Tool, SharedThis(this), WeakProviderItems, bInRecursive);

			OutWeakChildren.Append(WeakProviderItems);

			return true;
		});
}

bool FNavigationToolTreeRoot::CanAddChild(const FNavigationToolViewModelPtr& InChild) const
{
	return FNavigationToolItem::CanAddChild(InChild) && InChild->CanBeTopLevel();
}

bool FNavigationToolTreeRoot::IsAllowedInTool() const
{
	checkNoEntry();
	return false;
}

FText FNavigationToolTreeRoot::GetDisplayName() const
{
	checkNoEntry();
	return FText::GetEmpty();
}

FText FNavigationToolTreeRoot::GetClassName() const
{
	checkNoEntry();
	return FText::GetEmpty();
}

FText FNavigationToolTreeRoot::GetIconTooltipText() const
{
	checkNoEntry();
	return FText::GetEmpty();
}

FSlateIcon FNavigationToolTreeRoot::GetIcon() const
{
	checkNoEntry();
	return FSlateIcon();
}

TSharedRef<SWidget> FNavigationToolTreeRoot::GenerateLabelWidget(const TSharedRef<SNavigationToolTreeRow>& InRow)
{
	checkNoEntry();
	return SNullWidget::NullWidget;
}

TOptional<EItemDropZone> FNavigationToolTreeRoot::CanAcceptDrop(const FDragDropEvent& InDragDropEvent, const EItemDropZone InDropZone)
{
	return FNavigationToolItem::CanAcceptDrop(InDragDropEvent, InDropZone);
}

FReply FNavigationToolTreeRoot::AcceptDrop(const FDragDropEvent& InDragDropEvent, const EItemDropZone InDropZone)
{
	// Force tree root to always be OntoItem so we always create new items as children of root
	constexpr EItemDropZone ForcedDropZone = EItemDropZone::OntoItem;
	return FNavigationToolItem::AcceptDrop(InDragDropEvent, ForcedDropZone);
}

FNavigationToolItemId FNavigationToolTreeRoot::CalculateItemId() const
{
	return FNavigationToolItemId::RootId;
}

} // namespace UE::SequenceNavigator
