// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Items/NavigationToolItem.h"
#include "MVVM/ICastable.h"

#define UE_API SEQUENCENAVIGATOR_API

namespace UE::SequenceNavigator
{

class INavigationTool;

/**
 * The Item that ensures that every item (except self) has a parent in the hierarchy to make it easier
 * to handle. This is not really a visual item, so it can't appear in the Navigation Tool view.
 */
class FNavigationToolTreeRoot final
	: public FNavigationToolItem
{
public:
	UE_SEQUENCER_DECLARE_CASTABLE_API(UE_API, FNavigationToolTreeRoot
		, FNavigationToolItem)

	FNavigationToolTreeRoot(INavigationTool& InTool);

	//~ Begin INavigationToolItem
	virtual void FindChildren(TArray<FNavigationToolViewModelWeakPtr>& OutWeakChildren, const bool bInRecursive) override;
	virtual bool CanAddChild(const FNavigationToolViewModelPtr& InChild) const override;
	virtual bool IsAllowedInTool() const override;
	virtual FText GetDisplayName() const override;
	virtual FText GetClassName() const override;
	virtual FText GetIconTooltipText() const override;
	virtual FSlateIcon GetIcon() const override;
	virtual TSharedRef<SWidget> GenerateLabelWidget(const TSharedRef<SNavigationToolTreeRow>& InRow) override;
	virtual TOptional<EItemDropZone> CanAcceptDrop(const FDragDropEvent& InDragDropEvent, const EItemDropZone InDropZone) override;	
	virtual FReply AcceptDrop(const FDragDropEvent& InDragDropEvent, const EItemDropZone InDropZone) override;
	//~ End INavigationToolItem

protected:
	//~ Begin FNavigationToolItem
	virtual FNavigationToolItemId CalculateItemId() const override;
	//~ End FNavigationToolItem
};

} // namespace UE::SequenceNavigator

#undef UE_API
