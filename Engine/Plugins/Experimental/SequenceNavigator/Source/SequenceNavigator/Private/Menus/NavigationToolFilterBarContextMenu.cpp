// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavigationToolFilterBarContextMenu.h"
#include "Filters/INavigationToolFilterBar.h"
#include "Filters/NavigationToolFilterCommands.h"
#include "Menus/NavigationToolFilterBarContext.h"
#include "ToolMenu.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "NavigationToolFilterBarContextMenu"

namespace UE::SequenceNavigator
{

TSharedRef<SWidget> FNavigationToolFilterBarContextMenu::CreateMenu(const TSharedRef<INavigationToolFilterBar>& InFilterBar)
{
	const FName FilterMenuName = TEXT("SequenceNavigator.FilterBarContextMenu");
	if (!UToolMenus::Get()->IsMenuRegistered(FilterMenuName))
	{
		UToolMenu* const Menu = UToolMenus::Get()->RegisterMenu(FilterMenuName);
		Menu->AddDynamicSection(NAME_None, FNewToolMenuDelegate::CreateLambda([this](UToolMenu* const InMenu)
			{
				if (UNavigationToolFilterBarContext* const Context = InMenu->FindContext<UNavigationToolFilterBarContext>())
				{
					Context->OnPopulateMenu.ExecuteIfBound(InMenu);
				}
			}));
	}

	UNavigationToolFilterBarContext* const ContextObject = NewObject<UNavigationToolFilterBarContext>();
	ContextObject->Init(InFilterBar);
	ContextObject->OnPopulateMenu = FOnPopulateFilterBarMenu::CreateRaw(this, &FNavigationToolFilterBarContextMenu::PopulateMenu);

	const FToolMenuContext MenuContext(InFilterBar->GetCommandList(), nullptr, ContextObject);
	return UToolMenus::Get()->GenerateWidget(FilterMenuName, MenuContext);
}

void FNavigationToolFilterBarContextMenu::PopulateMenu(UToolMenu* const InMenu)
{
	if (!InMenu)
	{
		return;
	}

	UNavigationToolFilterBarContext* const Context = InMenu->FindContext<UNavigationToolFilterBarContext>();
	if (!Context)
	{
		return;
	}

	WeakFilterBar = Context->GetFilterBar();

	UToolMenu& MenuRef = *InMenu;

	PopulateOptionsSection(MenuRef);
}

void FNavigationToolFilterBarContextMenu::PopulateOptionsSection(UToolMenu& InMenu)
{
	const TSharedPtr<INavigationToolFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return;
	}

	const TSharedRef<INavigationToolFilterBar> FilterBarRef = FilterBar.ToSharedRef();

	const FNavigationToolFilterCommands& FilterCommands = FNavigationToolFilterCommands::Get();

	FToolMenuSection& Section = InMenu.FindOrAddSection(TEXT("Options")
		, LOCTEXT("OptionsHeading", "Filter Bar Options"));

	Section.AddMenuEntry(FilterCommands.ToggleFilterBarVisibility);

	Section.AddSeparator(NAME_Name);

	Section.AddMenuEntry(TEXT("SaveCurrentFilterSetAsCustomTextFilter"),
		LOCTEXT("SaveCurrentFilterSetAsCustomTextFilter", "Save Current as New Filter"),
		LOCTEXT("SaveCurrentFilterSetAsCustomTextFilterTooltip", "Saves the enabled and active set of common filters as a custom text filter"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("LevelEditor.SaveAs")),
		FExecuteAction::CreateSP(FilterBarRef, &INavigationToolFilterBar::SaveCurrentFilterSetAsCustomTextFilter));

	Section.AddSeparator(NAME_Name);

	PopulateFilterBulkOptionsSection(InMenu);
}

void FNavigationToolFilterBarContextMenu::PopulateFilterBulkOptionsSection(UToolMenu& InMenu)
{
	const FNavigationToolFilterCommands& FilterCommands = FNavigationToolFilterCommands::Get();

	FToolMenuSection& Section = InMenu.FindOrAddSection(TEXT("FilterBulkOptions")
		, LOCTEXT("BulkOptionsContextHeading", "Filter Bulk Options"));

	Section.AddMenuEntry(TEXT("ActivateAllFilters"),
		LOCTEXT("ActivateAllFilters", "Activate All Filters"),
		LOCTEXT("ActivateAllFiltersTooltip", "Activates all enabled filters."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.Plus")),
		FExecuteAction::CreateSP(this, &FNavigationToolFilterBarContextMenu::OnActivateAllFilters, true));

	Section.AddMenuEntry(TEXT("DeactivateAllFilters"),
		LOCTEXT("DeactivateAllFilters", "Deactivate All Filters"),
		LOCTEXT("DeactivateAllFiltersTooltip", "Deactivates all enabled filters."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.Minus")),
		FExecuteAction::CreateSP(this, &FNavigationToolFilterBarContextMenu::OnActivateAllFilters, false));

	Section.AddSeparator(NAME_None);

	Section.AddMenuEntry(FilterCommands.ResetFilters
		, FilterCommands.ResetFilters->GetLabel()
		, FilterCommands.ResetFilters->GetDescription()
		, FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("PropertyWindow.DiffersFromDefault")));

	Section.AddSeparator(NAME_None);

	Section.AddMenuEntry(FilterCommands.ToggleMuteFilters
		, FilterCommands.ToggleMuteFilters->GetLabel()
		, FilterCommands.ToggleMuteFilters->GetDescription()
		, FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("Denied")));
}

void FNavigationToolFilterBarContextMenu::OnActivateAllFilters(const bool bInActivate)
{
	const TSharedPtr<INavigationToolFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return;
	}

	FilterBar->ActivateAllEnabledFilters(bInActivate, {});
}

void FNavigationToolFilterBarContextMenu::OnResetFilters()
{
	const TSharedPtr<INavigationToolFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return;
	}

	FilterBar->EnableAllFilters(false, {});
	FilterBar->EnableCustomTextFilters(false, {});
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
