// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class SWidget;
class UToolMenu;

namespace UE::SequenceNavigator
{

class INavigationToolFilterBar;

class FNavigationToolFilterBarContextMenu : public TSharedFromThis<FNavigationToolFilterBarContextMenu>
{
public:
	TSharedRef<SWidget> CreateMenu(const TSharedRef<INavigationToolFilterBar>& InFilterBar);

protected:
	void PopulateMenu(UToolMenu* const InMenu);

	void PopulateOptionsSection(UToolMenu& InMenu);
	void PopulateFilterBulkOptionsSection(UToolMenu& InMenu);

	void OnActivateAllFilters(const bool bInActivate);
	void OnResetFilters();

	TWeakPtr<INavigationToolFilterBar> WeakFilterBar;
};

} // namespace UE::SequenceNavigator
