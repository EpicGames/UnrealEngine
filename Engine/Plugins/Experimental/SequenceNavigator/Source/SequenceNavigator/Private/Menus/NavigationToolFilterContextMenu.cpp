// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavigationToolFilterContextMenu.h"
#include "Filters/Filters/NavigationToolFilter_CustomText.h"
#include "Filters/Filters/NavigationToolFilterBase.h"
#include "Filters/INavigationToolFilterBar.h"
#include "Filters/NavigationToolFilterCommands.h"
#include "Filters/NavigationToolFilterBar.h"
#include "Menus/NavigationToolFilterMenuContext.h"
#include "NavigationToolSettings.h"
#include "ToolMenu.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "NavigationToolFilterContextMenu"

namespace UE::SequenceNavigator
{

TSharedRef<SWidget> FNavigationToolFilterContextMenu::CreateMenuWidget(const TSharedRef<FNavigationToolFilter>& InFilter)
{
	WeakFilter = InFilter;

	UToolMenus* const ToolMenus = UToolMenus::Get();
	check(ToolMenus);

	const FName FilterMenuName = TEXT("SequenceNavigator.FilterContextMenu");
	if (!ToolMenus->IsMenuRegistered(FilterMenuName))
	{
		UToolMenu* const Menu = ToolMenus->RegisterMenu(FilterMenuName);
		Menu->AddDynamicSection(TEXT("PopulateMenu"), FNewToolMenuDelegate::CreateLambda([this](UToolMenu* const InMenu)
			{
				if (UNavigationToolFilterMenuContext* const Context = InMenu->FindContext<UNavigationToolFilterMenuContext>())
				{
					Context->OnPopulateMenu.ExecuteIfBound(InMenu);
				}
			}));
	}

	const TSharedPtr<FUICommandList> CommandList = InFilter->GetFilterInterface().GetCommandList();

	UNavigationToolFilterMenuContext* const ContextObject = NewObject<UNavigationToolFilterMenuContext>();
	ContextObject->Init(InFilter);
	ContextObject->OnPopulateMenu = FOnPopulateFilterBarMenu::CreateSP(this, &FNavigationToolFilterContextMenu::PopulateMenu);

	const FToolMenuContext MenuContext(CommandList, nullptr, ContextObject);
	return UToolMenus::Get()->GenerateWidget(FilterMenuName, MenuContext);
}

void FNavigationToolFilterContextMenu::PopulateMenu(UToolMenu* const InMenu)
{
	if (!InMenu)
	{
		return;
	}

	UToolMenu& MenuRef = *InMenu;

	PopulateFilterOptionsSection(MenuRef);
	PopulateCustomFilterOptionsSection(MenuRef);
	PopulateBulkOptionsSection(MenuRef);
}

void FNavigationToolFilterContextMenu::PopulateFilterOptionsSection(UToolMenu& InMenu)
{
	const TSharedPtr<FNavigationToolFilter> Filter = GetFilter();
	if (!Filter.IsValid())
	{
		return;
	}

	const FText FilterDisplayName = Filter->GetDisplayName();

	FToolMenuSection& Section = InMenu.FindOrAddSection(TEXT("FilterOptions")
		, LOCTEXT("FilterOptionsContextHeading", "Filter Options"));

	Section.AddMenuEntry(TEXT("ActivateOnlyThisFilter"),
		FText::Format(LOCTEXT("ActivateOnlyThisFilter", "Activate Only: {0}"), FilterDisplayName),
		LOCTEXT("ActivateOnlyThisFilterTooltip", "Activate only this filter from the list."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.Plus")),
		FExecuteAction::CreateSP(this, &FNavigationToolFilterContextMenu::OnActivateWithFilterException));

	Section.AddMenuEntry(TEXT("DisableFilter"),
		FText::Format(LOCTEXT("DisableFilter", "Remove: {0}"), FilterDisplayName),
		LOCTEXT("DisableFilterTooltip", "Disable this filter and remove it from the list. It can be added again in the filters menu."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.Minus")),
		FExecuteAction::CreateSP(this, &FNavigationToolFilterContextMenu::OnDisableFilter));
}

void FNavigationToolFilterContextMenu::PopulateCustomFilterOptionsSection(UToolMenu& InMenu)
{
	const TSharedPtr<FNavigationToolFilter> Filter = GetFilter();
	if (!Filter.IsValid())
	{
		return;
	}

	const TSharedPtr<FNavigationToolFilter_CustomText> CustomTextFilter
		= StaticCastSharedPtr<FNavigationToolFilter_CustomText>(Filter);
	if (!CustomTextFilter.IsValid() || !CustomTextFilter->IsCustomTextFilter())
	{
		return;
	}

	const FText FilterDisplayName = CustomTextFilter->GetDisplayName();

	FToolMenuSection& Section = InMenu.FindOrAddSection(TEXT("CustomFilterOptions")
		, LOCTEXT("CustomFilterOptionsContextHeading", "Custom Filter Options"));

	Section.AddMenuEntry(TEXT("EditCustomTextFilter"),
		FText::Format(LOCTEXT("EditCustomTextFilter", "Edit: {0}"), FilterDisplayName),
		LOCTEXT("EditCustomTextFilterTooltip", "Edit this custom text filter saved to config."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.Edit")),
		FExecuteAction::CreateSP(this, &FNavigationToolFilterContextMenu::OnEditFilter));

	Section.AddMenuEntry(TEXT("DeleteCustomTextFilter"),
		FText::Format(LOCTEXT("DeleteCustomTextFilter", "Delete: {0}"), FilterDisplayName),
		LOCTEXT("DeleteCustomTextFilterTooltip", "Delete this custom text filter from config.\n\nCAUTION: This cannot be undone!"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.Delete")),
		FExecuteAction::CreateSP(this, &FNavigationToolFilterContextMenu::OnDeleteFilter));
}

void FNavigationToolFilterContextMenu::PopulateBulkOptionsSection(UToolMenu& InMenu)
{
	const FNavigationToolFilterCommands& FilterCommands = FNavigationToolFilterCommands::Get();

	FToolMenuSection& Section = InMenu.FindOrAddSection(TEXT("FilterBulkOptions")
		, LOCTEXT("BulkOptionsContextHeading", "Filter Bulk Options"));

	Section.AddMenuEntry(FilterCommands.ActivateAllFilters);
	Section.AddMenuEntry(FilterCommands.DeactivateAllFilters);
	Section.AddSeparator(NAME_None);
	Section.AddMenuEntry(FilterCommands.ResetFilters);
	Section.AddSeparator(NAME_None);
	Section.AddMenuEntry(FilterCommands.ToggleMuteFilters);
}

void FNavigationToolFilterContextMenu::OnDisableFilter()
{
	const TSharedPtr<FNavigationToolFilter> Filter = GetFilter();
	if (!Filter.IsValid())
	{
		return;
	}

	const FString FilterDisplayName = Filter->GetDisplayName().ToString();
	Filter->GetFilterInterface().SetFilterEnabledByDisplayName(FilterDisplayName, false);
}

void FNavigationToolFilterContextMenu::OnResetFilters()
{
	const TSharedPtr<FNavigationToolFilter> Filter = GetFilter();
	if (!Filter.IsValid())
	{
		return;
	}

	Filter->GetFilterInterface().EnableAllFilters(false, {});
}

void FNavigationToolFilterContextMenu::OnActivateWithFilterException()
{
	const TSharedPtr<FNavigationToolFilter> Filter = GetFilter();
	if (!Filter.IsValid())
	{
		return;
	}

	INavigationToolFilterBar& FilterInterface = Filter->GetFilterInterface();
	const FString FilterDisplayName = Filter->GetDisplayName().ToString();

	FilterInterface.ActivateAllEnabledFilters(false, { FilterDisplayName });
	FilterInterface.SetFilterActiveByDisplayName(FilterDisplayName, true);
}

void FNavigationToolFilterContextMenu::OnActivateAllFilters(const bool bInActivate)
{
	const TSharedPtr<FNavigationToolFilter> Filter = GetFilter();
	if (!Filter.IsValid())
	{
		return;
	}

	Filter->GetFilterInterface().ActivateAllEnabledFilters(bInActivate, {});
}

void FNavigationToolFilterContextMenu::OnEditFilter()
{
	const TSharedPtr<FNavigationToolFilter> Filter = GetFilter();
	if (!Filter.IsValid())
	{
		return;
	}

	const TSharedPtr<FNavigationToolFilter_CustomText> CustomTextFilter = StaticCastSharedPtr<FNavigationToolFilter_CustomText>(Filter);
	if (!CustomTextFilter.IsValid())
	{
		return;
	}

	FNavigationToolFilterBar& FilterBar = static_cast<FNavigationToolFilterBar&>(Filter->GetFilterInterface());

	FilterBar.CreateWindow_EditCustomTextFilter(CustomTextFilter);
}

void FNavigationToolFilterContextMenu::OnDeleteFilter()
{
	const TSharedPtr<FNavigationToolFilter> Filter = GetFilter();
	if (!Filter.IsValid())
	{
		return;
	}

	const TSharedPtr<FNavigationToolFilter_CustomText> CustomTextFilter = StaticCastSharedPtr<FNavigationToolFilter_CustomText>(Filter);
	if (!CustomTextFilter.IsValid())
	{
		return;
	}

	INavigationToolFilterBar& FilterInterface = Filter->GetFilterInterface();
	const FString FilterDisplayName = Filter->GetDisplayName().ToString();

	FilterInterface.SetFilterActiveByDisplayName(FilterDisplayName, false);
	FilterInterface.RemoveCustomTextFilter(CustomTextFilter.ToSharedRef(), false);

	if (UNavigationToolSettings* const ToolSettings = GetMutableDefault<UNavigationToolSettings>())
	{
		FSequencerFilterBarConfig& Config = ToolSettings->FindOrAddFilterBar(FilterInterface.GetIdentifier(), false);

		if (!Config.RemoveCustomTextFilter(FilterDisplayName))
		{
			return;
		}

		ToolSettings->SaveConfig();
	}
}

const TSharedPtr<FNavigationToolFilter> FNavigationToolFilterContextMenu::GetFilter() const
{
	return WeakFilter.IsValid() ? WeakFilter.Pin() : nullptr;
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
