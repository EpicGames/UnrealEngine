// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class SWidget;
class UToolMenu;

namespace UE::SequenceNavigator
{

class FNavigationToolFilter;

class FNavigationToolFilterContextMenu : public TSharedFromThis<FNavigationToolFilterContextMenu>
{
public:
	TSharedRef<SWidget> CreateMenuWidget(const TSharedRef<FNavigationToolFilter>& InFilter);

protected:
	void PopulateMenu(UToolMenu* const InMenu);

	void PopulateFilterOptionsSection(UToolMenu& InMenu);
	void PopulateCustomFilterOptionsSection(UToolMenu& InMenu);
	void PopulateBulkOptionsSection(UToolMenu& InMenu);

	void OnDisableFilter();
	void OnResetFilters();

	void OnActivateWithFilterException();

	void OnActivateAllFilters(const bool bInActivate);

	void OnEditFilter();
	void OnDeleteFilter();

	const TSharedPtr<FNavigationToolFilter> GetFilter() const;

	TWeakPtr<FNavigationToolFilter> WeakFilter;
};

} // namespace UE::SequenceNavigator
