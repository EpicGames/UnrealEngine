// Copyright Epic Games, Inc. All Rights Reserved.

#include "Menus/NavigationToolFilterMenu.h"
#include "Filters/Filters/NavigationToolFilter_CustomText.h"
#include "Filters/Filters/NavigationToolFilterBase.h"
#include "Filters/NavigationToolFilterBar.h"
#include "Menus/NavigationToolFilterBarContext.h"
#include "NavigationToolSettings.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "ToolMenuSection.h"

#define LOCTEXT_NAMESPACE "NavigationToolFilterMenu"

namespace UE::SequenceNavigator
{

TSharedRef<SWidget> FNavigationToolFilterMenu::CreateMenu(const TSharedRef<INavigationToolFilterBar>& InFilterBar)
{
	const FName FilterMenuName = TEXT("SequenceNavigator.FilterMenu");
	if (!UToolMenus::Get()->IsMenuRegistered(FilterMenuName))
	{
		UToolMenu* const Menu = UToolMenus::Get()->RegisterMenu(FilterMenuName);
		Menu->bShouldCloseWindowAfterMenuSelection = false;
		Menu->AddDynamicSection(NAME_None, FNewToolMenuDelegate::CreateLambda([this](UToolMenu* const InMenu)
			{
				if (UNavigationToolFilterBarContext* const Context = InMenu->FindContext<UNavigationToolFilterBarContext>())
				{
					Context->OnPopulateMenu.ExecuteIfBound(InMenu);
				}
			}));
	}

	UNavigationToolFilterBarContext* const ContextObject = NewObject<UNavigationToolFilterBarContext>();
	ContextObject->Init(InFilterBar);
	ContextObject->OnPopulateMenu = FOnPopulateFilterBarMenu::CreateRaw(this, &FNavigationToolFilterMenu::PopulateMenu);

	const FToolMenuContext MenuContext(InFilterBar->GetCommandList(), nullptr, ContextObject);
	return UToolMenus::Get()->GenerateWidget(FilterMenuName, MenuContext);
}

void FNavigationToolFilterMenu::PopulateMenu(UToolMenu* const InMenu)
{
	if (!InMenu)
	{
		return;
	}

	UNavigationToolFilterBarContext* const Context = InMenu->FindContext<UNavigationToolFilterBarContext>();
	if (!Context)
	{
		return;
	}

	WeakFilterBar = Context->GetFilterBar();

	UToolMenu& Menu = *InMenu;

	PopulateFilterOptionsSection(Menu);
	PopulateCustomsSection(Menu);
	PopulateCommonFilterSections(Menu);
}

void FNavigationToolFilterMenu::PopulateCustomsSection(UToolMenu& InMenu)
{
	const TSharedPtr<INavigationToolFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return;
	}

	FToolMenuSection& Section = InMenu.FindOrAddSection(TEXT("UserCreated"));

	Section.AddSubMenu(TEXT("CustomTextFilters"),
		LOCTEXT("CustomTextFilters_SubMenuLabel", "Custom Text Filters"),
		LOCTEXT("CustomTextFilters_SubMenuTooltip", "Custom Text Filters"),
		FNewToolMenuDelegate::CreateSP(this, &FNavigationToolFilterMenu::FillCustomTextFiltersMenu),
		false,
		FSlateIcon(),
		false);
}

void FNavigationToolFilterMenu::PopulateFilterOptionsSection(UToolMenu& InMenu)
{
	const TSharedPtr<INavigationToolFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return;
	}

	FToolMenuSection& Section = InMenu.FindOrAddSection(TEXT("FilterOptions"), LOCTEXT("FilterOptionsHeading", "Filters"));

	Section.AddMenuEntry(TEXT("ResetFilters"),
		LOCTEXT("FilterListResetFilters", "Reset Filters"),
		LOCTEXT("FilterListResetToolTip", "Resets current filter selection"),
		FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("PropertyWindow.DiffersFromDefault")),
		FUIAction(
			FExecuteAction::CreateSP(this, &FNavigationToolFilterMenu::ResetFilters),
			FCanExecuteAction::CreateSP(this, &FNavigationToolFilterMenu::CanResetFilters)
		));

	const UNavigationToolSettings* const ToolSettings = GetDefault<UNavigationToolSettings>();
	if (ToolSettings && ToolSettings->ShouldUseFilterSubmenusForCategories())
	{
		Section.AddSeparator(NAME_None);
	}
}

void FNavigationToolFilterMenu::PopulateCommonFilterSections(UToolMenu& InMenu)
{
	const TSharedPtr<INavigationToolFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return;
	}

	const UNavigationToolSettings* const ToolSettings = GetDefault<UNavigationToolSettings>();
	if (!ToolSettings)
	{
		return;
	}

	if (ToolSettings->ShouldUseFilterSubmenusForCategories())
	{
		FToolMenuSection& Section = InMenu.FindOrAddSection(TEXT("OtherFilters"), LOCTEXT("OtherFiltersHeading", "Other Filters"));

		for (const TSharedRef<FFilterCategory>& Category : FilterBar->GetFilterCategories())
		{
			Section.AddSubMenu(*Category->Title.ToString(),
				Category->Title,
				Category->Tooltip,
				FNewToolMenuDelegate::CreateSP(this, &FNavigationToolFilterMenu::FillFiltersMenuCategory, Category),
				FUIAction(
					FExecuteAction::CreateSP(this, &FNavigationToolFilterMenu::OnFilterCategoryClicked, Category),
					FCanExecuteAction(),
					FGetActionCheckState::CreateSP(this, &FNavigationToolFilterMenu::GetFilterCategoryCheckedState, Category)
				),
				EUserInterfaceActionType::ToggleButton,
				false,
				FSlateIcon(),
				false);
		}
	}
	else
	{
		for (const TSharedRef<FFilterCategory>& Category : FilterBar->GetFilterCategories())
		{
			FToolMenuSection& Section = InMenu.FindOrAddSection(*Category->Title.ToString(), Category->Title);
			FillFiltersMenuCategory(Section, Category);
		}
	}
}

void FNavigationToolFilterMenu::FillCustomTextFiltersMenu(UToolMenu* const InMenu)
{
	if (!InMenu)
	{
		return;
	}

	const TSharedPtr<INavigationToolFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return;
	}

	UNavigationToolSettings* const ToolSettings = GetMutableDefault<UNavigationToolSettings>();
	if (!ToolSettings)
	{
		return;
	}

	const TSharedRef<INavigationToolFilterBar> FilterBarRef = FilterBar.ToSharedRef();

	FToolMenuSection& CustomTextOptionsSection = InMenu->FindOrAddSection(TEXT("CustomTextFilterOptions")
		, LOCTEXT("CustomTextFilterOptions", "Custom Text Filter Options")
		, FToolMenuInsert(NAME_None, EToolMenuInsertType::First));

	CustomTextOptionsSection.AddMenuEntry(TEXT("TextExpressionHelp"),
		LOCTEXT("TextExpressionHelp", "Text Expression Help"),
		LOCTEXT("TextExpressionHelpToolTip", "Opens the help dialog for the advanced search syntax text expressions"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.Help")),
		FExecuteAction::CreateSP(FilterBarRef, &INavigationToolFilterBar::OpenTextExpressionHelp));

	CustomTextOptionsSection.AddMenuEntry(TEXT("SaveCurrentAsNewTextFilter"),
		LOCTEXT("SaveCurrentAsNewTextFilter", "Save Current as New Filter"),
		LOCTEXT("SaveCurrentAsNewTextFilterToolTip", "Saves the enabled and active set of common filters as a custom text filter"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("LevelEditor.SaveAs")),
		FExecuteAction::CreateSP(FilterBarRef, &INavigationToolFilterBar::SaveCurrentFilterSetAsCustomTextFilter));

	CustomTextOptionsSection.AddMenuEntry(TEXT("CreateNewTextFilter"),
		LOCTEXT("CreateNewTextFilter", "Create New Filter"),
		LOCTEXT("CreateNewTextFilterTooltip", "Create a new text filter"),
		FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("Icons.PlusCircle")),
		FExecuteAction::CreateSP(FilterBarRef, &INavigationToolFilterBar::CreateNewTextFilter));

	FSequencerFilterBarConfig& Config = ToolSettings->FindOrAddFilterBar(FilterBar->GetIdentifier(), true);

	const TArray<FCustomTextFilterData>& CustomTextFilterDatas = Config.GetCustomTextFilters();
	if (CustomTextFilterDatas.IsEmpty())
	{
		return;
	}

	FToolMenuSection& FiltersSection = InMenu->AddSection(TEXT("CustomTextFilters"), LOCTEXT("CustomTextFilters", "Custom Text Filters"));

	for (const FCustomTextFilterData& CustomTextFilterData : CustomTextFilterDatas)
	{
		const TSharedPtr<SWidget> MenuItem = ConstructCustomMenuItemWidget(CustomTextFilterData.FilterLabel
			, FSimpleDelegate::CreateSP(this, &FNavigationToolFilterMenu::OnCustomTextFilterClicked, CustomTextFilterData.FilterLabel)
			, TAttribute<ECheckBoxState>::CreateSP(this, &FNavigationToolFilterMenu::GetCustomTextFilerCheckState, CustomTextFilterData.FilterLabel)
			, FAppStyle::GetBrush(TEXT("Icons.Edit"))
			, FSimpleDelegate::CreateSP(this, &FNavigationToolFilterMenu::OnEditCustomTextFilterClicked, CustomTextFilterData.FilterLabel));
		FiltersSection.AddEntry(FToolMenuEntry::InitWidget(*CustomTextFilterData.FilterLabel.ToString(), MenuItem.ToSharedRef(), FText::GetEmpty()));
	}
}

void FNavigationToolFilterMenu::FillFiltersMenuCategory(FToolMenuSection& InOutSection, const TSharedRef<FFilterCategory> InMenuCategory)
{
	const TSharedPtr<INavigationToolFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return;
	}

	UMovieSceneSequence* const FocusedSequence = FilterBar->GetSequencer().GetFocusedMovieSceneSequence();
	if (!FocusedSequence)
	{
		return;
	}

	for (const TSharedRef<FNavigationToolFilter>& Filter : FilterBar->GetCommonFilters())
	{
		if (Filter->GetCategory() != InMenuCategory)
		{
			continue;
		}

		if (!FocusedSequence->IsFilterSupported(Filter->GetName()) && !Filter->SupportsSequence(FocusedSequence))
		{
			continue;
		}

		const FText DisplayName = Filter->GetDisplayName();

		FToolMenuEntry& NewMenuEntry = InOutSection.AddMenuEntry(*DisplayName.ToString(),
			DisplayName,
			Filter->GetToolTipText(),
			Filter->GetIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &FNavigationToolFilterMenu::OnFilterClicked, Filter),
				FCanExecuteAction::CreateLambda([this]()
					{
						return WeakFilterBar.IsValid() ? !WeakFilterBar.Pin()->AreFiltersMuted() : false;
					}),
				FIsActionChecked::CreateLambda([this, Filter]()
					{
						return WeakFilterBar.IsValid() ? WeakFilterBar.Pin()->IsFilterEnabled(Filter) : false;
					})
			),
			EUserInterfaceActionType::ToggleButton);

		if (const TSharedPtr<FUICommandInfo> ToggleCommand = Filter->GetToggleCommand())
		{
			NewMenuEntry.InputBindingLabel = ToggleCommand->GetInputText();
		}
	}
}

void FNavigationToolFilterMenu::FillFiltersMenuCategory(UToolMenu* const InMenu, const TSharedRef<FFilterCategory> InMenuCategory)
{
	if (InMenu)
	{
		FillFiltersMenuCategory(InMenu->AddSection(TEXT("Section")), InMenuCategory);
	}
}

TSharedRef<SWidget> FNavigationToolFilterMenu::ConstructCustomMenuItemWidget(const TAttribute<FText>& InItemText
	, const FSimpleDelegate& InOnItemClicked
	, const TAttribute<ECheckBoxState>& InIsChecked
	, const FSlateBrush* const InButtonImage
	, const FSimpleDelegate& InOnButtonClicked
	, const bool bInRadioButton)
{
	const FCheckBoxStyle& CheckBoxStyle = bInRadioButton
		? FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>(TEXT("Menu.RadioButton"))
		: FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>(TEXT("Checkbox"));

	return SNew(SButton)
		.ContentPadding(0.f)
		.ButtonStyle(FAppStyle::Get(), TEXT("Menu.Button"))
		.ForegroundColor(FSlateColor::UseForeground())
		.ToolTipText(InItemText)
		.OnClicked_Lambda([InOnItemClicked]()
			{
				InOnItemClicked.ExecuteIfBound();
				return FReply::Handled();
			})
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(12.f, 0.f, 12.f, 0.f)
			[
				SNew(SCheckBox)
				.Style(&CheckBoxStyle)
				.OnCheckStateChanged_Lambda([InOnItemClicked](const ECheckBoxState InNewState)
					{
						InOnItemClicked.ExecuteIfBound();
					})
				.IsChecked(InIsChecked)
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.ColorAndOpacity(FStyleColors::White)
				.Text(InItemText)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(12.f, 0.f)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), TEXT("SimpleButton"))
				.ContentPadding(FMargin(0, 2))
				.OnClicked_Lambda([InOnButtonClicked]()
					{
						InOnButtonClicked.ExecuteIfBound();
						return FReply::Handled();
					})
				[
					SNew(SImage)
					.DesiredSizeOverride(FVector2D(16.f, 16.f))
					.Image(InButtonImage)
				]
			]
		];
}

void FNavigationToolFilterMenu::OnFilterCategoryClicked(const TSharedRef<FFilterCategory> InMenuCategory)
{
	const TSharedPtr<FNavigationToolFilterBar> FilterBar = StaticCastSharedPtr<FNavigationToolFilterBar>(WeakFilterBar.Pin());
	if (!FilterBar.IsValid())
	{
		return;
	}

	const ECheckBoxState CategoryCheckState = GetFilterCategoryCheckedState(InMenuCategory);
	const bool bIsCategoryEnabled = (CategoryCheckState == ECheckBoxState::Checked);
	FilterBar->ActivateCommonFilters(bIsCategoryEnabled, { InMenuCategory }, {});
}

ECheckBoxState FNavigationToolFilterMenu::GetFilterCategoryCheckedState(const TSharedRef<FFilterCategory> InMenuCategory) const
{
	const TSharedPtr<INavigationToolFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return ECheckBoxState::Unchecked;
	}

	for (const TSharedRef<FNavigationToolFilter>& Filter : FilterBar->GetCommonFilters())
	{
		if (Filter->GetCategory() == InMenuCategory && !FilterBar->IsFilterEnabled(Filter))
		{
			return ECheckBoxState::Unchecked;
		}
	}

	return ECheckBoxState::Checked;
}

void FNavigationToolFilterMenu::OnFilterClicked(const TSharedRef<FNavigationToolFilter> InFilter)
{
	const TSharedPtr<INavigationToolFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return;
	}

	if (FilterBar->IsFilterEnabled(InFilter))
	{
		FilterBar->SetFilterEnabled(InFilter, false, true);
	}
	else
	{
		FilterBar->SetFilterActive(InFilter, true, true);
	}
}

void FNavigationToolFilterMenu::OnCustomTextFilterClicked(const FText InFilterLabel)
{
	const TSharedPtr<INavigationToolFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return;
	}

	const TSharedPtr<FNavigationToolFilter_CustomText> CustomTextFilter = FilterBar->FindCustomTextFilterByDisplayName(InFilterLabel.ToString());
	if (!CustomTextFilter.IsValid())
	{
		return;
	}

	const TSharedRef<FNavigationToolFilter_CustomText> CustomTextFilterRef = CustomTextFilter.ToSharedRef();

	if (FilterBar->IsFilterEnabled(CustomTextFilterRef))
	{
		FilterBar->SetFilterEnabled(CustomTextFilterRef, false, true);
	}
	else
	{
		FilterBar->SetFilterActive(CustomTextFilterRef, true, true);
	}
}

ECheckBoxState FNavigationToolFilterMenu::GetCustomTextFilerCheckState(const FText InFilterLabel) const
{
	const TSharedPtr<INavigationToolFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return ECheckBoxState::Unchecked;
	}

	if (const TSharedPtr<FNavigationToolFilter_CustomText> CustomTextFilter = FilterBar->FindCustomTextFilterByDisplayName(InFilterLabel.ToString()))
	{
		return FilterBar->IsFilterEnabled(CustomTextFilter.ToSharedRef()) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	return ECheckBoxState::Unchecked;
}

void FNavigationToolFilterMenu::OnEditCustomTextFilterClicked(const FText InFilterLabel)
{
	const TSharedPtr<INavigationToolFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return;
	}

	if (const TSharedPtr<FNavigationToolFilter_CustomText> CustomTextFilter = FilterBar->FindCustomTextFilterByDisplayName(InFilterLabel.ToString()))
	{
		FilterBar->CreateWindow_EditCustomTextFilter(CustomTextFilter);
	}
}

bool FNavigationToolFilterMenu::CanResetFilters() const
{
	const TSharedPtr<FNavigationToolFilterBar> FilterBar = StaticCastSharedPtr<FNavigationToolFilterBar>(WeakFilterBar.Pin());
	if (!FilterBar.IsValid())
	{
		return false;
	}

	if (FilterBar->AreFiltersMuted())
	{
		return true;
	}

	const TArray<TSharedRef<FFilterCategory>> Categories = {
		FilterBar->GetClassTypeCategory(),
		FilterBar->GetComponentTypeCategory(),
		FilterBar->GetMiscCategory()
	};
	const TArray<TSharedRef<FNavigationToolFilter>> ClassAndCompFilters = FilterBar->GetCommonFilters(Categories);

	return FilterBar->HasEnabledFilter(ClassAndCompFilters) || FilterBar->HasEnabledCustomTextFilters();
}

void FNavigationToolFilterMenu::ResetFilters()
{
	const TSharedPtr<INavigationToolFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return;
	}

	FilterBar->ResetFilters();
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
