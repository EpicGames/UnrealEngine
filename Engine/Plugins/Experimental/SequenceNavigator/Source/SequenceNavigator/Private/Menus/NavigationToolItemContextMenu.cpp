// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavigationToolItemContextMenu.h"
#include "Framework/Commands/GenericCommands.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "Menus/NavigationToolItemMenuContext.h"
#include "NavigationTool.h"
#include "NavigationToolCommands.h"
#include "NavigationToolView.h"

#define LOCTEXT_NAMESPACE "NavigationToolItemContextMenu"

namespace UE::SequenceNavigator
{

using namespace Sequencer;

FName FNavigationToolItemContextMenu::GetMenuName()
{
	static const FName MenuName(TEXT("SequenceNavigator.ItemContextMenu"));
	return MenuName;
}

TSharedRef<SWidget> FNavigationToolItemContextMenu::CreateMenu(const TSharedRef<FNavigationToolView>& InToolView
	, const TArray<FNavigationToolViewModelWeakPtr>& InWeakItemList)
{
	UToolMenus* const ToolMenus = UToolMenus::Get();

	const FName MenuName = GetMenuName();

	if (!ToolMenus->IsMenuRegistered(MenuName))
	{
		UToolMenu* const ContextMenu = ToolMenus->RegisterMenu(MenuName, NAME_None, EMultiBoxType::Menu);
		ContextMenu->AddDynamicSection(TEXT("Main"), FNewToolMenuDelegate::CreateStatic(&FNavigationToolItemContextMenu::PopulateMenu));
	}

	UNavigationToolItemMenuContext* const ContextObject = NewObject<UNavigationToolItemMenuContext>();
	ContextObject->Init(InToolView->GetOwnerTool(), InWeakItemList);

	FToolMenuContext Context(InToolView->GetBaseCommandList(), nullptr, ContextObject);
	return ToolMenus->GenerateWidget(MenuName, Context);
}

void FNavigationToolItemContextMenu::PopulateMenu(UToolMenu* const InMenu)
{
	if (!InMenu)
	{
		return;
	}

	UToolMenu& MenuRef = *InMenu;

	UNavigationToolItemMenuContext* const ItemsContext = MenuRef.FindContext<UNavigationToolItemMenuContext>();
	if (!ItemsContext)
	{
		return;
	}

	CreateToolSection(MenuRef);
	CreateGenericSection(MenuRef);

	/*FToolMenuSection* ContextActionsSection = MenuRef.FindSection(TEXT("ContextActions"));
	if (!ContextActionsSection)
	{
		ContextActionsSection = &MenuRef.AddSection(TEXT("ContextActions")
			, LOCTEXT("ContextActions", "Context Actions")
			, FToolMenuInsert(NAME_None, EToolMenuInsertType::First));
	}*/

	// Note: Since the Navigation Tool Command List is linked to the Command List (see INavigationTool::SetBaseCommandList),
	// we do NOT need to add the entry with a different Command List
	/*ContextActionsSection->AddMenuEntry(
		ToolCommands.OpenAdvancedRenamerTool_SelectedActors,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("GenericCommands.Rename")));

	ContextActionsSection->AddMenuEntry(
		ToolCommands.OpenAdvancedRenamerTool_SharedClassActors,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("GenericCommands.Rename")));*/
}

void FNavigationToolItemContextMenu::CreateGenericSection(UToolMenu& InMenu)
{
	// Disabling all for now
	/*const FGenericCommands& GenericCommands = FGenericCommands::Get();

	FToolMenuSection& GenericSection = InMenu.FindOrAddSection(TEXT("GenericActions"), LOCTEXT("GenericActionsHeader", "Generic Actions"));

	//GenericSection.AddMenuEntry(GenericCommands.Cut);
	//GenericSection.AddMenuEntry(GenericCommands.Copy);
	//GenericSection.AddMenuEntry(GenericCommands.Paste);
	//GenericSection.AddMenuEntry(GenericCommands.Duplicate);
	GenericSection.AddMenuEntry(GenericCommands.Delete);
	GenericSection.AddMenuEntry(GenericCommands.Rename);*/
}

void FNavigationToolItemContextMenu::CreateToolSection(UToolMenu& InMenu)
{
	const FNavigationToolCommands& ToolCommands = FNavigationToolCommands::Get();

	FToolMenuSection& ToolSection = InMenu.FindOrAddSection(TEXT("ToolActions"), LOCTEXT("ToolActionsHeader", "Sequence Navigator Actions"));

	FToolMenuEntry& ExpandAllEntry = ToolSection.AddMenuEntry(ToolCommands.ExpandAll);
	ExpandAllEntry.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("TreeArrow_Expanded"));

	FToolMenuEntry& CollapseAllEntry = ToolSection.AddMenuEntry(ToolCommands.CollapseAll);
	CollapseAllEntry.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("TreeArrow_Collapsed"));

	FToolMenuEntry& SelectAllChildrenEntry = ToolSection.AddMenuEntry(ToolCommands.SelectAllChildren);
	SelectAllChildrenEntry.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("LevelEditor.SelectAllDescendants"));

	FToolMenuEntry& SelectImmediateChildrenEntry = ToolSection.AddMenuEntry(ToolCommands.SelectImmediateChildren);
	SelectImmediateChildrenEntry.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("LevelEditor.SelectImmediateChildren"));

	FToolMenuEntry& FocusSelectionEntry = ToolSection.AddMenuEntry(ToolCommands.FocusSingleSelection);
	FocusSelectionEntry.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("WorldPartition.FocusSelection"));

	FToolMenuEntry& FocusInContentBrowserEntry = ToolSection.AddMenuEntry(ToolCommands.FocusInContentBrowser);
	FocusInContentBrowserEntry.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("ContentBrowser.TabIcon"));
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
