// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NavigationToolDefines.h"
#include "Templates/SharedPointer.h"

class FName;
class SWidget;
class UToolMenu;

namespace UE::SequenceNavigator
{

class FNavigationToolView;
class INavigationToolItem;

class FNavigationToolItemContextMenu : public TSharedFromThis<FNavigationToolItemContextMenu>
{
public:
	static FName GetMenuName();

	TSharedRef<SWidget> CreateMenu(const TSharedRef<FNavigationToolView>& InToolView
		, const TArray<FNavigationToolViewModelWeakPtr>& InWeakItemList);

protected:
	static void PopulateMenu(UToolMenu* const InMenu);

	static void CreateGenericSection(UToolMenu& InMenu);
	static void CreateToolSection(UToolMenu& InMenu);
};

} // namespace UE::SequenceNavigator
