// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavigationToolToolbarMenu.h"
#include "Columns/INavigationToolColumn.h"
#include "Filters/Filters/NavigationToolBuiltInFilter.h"
#include "Filters/NavigationToolFilterBar.h"
#include "Filters/NavigationToolFilterCommands.h"
#include "Filters/Widgets/SSequencerSearchBox.h"
#include "Menus/NavigationToolFilterMenu.h"
#include "Menus/NavigationToolViewMenuContext.h"
#include "NavigationTool.h"
#include "NavigationToolCommands.h"
#include "NavigationToolSettings.h"
#include "NavigationToolView.h"
#include "ToolMenu.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "NavigationToolToolbarMenu"

namespace UE::SequenceNavigator
{

FName FNavigationToolToolbarMenu::GetMenuName()
{
	static const FName ToolBarName(TEXT("SequenceNavigator.ToolBar"));
	return ToolBarName;
}

TSharedRef<SWidget> FNavigationToolToolbarMenu::CreateToolbar(const TSharedRef<FNavigationToolView>& InToolView)
{
	UToolMenus* const ToolMenus = UToolMenus::Get();

	const FName ToolbarName = GetMenuName();

	if (!ToolMenus->IsMenuRegistered(ToolbarName))
	{
		UToolMenu* const ToolBar = ToolMenus->RegisterMenu(ToolbarName, NAME_None, EMultiBoxType::SlimHorizontalToolBar);
		ToolBar->StyleName = TEXT("StatusBarToolBar");
		ToolBar->bToolBarForceSmallIcons = true;
		ToolBar->bToolBarIsFocusable = true;
		ToolBar->bShouldCloseWindowAfterMenuSelection = false;
		ToolBar->AddDynamicSection(TEXT("Main"), FNewToolMenuDelegate::CreateLambda([this](UToolMenu* const InMenu)
			{
				if (UNavigationToolViewMenuContext* const Context = InMenu->FindContext<UNavigationToolViewMenuContext>())
				{
					Context->OnPopulateMenu.ExecuteIfBound(InMenu);
				}
			}));
	}

	UNavigationToolViewMenuContext* const ContextObject = NewObject<UNavigationToolViewMenuContext>();
	ContextObject->Init(InToolView);
	ContextObject->OnPopulateMenu = FOnPopulateNavigationToolViewToolbarMenu::CreateRaw(this, &FNavigationToolToolbarMenu::PopulateToolBar);

	const FToolMenuContext Context(InToolView->GetBaseCommandList(), nullptr, ContextObject);
	return ToolMenus->GenerateWidget(ToolbarName, Context);
}

void FNavigationToolToolbarMenu::PopulateToolBar(UToolMenu* const InMenu)
{
	if (!InMenu)
	{
		return;
	}

	UNavigationToolViewMenuContext* const ToolBarContext = InMenu->FindContext<UNavigationToolViewMenuContext>();
	if (!ToolBarContext)
	{
		return;
	}

	const TSharedPtr<FNavigationToolView> ToolView = ToolBarContext->GetToolView();
	if (!ToolView.IsValid())
	{
		return;
	}

	const TSharedPtr<FNavigationToolFilterBar> FilterBar = ToolView->GetFilterBar();
	if (!FilterBar.IsValid())
	{
		return;
	}

	FToolMenuSection& MainSection = InMenu->FindOrAddSection(TEXT("Main")
		, LOCTEXT("MainSection", "Main"));

	MainSection.AddEntry(FToolMenuEntry::InitWidget(TEXT("AddFilter"),
		FilterBar->MakeAddFilterButton(),
		LOCTEXT("AddFilter", "Add Filter")));

	FToolMenuEntry SearchBoxEntry = FToolMenuEntry::InitWidget(TEXT("SearchBox"),
		FilterBar->GetOrCreateSearchBoxWidget(), FText());
	SearchBoxEntry.WidgetData.StyleParams.VerticalAlignment = VAlign_Center;
	SearchBoxEntry.WidgetData.StyleParams.HorizontalAlignment = HAlign_Fill;
	SearchBoxEntry.WidgetData.StyleParams.FillSize = 1.f;
	SearchBoxEntry.WidgetData.StyleParams.SizeRule = FSizeParam::ESizeRule::SizeRule_Stretch;
	MainSection.AddEntry(SearchBoxEntry);

	const FToolMenuEntry SettingsEntry = FToolMenuEntry::InitComboButton(TEXT("Settings"),
		FUIAction(),
		FNewToolMenuDelegate::CreateSP(this, &FNavigationToolToolbarMenu::CreateSettingsMenu),
		LOCTEXT("SettingsLabel", "Settings"),
		LOCTEXT("SettingsToolTip", "Settings"),
		FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("Icons.Settings")));
	MainSection.AddEntry(SettingsEntry);
}

void FNavigationToolToolbarMenu::CreateSettingsMenu(UToolMenu* const InMenu)
{
	if (!InMenu)
	{
		return;
	}

	const FNavigationToolCommands& ToolCommands = FNavigationToolCommands::Get();

	FToolMenuSection& ViewOptionsSection = InMenu->FindOrAddSection(TEXT("ViewOptions")
		, LOCTEXT("ViewOptionsHeading", "View Options"));

	ViewOptionsSection.AddMenuEntry(ToolCommands.ExpandAll);
	ViewOptionsSection.AddMenuEntry(ToolCommands.CollapseAll);
	// Disable until this is applicable
	//ViewOptionsSection.AddMenuEntry(ToolCommands.ToggleMutedHierarchy);
	ViewOptionsSection.AddMenuEntry(ToolCommands.ToggleAutoExpandToSelection);
	ViewOptionsSection.AddMenuEntry(ToolCommands.ToggleShortNames);

	CreateItemViewOptionsMenu(InMenu);

	FToolMenuSection& MiscSection = InMenu->FindOrAddSection(TEXT("Misc")
		, LOCTEXT("MiscHeading", "Misc"));

	MiscSection.AddSubMenu(TEXT("ColumnViews")
		, LOCTEXT("ColumnViewsSubMenu", "Column Views")
		, LOCTEXT("ColumnViewsSubMenuTooltip", "Column View Options")
		, FNewToolMenuDelegate::CreateSP(this, &FNavigationToolToolbarMenu::CreateColumnViewOptionsMenu));

	MiscSection.AddSubMenu(TEXT("FilterBarOptions")
		, LOCTEXT("FilterBarOptionsSubMenu", "Filter Bar Options")
		, LOCTEXT("FilterBarOptionsSubMenuTooltip", "Filter Bar Options")
		, FNewToolMenuDelegate::CreateSP(this, &FNavigationToolToolbarMenu::CreateFilterBarOptionsMenu));

	MiscSection.AddMenuEntry(ToolCommands.OpenToolSettings);
}

void FNavigationToolToolbarMenu::CreateItemViewOptionsMenu(UToolMenu* const InMenu)
{
	if (!InMenu)
	{
		return;
	}

	UNavigationToolViewMenuContext* const ToolBarContext = InMenu->FindContext<UNavigationToolViewMenuContext>();
	if (!ToolBarContext)
	{
		return;
	}

	const TSharedPtr<FNavigationToolView> ToolView = ToolBarContext->GetToolView();
	if (!ToolView.IsValid())
	{
		return;
	}

	const TSharedPtr<FNavigationTool> Tool = ToolBarContext->GetTool();
	if (!Tool.IsValid())
	{
		return;
	}

	const TSharedRef<FNavigationToolView> ToolViewRef = ToolView.ToSharedRef();

	// Toggle global proxy items
	FToolMenuSection& ItemTypeSection = InMenu->FindOrAddSection(TEXT("ItemTypeVisibility")
		, LOCTEXT("ItemTypeVisibilityHeading", "Item Type Visibility"));

	const TArray<FName> ItemProxyTypeNames = Tool->GetRegisteredItemProxyTypeNames();

	for (const FName RegisteredTypeName : ItemProxyTypeNames)
	{
		INavigationToolItemProxyFactory* const ItemProxyFactory = Tool->GetItemProxyFactory(RegisteredTypeName);
		check(ItemProxyFactory);

		const TSharedPtr<FNavigationToolItemProxy> TemplateProxyItem = ItemProxyFactory->CreateItemProxy(*Tool, nullptr);

		// Template Proxy Item creation might fail if ItemProxyFactory is not the default (where it always returns a constructed item)
		if (!TemplateProxyItem.IsValid())
		{
			continue;
		}

		const FName ItemProxyTypeName = TemplateProxyItem->GetTypeTable().GetTypeName();

		// Type mismatch here can continue but recommended to be addressed for Proxy Items / Factories
		// not correctly overriding Type Name Functions
		ensureMsgf(ItemProxyTypeName == RegisteredTypeName,
			TEXT("Item Proxy Type (%s) does not match Factory registered type (%s)! Check override of the Type Name Getters for both"),
			*ItemProxyTypeName.ToString(),
			*RegisteredTypeName.ToString());

		ItemTypeSection.AddMenuEntry(NAME_None,
			TemplateProxyItem->GetDisplayName(),
			TemplateProxyItem->GetIconTooltipText(),
			TemplateProxyItem->GetIcon(),
			FUIAction(
				FExecuteAction::CreateSP(ToolViewRef, &FNavigationToolView::ToggleHideItemTypes, ItemProxyTypeName),
				FCanExecuteAction(),
				FGetActionCheckState::CreateSP(ToolViewRef, &FNavigationToolView::GetToggleHideItemTypesState, ItemProxyTypeName)
			),
			EUserInterfaceActionType::ToggleButton);
	}

	// Toggle global view modes
	auto AddViewModeEntries = [this, &ToolViewRef](FToolMenuSection& InSection,
		TMemFunPtrType<false, FNavigationToolView, void(ENavigationToolItemViewMode)>::Type InExecuteFunction,
		TMemFunPtrType<true, FNavigationToolView, ECheckBoxState(ENavigationToolItemViewMode)>::Type InGetActionCheckState)
		{
			const UEnum* const ViewModeEnum = StaticEnum<ENavigationToolItemViewMode>();
			check(ViewModeEnum);
			
			for (const ENavigationToolItemViewMode ViewModeFlags : MakeFlagsRange(ENavigationToolItemViewMode::All))
			{
				const int32 EnumIndex = ViewModeEnum->GetIndexByValue(static_cast<int64>(ViewModeFlags));

				InSection.AddMenuEntry(NAME_None,
					ViewModeEnum->GetDisplayNameTextByIndex(EnumIndex),
					ViewModeEnum->GetToolTipTextByIndex(EnumIndex),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateSP(ToolViewRef, InExecuteFunction, ViewModeFlags),
						FCanExecuteAction(),
						FGetActionCheckState::CreateSP(ToolViewRef, InGetActionCheckState, ViewModeFlags)
					),
					EUserInterfaceActionType::ToggleButton);
			}
		};

	FToolMenuSection& ItemDefaultViewModeSection = InMenu->FindOrAddSection(TEXT("ItemDefaultViewMode")
		, LOCTEXT("ItemDefaultViewMode", "Default Item View Mode"));

	AddViewModeEntries(ItemDefaultViewModeSection
		, &FNavigationToolView::ToggleItemDefaultViewModeSupport
		, &FNavigationToolView::GetItemDefaultViewModeCheckState);

	if (ItemProxyTypeNames.Num() > 0)
	{
		FToolMenuSection& ItemProxyViewModeSection = InMenu->FindOrAddSection(TEXT("ItemProxyViewMode")
			, LOCTEXT("ItemProxyViewMode", "Proxy Item View Mode"));

		AddViewModeEntries(ItemProxyViewModeSection
			, &FNavigationToolView::ToggleItemProxyViewModeSupport
			, &FNavigationToolView::GetItemProxyViewModeCheckState);
	}

	// Toggle global filters
	FToolMenuSection& GlobalFiltersSection = InMenu->FindOrAddSection(TEXT("GlobalFilters")
		, LOCTEXT("GlobalFiltersHeading", "Global Filters"));

	const TSharedRef<FNavigationTool> ToolRef = Tool.ToSharedRef();

	for (const TSharedPtr<FNavigationToolBuiltInFilter>& Filter : Tool->GlobalFilters)
	{
		const TSharedRef<FNavigationToolBuiltInFilter> FilterRef = Filter.ToSharedRef();
		const FText DisplayName = FilterRef->GetDisplayName();

		FToolMenuEntry& NewMenuEntry = GlobalFiltersSection.AddMenuEntry(*DisplayName.ToString(),
			DisplayName,
			FilterRef->GetToolTipText(),
			FilterRef->GetIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &FNavigationToolToolbarMenu::OnToggleGlobalFilter, FilterRef, ToolRef),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &FNavigationToolToolbarMenu::IsGlobalFilterActive, FilterRef)
			),
			EUserInterfaceActionType::ToggleButton);

		if (const TSharedPtr<FUICommandInfo> ToggleCommand = FilterRef->GetToggleCommand())
		{
			NewMenuEntry.InputBindingLabel = ToggleCommand->GetInputText();
		}
	}
}

void FNavigationToolToolbarMenu::CreateColumnViewOptionsMenu(UToolMenu* const InMenu)
{
	if (!InMenu)
	{
		return;
	}

	UNavigationToolViewMenuContext* const ToolBarContext = InMenu->FindContext<UNavigationToolViewMenuContext>();
	if (!ToolBarContext)
	{
		return;
	}

	const TSharedPtr<FNavigationToolView> ToolView =  ToolBarContext->GetToolView();
	if (!ToolView.IsValid())
	{
		return;
	}

	const TSharedRef<FNavigationToolView> ToolViewRef = ToolView.ToSharedRef();

	UNavigationToolSettings* const ToolSettings = GetMutableDefault<UNavigationToolSettings>();
	if (!ToolSettings)
	{
		return;
	}

	const FNavigationToolCommands& ToolCommands = FNavigationToolCommands::Get();
	const TSharedPtr<FUICommandList> ViewCommandList = ToolViewRef->GetViewCommandList();

	FToolMenuSection& ColumnViewsSection = InMenu->FindOrAddSection(TEXT("ColumnViews"), LOCTEXT("ColumnViewsHeading", "Column Views"));

	ColumnViewsSection.AddMenuEntryWithCommandList(ToolCommands.ResetVisibleColumnSizes, ViewCommandList);
	ColumnViewsSection.AddMenuEntryWithCommandList(ToolCommands.SaveCurrentColumnView, ViewCommandList);

	TSet<FNavigationToolColumnView>& CustomColumnViews = ToolSettings->GetCustomColumnViews();
	CustomColumnViews.Sort([](const FNavigationToolColumnView& InA, const FNavigationToolColumnView& InB)
		{
			return InA.ViewName.CompareTo(InB.ViewName) > 0;
		});

	if (!CustomColumnViews.IsEmpty())
	{
		ColumnViewsSection.AddSeparator(NAME_None);

		for (const FNavigationToolColumnView& ColumnView : CustomColumnViews)
		{
			const TSharedRef<SWidget> MenuItemWidget = FNavigationToolFilterMenu::ConstructCustomMenuItemWidget(ColumnView.ViewName,
				FSimpleDelegate::CreateSP(ToolViewRef, &FNavigationToolView::ApplyCustomColumnView, ColumnView.ViewName),
				TAttribute<ECheckBoxState>::CreateSP(this, &FNavigationToolToolbarMenu::GetCustomColumnViewMenuItemCheckState, ColumnView.ViewName, ToolViewRef),
				FAppStyle::GetBrush(TEXT("Icons.Delete")),
				FSimpleDelegate::CreateSP(this, &FNavigationToolToolbarMenu::OnDeleteCustomColumnViewMenuItemClick, ColumnView.ViewName),
				true);
			ColumnViewsSection.AddEntry(FToolMenuEntry::InitWidget(NAME_None, MenuItemWidget, FText()));
		}
	}
}

void FNavigationToolToolbarMenu::CreateFilterBarOptionsMenu(UToolMenu* const InMenu)
{
	if (!InMenu)
	{
		return;
	}

	const FNavigationToolFilterCommands& FilterCommands = FNavigationToolFilterCommands::Get();

	FToolMenuSection& FilterBarVisibilitySection = InMenu->FindOrAddSection(TEXT("FilterBarVisibility")
		, LOCTEXT("FilterBarVisibilityHeading", "Filter Bar"));

	FilterBarVisibilitySection.AddMenuEntry(FilterCommands.ToggleFilterBarVisibility);

	FToolMenuSection& FilterBarLayoutSection = InMenu->FindOrAddSection(TEXT("FilterBarLayout")
		, LOCTEXT("FilterBarLayoutHeading", "Filter Bar Layout"));

	FilterBarLayoutSection.AddMenuEntry(FilterCommands.SetToVerticalLayout);
	FilterBarLayoutSection.AddMenuEntry(FilterCommands.SetToHorizontalLayout);
}

ECheckBoxState FNavigationToolToolbarMenu::GetCustomColumnViewMenuItemCheckState(const FText InColumnViewName, const TSharedRef<FNavigationToolView> InToolView) const
{
	UNavigationToolSettings* const ToolSettings = GetMutableDefault<UNavigationToolSettings>();
	if (!ToolSettings)
	{
		return ECheckBoxState::Unchecked;
	}

	FNavigationToolColumnView* const SavedColumnView = ToolSettings->FindCustomColumnView(InColumnViewName);
	if (!SavedColumnView)
	{
		return ECheckBoxState::Unchecked;
	}

	for (const TPair<FName, TSharedPtr<INavigationToolColumn>>& Column : InToolView->GetColumns())
	{
		const FName ColumnId = Column.Value->GetColumnId();
		const bool bColumnVisible = SavedColumnView->VisibleColumns.Contains(ColumnId);
		if (InToolView->IsColumnVisible(Column.Value) != bColumnVisible)
		{
			return ECheckBoxState::Unchecked;
		}
	}

	return ECheckBoxState::Checked;
}

void FNavigationToolToolbarMenu::OnDeleteCustomColumnViewMenuItemClick(const FText InColumnViewName)
{
	UNavigationToolSettings* const ToolSettings = GetMutableDefault<UNavigationToolSettings>();
	if (!ToolSettings)
	{
		return;
	}

	TSet<FNavigationToolColumnView>& CustomColumnViews = ToolSettings->GetCustomColumnViews();

	CustomColumnViews.Remove(InColumnViewName);

	ToolSettings->SaveConfig();

	// We aren't manually removing the menu item while keeping the menu open, so we need to force rebuild by dismissing
	FSlateApplication::Get().DismissAllMenus();
}

bool FNavigationToolToolbarMenu::IsGlobalFilterActive(const TSharedRef<FNavigationToolBuiltInFilter> InFilter) const
{
	return InFilter->IsActive();
}

void FNavigationToolToolbarMenu::OnToggleGlobalFilter(const TSharedRef<FNavigationToolBuiltInFilter> InFilter, const TSharedRef<FNavigationTool> InTool)
{
	const bool bNewActiveState = !InFilter->IsActive();

	InFilter->SetActive(bNewActiveState);

	if (UNavigationToolSettings* const ToolSettings = GetMutableDefault<UNavigationToolSettings>())
	{
		ToolSettings->SetBuiltInFilterEnabled(InFilter->GetFilterParams().GetFilterId(), bNewActiveState);
	}

	InTool->Refresh();
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
