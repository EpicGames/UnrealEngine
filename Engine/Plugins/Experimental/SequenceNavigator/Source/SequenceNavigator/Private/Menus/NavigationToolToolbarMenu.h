// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class FName;
class SWidget;
class UToolMenu;
enum class ECheckBoxState : uint8;

namespace UE::SequenceNavigator
{

class FNavigationTool;
class FNavigationToolBuiltInFilter;
class FNavigationToolView;

class FNavigationToolToolbarMenu : public TSharedFromThis<FNavigationToolToolbarMenu>
{
public:
	static FName GetMenuName();

	TSharedRef<SWidget> CreateToolbar(const TSharedRef<FNavigationToolView>& InToolView);

protected:
	void PopulateToolBar(UToolMenu* const InMenu);

	void CreateSettingsMenu(UToolMenu* const InMenu);
	void CreateItemViewOptionsMenu(UToolMenu* const InMenu);
	void CreateColumnViewOptionsMenu(UToolMenu* const InMenu);
	void CreateFilterBarOptionsMenu(UToolMenu* const InMenu);

	ECheckBoxState GetCustomColumnViewMenuItemCheckState(const FText InColumnViewName, const TSharedRef<FNavigationToolView> InToolView) const;
	void OnDeleteCustomColumnViewMenuItemClick(const FText InColumnViewName);

	bool IsGlobalFilterActive(const TSharedRef<FNavigationToolBuiltInFilter> InFilter) const;
	void OnToggleGlobalFilter(const TSharedRef<FNavigationToolBuiltInFilter> InFilter, const TSharedRef<FNavigationTool> InTool);
};

} // namespace UE::SequenceNavigator
