// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NavigationTool.h"
#include "NavigationToolView.h"
#include "Templates/SharedPointer.h"
#include "UObject/Object.h"
#include "ToolMenu.h"
#include "NavigationToolViewMenuContext.generated.h"

DECLARE_DELEGATE_OneParam(FOnPopulateNavigationToolViewToolbarMenu, UToolMenu*);

UCLASS(Transient)
class UNavigationToolViewMenuContext : public UObject
{
	GENERATED_BODY()

	friend UE::SequenceNavigator::FNavigationToolView;

public:
	void Init(const TWeakPtr<UE::SequenceNavigator::FNavigationToolView>& InToolView)
	{
		WeakToolView = InToolView;
	}

	TSharedPtr<UE::SequenceNavigator::FNavigationToolView> GetToolView() const
	{
		return WeakToolView.Pin();
	}

	TSharedPtr<UE::SequenceNavigator::FNavigationTool> GetTool() const
	{
		if (const TSharedPtr<UE::SequenceNavigator::FNavigationToolView> ToolView = GetToolView())
		{
			return StaticCastSharedPtr<UE::SequenceNavigator::FNavigationTool>(ToolView->GetOwnerTool());
		}
		return nullptr;
	}

	FOnPopulateNavigationToolViewToolbarMenu OnPopulateMenu;

private:
	TWeakPtr<UE::SequenceNavigator::FNavigationToolView> WeakToolView;
};
