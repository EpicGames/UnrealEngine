// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavigationTool.h"
#include "Algo/Reverse.h"
#include "BlueprintEditorSettings.h"
#include "Components/ActorComponent.h"
#include "Editor.h"
#include "Editor/Transactor.h"
#include "Filters/Filters/NavigationToolBuiltInFilter.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GameFramework/Actor.h"
#include "ISequencer.h"
#include "ItemActions/INavigationToolItemAction.h"
#include "ItemProxies/NavigationToolItemProxyRegistry.h"
#include "Items/NavigationToolBinding.h"
#include "Items/NavigationToolItemUtils.h"
#include "Items/NavigationToolTreeRoot.h"
#include "Kismet2/ComponentEditorUtils.h"
#include "Misc/Optional.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/ViewModels/ObjectBindingModel.h"
#include "MVVM/ViewModels/TrackModel.h"
#include "MVVM/ViewModels/TrackRowModel.h"
#include "NavigationToolCommands.h"
#include "NavigationToolExtender.h"
#include "NavigationToolScopedSelection.h"
#include "NavigationToolSettings.h"
#include "NavigationToolTab.h"
#include "NavigationToolView.h"
#include "Providers/NavigationToolProvider.h"
#include "ScopedTransaction.h"
#include "Sections/MovieSceneSubSection.h"
#include "Utils/NavigationToolMiscUtils.h"
#include "Widgets/Views/STableRow.h"

#define LOCTEXT_NAMESPACE "NavigationTool"

namespace UE::SequenceNavigator
{

using namespace Sequencer;

FNavigationTool::FNavigationTool(const TWeakPtr<ISequencer>& InWeakSequencer)
	: WeakSequencer(InWeakSequencer)
	, ToolTab(MakeShared<FNavigationToolTab>(*this))
	, RootItem(MakeShared<FNavigationToolTreeRoot>(*this))
	, BaseCommandList(MakeShared<FUICommandList>())
{
}

void FNavigationTool::Init()
{
	// Register the default view. @TODO: This could/should probably be extendable and setup by the providers?
	constexpr int32 DefaultViewId = 0;
	const TSharedPtr<INavigationToolView> ToolView = RegisterToolView(DefaultViewId);
	check(ToolView.IsValid());

	ToolTab->OnVisibilityChanged().AddSP(this, &FNavigationTool::OnTabVisibilityChanged);

	FNavigationToolExtender::OnProvidersChanged().AddSP(this, &FNavigationTool::OnProvidersChanged);

	if (const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
	{
		const TSharedPtr<FUICommandList> SequencerCommandBindings = Sequencer->GetCommandBindings();
		const FNavigationToolCommands& NavigationToolCommands = FNavigationToolCommands::Get();

		SequencerCommandBindings->MapAction(
			NavigationToolCommands.ToggleToolTabVisible,
			FExecuteAction::CreateSP(this, &FNavigationTool::ToggleToolTabVisible),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &FNavigationTool::IsToolTabVisible));
	}

	RefreshColumnView();

	RefreshGlobalFilters();

	ToolTab->Init();

	OnTreeViewChanged();
}

void FNavigationTool::Shutdown()
{
	UnbindEvents();

	ToolTab->OnVisibilityChanged().RemoveAll(this);
	ToolTab->Shutdown();

	FNavigationToolExtender::OnProvidersChanged().RemoveAll(this);

	if (const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
	{
		Sequencer->OnActivateSequence().RemoveAll(this);
		Sequencer->OnCloseEvent().RemoveAll(this);
	}
}

void FNavigationTool::ForEachProvider(const TFunction<bool(const TSharedRef<FNavigationToolProvider>& InToolProvider)>& InPredicate) const
{
	const TSharedPtr<ISequencer> Sequencer = GetSequencer();
	if (!Sequencer.IsValid())
	{
		return;
	}

	UMovieSceneSequence* const RootSequence = Sequencer->GetRootMovieSceneSequence();
	if (!RootSequence)
	{
		return;
	}

	const FName ToolId = FNavigationToolExtender::GetToolInstanceId(*this);

	FNavigationToolExtender::ForEachToolProvider(ToolId, [&InPredicate, RootSequence]
		(const TSharedRef<FNavigationToolProvider>& InProvider)
		{
			if (InProvider->IsSequenceSupported(RootSequence))
			{
				return InPredicate(InProvider);
			}
			return true; // Continue processing providers
		});
}

USequencerSettings* FNavigationTool::GetSequencerSettings() const
{
	if (const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
	{
		return Sequencer->GetSequencerSettings();
	}
	return nullptr;
}

bool FNavigationTool::CanProcessSequenceSpawn(UMovieSceneSequence* const InSequence) const
{
	return true;
}

TSharedPtr<FUICommandList> FNavigationTool::GetBaseCommandList() const
{
	return BaseCommandList;
}

TArray<FName> FNavigationTool::GetRegisteredItemProxyTypeNames() const
{
	TArray<FName> OutItemProxyTypeNames;
	{
		// Get Tool-registered Item Types first
		TSet<FName> NameSet;
		ItemProxyRegistry.GetRegisteredItemProxyTypeNames(NameSet);

		// Get the Module-registered Item Types second
		TSet<FName> ModuleNameSet;
		FNavigationToolExtender::GetItemProxyRegistry().GetRegisteredItemProxyTypeNames(ModuleNameSet);
		NameSet.Append(ModuleNameSet);

		OutItemProxyTypeNames = NameSet.Array();
	}

	OutItemProxyTypeNames.Sort(FNameLexicalLess());

	return OutItemProxyTypeNames;
}

void FNavigationTool::GetItemProxiesForItem(const FNavigationToolViewModelPtr& InItem, TArray<TSharedPtr<FNavigationToolItemProxy>>& OutItemProxies)
{
	// No Item Proxy support for Root
	if (!InItem.IsValid() || InItem.ToSharedRef() == RootItem.ToSharedRef())
	{
		return;
	}

	InItem->GetItemProxies(OutItemProxies);

	ForEachProvider([this, &InItem, &OutItemProxies]
		(const TSharedRef<FNavigationToolProvider>& InProvider)
		{
			InProvider->OnExtendItemProxiesForItem(*this, InItem, OutItemProxies);
			return true;
		});

	// Clean up any invalid item proxy
	OutItemProxies.RemoveAll([](const TSharedPtr<FNavigationToolItemProxy>& InItemProxy)
		{
			return !InItemProxy.IsValid();
		});

	// Sort proxies by their priority
	OutItemProxies.Sort([](const TSharedPtr<FNavigationToolItemProxy>& InItemProxyA, const TSharedPtr<FNavigationToolItemProxy>& InItemProxyB)
		{
			return InItemProxyA->GetPriority() > InItemProxyB->GetPriority();
		});
}

INavigationToolItemProxyFactory* FNavigationTool::GetItemProxyFactory(const FName InItemProxyTypeName) const
{
	// First look for the registry in Navigation Tool
	if (INavigationToolItemProxyFactory* const Factory = ItemProxyRegistry.GetItemProxyFactory(InItemProxyTypeName))
	{
		return Factory;
	}

	// Fallback to finding the factory in the module if the Navigation Tool did not find it
	return FNavigationToolExtender::GetItemProxyRegistry().GetItemProxyFactory(InItemProxyTypeName);
}

bool FNavigationTool::IsToolLocked() const
{
	bool bOutShouldLock = false;

	ForEachProvider([&bOutShouldLock]
		(const TSharedRef<FNavigationToolProvider>& InToolProvider)
		{
			if (InToolProvider->ShouldLockTool())
			{
				bOutShouldLock = true;
				return false; // No need to continue processing tool providers
			}
			return true;
		});

	return bOutShouldLock;
}

void FNavigationTool::HandleUndoRedoTransaction(const FTransaction* const InTransaction, const bool bInIsUndo)
{
	RequestRefresh();
}

TSharedPtr<ISequencer> FNavigationTool::GetSequencer() const
{
	return WeakSequencer.Pin();
}

void FNavigationTool::RefreshColumnView()
{
	const UNavigationToolSettings* const ToolSettings = GetDefault<UNavigationToolSettings>();

	// We apply default views *after* all columns have been created for all providers
	if (ToolSettings && ToolSettings->ShouldApplyDefaultColumnView())
	{
		TArray<FText> DefaultColumnViews;

		ForEachProvider([&DefaultColumnViews](const TSharedRef<FNavigationToolProvider>& InProvider)
			{
				const FText DefaultColumnView = InProvider->GetDefaultColumnView();
				if (!DefaultColumnView.IsEmptyOrWhitespace())
				{
					DefaultColumnViews.Add(DefaultColumnView);
				}
				return true;
			});

		// @TODO: priority int to better find a default priority view instead of just using the first index?
		if (!DefaultColumnViews.IsEmpty())
		{
			ForEachToolView([&DefaultColumnViews](const TSharedPtr<FNavigationToolView>& InToolView)
				{
					InToolView->ApplyCustomColumnView(DefaultColumnViews[0]);
				});
		}
	}
}

bool FNavigationTool::DoesGlobalFilterExist(const FName InFilterId)
{
	for (const TSharedPtr<FNavigationToolBuiltInFilter>& GlobalFilter : GlobalFilters)
	{
		if (GlobalFilter->GetName().Equals(InFilterId.ToString()))
		{
			return true;
		}
	}
	return false;
}

void FNavigationTool::RefreshGlobalFilters()
{
	const UNavigationToolSettings* const ToolSettings = GetDefault<UNavigationToolSettings>();
	if (!ToolSettings)
	{
		return;
	}

	TArray<FNavigationToolBuiltInFilterParams> NewGlobalFilterParams;

	ForEachProvider([&NewGlobalFilterParams](const TSharedRef<FNavigationToolProvider>& InProvider)
		{
			InProvider->OnExtendBuiltInFilters(NewGlobalFilterParams);
			return true;
		});

	GlobalFilters.Reset(NewGlobalFilterParams.Num());

	for (const FNavigationToolBuiltInFilterParams& NewFilterParams : NewGlobalFilterParams)
	{
		if (!DoesGlobalFilterExist(NewFilterParams.GetFilterId()))
		{
			const TSharedPtr<FNavigationToolBuiltInFilter> NewFilter = MakeShared<FNavigationToolBuiltInFilter>(NewFilterParams);

			const TSet<FName>& EnabledBuiltInFilters = ToolSettings->GetEnabledBuiltInFilters();
			if (!NewFilterParams.IsEnabledByDefault()
				&& !EnabledBuiltInFilters.Contains(NewFilterParams.GetFilterId()))
			{
				NewFilter->SetActive(false);
			}

			GlobalFilters.Add(NewFilter);
		}
	}
}

TSharedPtr<INavigationToolView> FNavigationTool::RegisterToolView(const int32 InToolViewId)
{
	const TSharedPtr<FNavigationToolView> ToolView = FNavigationToolView::CreateInstance(InToolViewId
		, SharedThis(this)
		, GetBaseCommandList());
	ToolViews.Add(InToolViewId, ToolView);

	ForEachProvider([this, &ToolView](const TSharedRef<FNavigationToolProvider>& InProvider)
		{
			ToolView->LoadViewState(InProvider);

			ToolView->CreateColumns(InProvider);
			ToolView->CreateDefaultColumnViews(InProvider);

			return true;
		});

	return ToolView;
}

TSharedPtr<INavigationToolView> FNavigationTool::GetToolView(const int32 InToolViewId) const
{
	if (const TSharedPtr<FNavigationToolView>* const FoundToolView = ToolViews.Find(InToolViewId))
	{
		return *FoundToolView;
	}
	return nullptr;
}

bool FNavigationTool::IsObjectAllowedInTool(const UObject* const InObject) const
{
	if (const AActor* const Actor = Cast<AActor>(InObject))
	{
		// Do not show Actors that aren't editable or not meant to be listed in Navigation Tool
		if (!Actor->IsEditable())
		{
			return false;
		}

		return true;
	}

	if (const UActorComponent* const ActorComponent = Cast<UActorComponent>(InObject))
	{
		const bool bHideConstructionScriptComponents = GetDefault<UBlueprintEditorSettings>()->bHideConstructionScriptComponentsInDetailsView;
		return !ActorComponent->IsVisualizationComponent()
			&& (ActorComponent->CreationMethod != EComponentCreationMethod::UserConstructionScript || !bHideConstructionScriptComponents)
			&& (ActorComponent->CreationMethod != EComponentCreationMethod::Native ||
				FComponentEditorUtils::GetPropertyForEditableNativeComponent(ActorComponent));
	}

	return false;
}

void FNavigationTool::RegisterItem(const FNavigationToolViewModelPtr& InItem)
{
	if (!InItem.IsValid())
	{
		return;
	}

	const FNavigationToolItemId ItemId = InItem->GetItemId();
	const FNavigationToolViewModelPtr ExistingItem = FindItem(ItemId);

	// if there's no existing item or the existing item does not match its new replacement,
	// then call OnItemRegistered and Refresh Navigation Tool
	if (!ExistingItem.IsValid() || ExistingItem != InItem)
	{
		InItem->OnItemRegistered();

		AddItem(InItem);

		RequestRefresh();
	}
}

void FNavigationTool::UnregisterItem(const FNavigationToolItemId& InItemId)
{
	const FNavigationToolViewModelPtr FoundItem = FindItem(InItemId);
	if (!FoundItem.IsValid())
	{
		return;
	}

	FoundItem->OnItemUnregistered();

	RemoveItem(InItemId);

	RequestRefresh();
}

void FNavigationTool::RequestRefresh()
{
	bRefreshRequested = true;
}

void FNavigationTool::Refresh()
{
	TGuardValue RefreshGuard(bRefreshing, true);
	bRefreshRequested = false;

	// Flush pending actions
	{
		// Make a transaction if there's a pending action requesting it
		TUniquePtr<FScopedTransaction> Transaction = nullptr;

		if (!GIsTransacting)
		{
			const bool bShouldTransact = PendingActions.ContainsByPredicate(
				[](const TSharedPtr<INavigationToolItemAction>& InAction)
				{
					return InAction.IsValid() && InAction->ShouldTransact();
				});

			if (bShouldTransact)
			{
				Transaction = MakeUnique<FScopedTransaction>(LOCTEXT("ItemAction", "Item Action"));
			}
		}

		// Execute pending actions
		for (const TSharedPtr<INavigationToolItemAction>& Action : PendingActions)
		{
			if (Action.IsValid())
			{
				Action->Execute(*this);
			}
		}

		PendingActions.Empty();
	}

	// Save the current item ordering before refreshing children.
	// Do not reset the tree just yet as there might be actors that still need to be discovered on the next pass.
	// This is done to save the items added from the queued actions and be considered when adding new items from discovery.
	ForEachProvider([this](const TSharedRef<FNavigationToolProvider>& InProvider)
		{
			InProvider->SaveSerializedTree(*this, /*bInResetTree*/false);
			return true;
		});

	// Refresh each item's children. This also updates each child's parent var.
	ForEachItem([](const FNavigationToolViewModelPtr& InItem)
		{
			InItem->RefreshChildren();
		});

	RootItem->RefreshChildren();

	ForEachToolView([](const TSharedPtr<FNavigationToolView>& InToolView)
		{
			InToolView->Refresh();
		});

	// Save so that the tree is updated to the latest Navigation Tool state
	ForEachProvider([this](const TSharedRef<FNavigationToolProvider>& InProvider)
		{
			InProvider->SaveSerializedTree(*this, /*bInResetTree*/false);
			return true;
		});
}

FNavigationToolViewModelWeakPtr FNavigationTool::GetTreeRoot() const
{
	return RootItem;
}

FNavigationToolViewModelPtr FNavigationTool::FindItem(const FNavigationToolItemId& InItemId) const
{
	if (bIteratingItemMap)
	{
		if (const FNavigationToolViewModelPtr* const FoundItem = ItemsPendingAdd.Find(InItemId))
		{
			return *FoundItem;
		}
	}

	if (const FNavigationToolViewModelPtr* const FoundItem = ItemMap.Find(InItemId))
	{
		return *FoundItem;
	}

	return nullptr;
}

TArray<FNavigationToolViewModelPtr> FNavigationTool::TryFindItems(const FViewModelPtr& InSequencerOutlinerViewModel) const
{
	if (const TViewModelPtr<ITrackExtension> TrackExtension = InSequencerOutlinerViewModel.ImplicitCast())
	{
		return FindItemsFromMovieSceneObject(TrackExtension->GetTrack());
	}

	if (const TViewModelPtr<ITrackRowExtension> TrackRowExtension = InSequencerOutlinerViewModel.ImplicitCast())
	{
		return FindItemsFromMovieSceneObject(TrackRowExtension->GetParentTrack());
	}

	if (const TViewModelPtr<FObjectBindingModel> ObjectBindingModel = InSequencerOutlinerViewModel.ImplicitCast())
	{
		return FindItemsFromObjectGuid(ObjectBindingModel->GetObjectGuid());
	}

	if (const TViewModelPtr<FSectionModel> SectionModel = InSequencerOutlinerViewModel.ImplicitCast())
	{
		if (UMovieSceneSubSection* const SubSection = Cast<UMovieSceneSubSection>(SectionModel->GetSection()))
		{
			return FindItemsFromMovieSceneObject(SubSection->GetSequence());
		}
	}

	return {};
}

void FNavigationTool::SetIgnoreNotify(const ENavigationToolIgnoreNotifyFlags InFlag, const bool bInIgnore)
{
	if (bInIgnore)
	{
		EnumAddFlags(IgnoreNotifyFlags, InFlag);
	}
	else
	{
		EnumRemoveFlags(IgnoreNotifyFlags, InFlag);
	}
}

void FNavigationTool::PostUndo(const bool bInSuccess)
{
	if (GEditor && bInSuccess)
	{
		const int32 QueueIndex = GEditor->Trans->GetQueueLength() - GEditor->Trans->GetUndoCount();
		const FTransaction* Transaction = GEditor->Trans->GetTransaction(QueueIndex);
		HandleUndoRedoTransaction(Transaction, true);
	}
}

void FNavigationTool::PostRedo(const bool bInSuccess)
{
	if (GEditor && bInSuccess)
	{
		const int32 QueueIndex = GEditor->Trans->GetQueueLength() - GEditor->Trans->GetUndoCount();
		const FTransaction* Transaction = GEditor->Trans->GetTransaction(QueueIndex);
		HandleUndoRedoTransaction(Transaction, false);
	}
}

TStatId FNavigationTool::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FNavigationTool, STATGROUP_Tickables);
}

void FNavigationTool::Tick(const float InDeltaTime)
{
	if (NeedsRefresh())
	{
		Refresh();
	}

	if (bToolDirty)
	{
		bToolDirty = false;
	}

	// Select Items Pending Selection
	if (ItemsLastSelected.IsValid())
	{
		TArray<FNavigationToolViewModelWeakPtr> WeakItemsToSelect;

		for (const FNavigationToolViewModelWeakPtr& WeakItem : *ItemsLastSelected)
		{
			if (const FNavigationToolViewModelPtr Item = WeakItem.Pin())
			{
				WeakItemsToSelect.Add(Item);
			}
		}

		// Only Scroll Into View, don't signal selection since we just came from the selection notify itself
		SelectItems(WeakItemsToSelect, ENavigationToolItemSelectionFlags::ScrollIntoView);

		ItemsLastSelected.Reset();
	}

	ForEachToolView([InDeltaTime](const TSharedPtr<FNavigationToolView>& InToolView)
		{
			InToolView->Tick(InDeltaTime);
		});
}

void FNavigationTool::DeleteItems(const TArray<FNavigationToolViewModelWeakPtr>& InWeakItems)
{
	//SortItems(InWeakItems);

	TArray<FNavigationToolViewModelPtr> ItemsToRemove;

	for (const FNavigationToolViewModelWeakPtr& WeakItem : InWeakItems)
	{
		const FNavigationToolViewModelPtr Item = WeakItem.Pin();
		if (Item.IsValid() || Item->CanDelete())
		{
			ItemsToRemove.Add(Item);
		}
	}

	if (ItemsToRemove.IsEmpty())
	{
		return;
	}

	const FScopedTransaction DeleteTransaction(LOCTEXT("ItemDeleteAction", "Delete Item(s)"), !GIsTransacting);

	bool bRequestRefresh = false;

	for (const FNavigationToolViewModelPtr& Item : ItemsToRemove)
	{
		if (Item.IsValid() && Item->Delete())
		{
			bRequestRefresh = true;
		}
	}

	if (bRequestRefresh)
	{
		RequestRefresh();
	}
}

void FNavigationTool::UnregisterToolView(const int32 InToolViewId)
{
	ToolViews.Remove(InToolViewId);
}

void FNavigationTool::UpdateRecentToolViews(const int32 InToolViewId)
{
	RecentToolViews.Remove(InToolViewId);
	RecentToolViews.Add(InToolViewId);
}

TSharedPtr<INavigationToolView> FNavigationTool::GetMostRecentToolView() const
{
	for (int32 Index = RecentToolViews.Num() - 1; Index >= 0; --Index)
	{
		if (const TSharedPtr<INavigationToolView> ToolView = GetToolView(RecentToolViews[Index]))
		{
			return StaticCastSharedPtr<FNavigationToolView>(ToolView);
		}
	}
	return nullptr;
}

void FNavigationTool::ForEachToolView(const TFunction<void(const TSharedRef<FNavigationToolView>& InToolView)>& InPredicate) const
{
	FNavigationTool* const MutableThis = const_cast<FNavigationTool*>(this);
	for (TMap<int32, TSharedPtr<FNavigationToolView>>::TIterator Iter(MutableThis->ToolViews); Iter; ++Iter)
	{
		const TSharedPtr<FNavigationToolView>& ToolView = Iter.Value();
		if (ToolView.IsValid())
		{
			InPredicate(ToolView.ToSharedRef());
		}
		else
		{
			Iter.RemoveCurrent();
		}
	}
}

void FNavigationTool::EnqueueItemActions(TArray<TSharedPtr<INavigationToolItemAction>>&& InItemActions) noexcept
{
	PendingActions.Append(MoveTemp(InItemActions));
}

int32 FNavigationTool::GetPendingItemActionCount() const
{
	return PendingActions.Num();
}

bool FNavigationTool::NeedsRefresh() const
{
	if (bRefreshing)
	{
		return false;
	}

	if (bRefreshRequested || PendingActions.Num() > 0)
	{
		return true;
	}

	return false;
}

TOptional<FColor> FNavigationTool::FindItemColor(const FNavigationToolViewModelPtr& InItem, bool bRecurseParent) const
{
	if (!InItem.IsValid())
	{
		return TOptional<FColor>();
	}

	const TSharedPtr<INavigationToolProvider> Provider = InItem->GetProvider();
	if (!Provider.IsValid())
	{
		return TOptional<FColor>();
	}

	FNavigationToolSaveState* const SaveState = Provider->GetSaveState(*this);
	if (!SaveState)
	{
		return TOptional<FColor>();
	}

	if (const FColor* const FoundColor = SaveState->ItemColorMap.Find(InItem->GetItemId().GetStringId()))
	{
		return *FoundColor;
	}

	//If no Item Coloring Mapping was found for the Specific Item, then try find the Item Color of the Parent
	if (bRecurseParent)
	{
		return FindItemColor(InItem->GetParent(), bRecurseParent);
	}

	return TOptional<FColor>();
}

void FNavigationTool::SetItemColor(const FNavigationToolViewModelPtr& InItem, const FColor& InColor)
{
	if (!InItem.IsValid())
	{
		return;
	}

	const TSharedPtr<INavigationToolProvider> Provider = InItem->GetProvider();
	if (!Provider.IsValid())
	{
		return;
	}

	FNavigationToolSaveState* const SaveState = Provider->GetSaveState(*this);
	if (!SaveState)
	{
		return;
	}

	bool bShouldChangeItemColor = true;

	const TOptional<FColor> InheritedColor = FindItemColor(InItem, true);
	const bool bHasInheritedColor = InheritedColor.IsSet();

	if (bHasInheritedColor)
	{
		//Make sure the Inherited Color is different than the Color we're trying to Set.
		bShouldChangeItemColor = InheritedColor != InColor;
	}

	if (bShouldChangeItemColor)
	{
		const TOptional<FColor> ParentInheritedColor = FindItemColor(InItem->GetParent(), true);

		//First check if the Color to Set matches the one inherited from the Parent
		if (ParentInheritedColor.IsSet() && ParentInheritedColor == InColor)
		{
			//If it matches, remove this Item from the map as we will inherit from parent
			SaveState->ItemColorMap.Remove(InItem->GetItemId().GetStringId());
		}
		else
		{
			SaveState->ItemColorMap.Add(InItem->GetItemId().GetStringId(), InColor);
		}

		// Recurse children to remove their mapping if same color with parent
		TArray<FNavigationToolViewModelWeakPtr> WeakRemainingChildren = InItem->GetChildren();
		while (WeakRemainingChildren.Num() > 0)
		{
			if (const FNavigationToolViewModelPtr Child = WeakRemainingChildren.Pop().Pin())
			{
				const TOptional<FColor> ChildColor = FindItemColor(Child, false);
				if (ChildColor.IsSet() && ChildColor == InColor)
				{
					RemoveItemColor(Child);

					//Only check Grand Children if Child had same Color.
					//Allow the situation where Parent is ColorA, Child is ColorB and GrandChild is ColorA.
					WeakRemainingChildren.Append(Child->GetChildren());
				}
			}
		}
		SetToolModified();
	}
}

void FNavigationTool::RemoveItemColor(const FNavigationToolViewModelPtr& InItem)
{
	if (!InItem.IsValid())
	{
		return;
	}

	const TSharedPtr<INavigationToolProvider> Provider = InItem->GetProvider();
	if (!Provider.IsValid())
	{
		return;
	}

	FNavigationToolSaveState* const SaveState = Provider->GetSaveState(*this);
	if (!SaveState)
	{
		return;
	}

	const bool bRemoved = SaveState->ItemColorMap.Remove(InItem->GetItemId().GetStringId()) > 0;
	if (bRemoved)
	{
		SetToolModified();
	}
}

void FNavigationTool::NotifyItemIdChanged(const FNavigationToolItemId& OldId, const FNavigationToolViewModelPtr& InItem)
{
	const FNavigationToolItemId NewId = InItem->GetItemId();
	if (OldId == NewId)
	{
		return;
	}

	const FNavigationToolViewModelPtr FoundItem = FindItem(OldId);
	if (FoundItem.IsValid() && FoundItem == InItem)
	{
		AddItem(InItem);
		RemoveItem(OldId);
		SetToolModified();
	}
}

void FNavigationTool::NotifyToolItemRenamed(const FNavigationToolViewModelPtr& InItem)
{
	ForEachProvider([&InItem](const TSharedRef<FNavigationToolProvider>& InProvider)
		{
			InProvider->NotifyToolItemRenamed(InItem);
			return true;
		});
}

void FNavigationTool::NotifyToolItemDeleted(const FNavigationToolViewModelPtr& InItem)
{
	ForEachProvider([&InItem](const TSharedRef<INavigationToolProvider>& InProvider)
		{
			InProvider->NotifyToolItemDeleted(InItem);
			return true;
		});
}

TArray<FNavigationToolViewModelWeakPtr> FNavigationTool::GetSelectedItems(const bool bInNormalizeToTopLevelSelections) const
{
	if (const TSharedPtr<INavigationToolView>& ToolView = GetMostRecentToolView())
	{
		TArray<FNavigationToolViewModelWeakPtr> WeakSelectedItems = ToolView->GetSelectedItems();

		if (bInNormalizeToTopLevelSelections)
		{
			NormalizeToTopLevelSelections(WeakSelectedItems);
			return WeakSelectedItems;
		}

		return WeakSelectedItems;
	}

	return {};
}

void FNavigationTool::SelectItems(const TArray<FNavigationToolViewModelWeakPtr>& InWeakItems
	, const ENavigationToolItemSelectionFlags InFlags) const
{
	ForEachToolView([InWeakItems, InFlags](const TSharedPtr<INavigationToolView>& InToolView)
		{
			InToolView->SelectItems(InWeakItems, InFlags);
		});
}

void FNavigationTool::ClearItemSelection(const bool bInSignalSelectionChange) const
{
	ForEachToolView([bInSignalSelectionChange](const TSharedPtr<INavigationToolView>& InToolView)
		{
			InToolView->ClearItemSelection(bInSignalSelectionChange);
		});
}

FNavigationToolViewModelPtr FNavigationTool::FindLowestCommonAncestor(const TArray<FNavigationToolViewModelPtr>& Items)
{
	TSet<FNavigationToolViewModelPtr> IntersectedAncestors;

	for (const FNavigationToolViewModelPtr& Item : Items)
	{
		FNavigationToolViewModelPtr Parent = Item->GetParent();
		TSet<FNavigationToolViewModelPtr> ItemAncestors;

		//Add all Item's Ancestors
		while (Parent.IsValid())
		{
			ItemAncestors.Add(Parent);
			Parent = Parent->GetParent();
		}

		//cant check for intersection if empty so just init
		if (IntersectedAncestors.Num() == 0)
		{
			IntersectedAncestors = ItemAncestors;
		}
		else
		{
			IntersectedAncestors = IntersectedAncestors.Intersect(ItemAncestors);

			//We are sure the intersection is the Root if only one item is remaining. Stop iterating
			if (IntersectedAncestors.Num() == 1)
			{
				break;
			}
		}
	}

	FNavigationToolViewModelPtr LowestCommonAncestor;
	for (const FNavigationToolViewModelPtr& Item : IntersectedAncestors)
	{
		const TViewModelPtr<IOutlinerExtension> OutlinerExtension = Item.ImplicitCast();
		if (!OutlinerExtension.IsValid())
		{
			continue;
		}

		const TViewModelPtr<IOutlinerExtension> LowestOutlinerExtension = LowestCommonAncestor.ImplicitCast();

		// Find item with most tree height (i.e. lowest down the tree, closer to the selected nodes)
		if (!LowestCommonAncestor.IsValid()
			|| (LowestOutlinerExtension.IsValid()
				&& OutlinerExtension->GetOutlinerSizing().Height > LowestOutlinerExtension->GetOutlinerSizing().Height))
		{
			LowestCommonAncestor = Item;
		}
	}
	return LowestCommonAncestor;
}

void FNavigationTool::SortItems(TArray<FNavigationToolViewModelWeakPtr>& WeakItems
	, const bool bInReverseOrder)
{
	WeakItems.Sort([bInReverseOrder]
		(const FNavigationToolViewModelWeakPtr& InWeakItemA, const FNavigationToolViewModelWeakPtr& InWeakItemB)
		{
			return ItemUtils::CompareToolItemOrder(InWeakItemA.Pin(), InWeakItemB.Pin()) != bInReverseOrder;
		});
}

void FNavigationTool::NormalizeToTopLevelSelections(TArray<FNavigationToolViewModelWeakPtr>& InWeakItems)
{
	if (InWeakItems.IsEmpty())
	{
		return;
	}

	const TSet<FNavigationToolViewModelWeakPtr> SelectedItemSet(InWeakItems);

	// Normalize Selection: Remove all items that have parents that are in the selection.
	// Swapping since we're sorting afterwards.
	InWeakItems.RemoveAllSwap([&SelectedItemSet](const FNavigationToolViewModelWeakPtr& InWeakItem)
	{
		const FNavigationToolViewModelPtr Item = InWeakItem.Pin();
		if (!Item.IsValid())
		{
			return false;
		}
		FNavigationToolViewModelPtr Parent = Item->GetParent();
		while (Parent.IsValid())
		{
			if (SelectedItemSet.Contains(Parent))
			{
				return true;
			}
			Parent = Parent->GetParent();
		}
		return false;
	});
}

void FNavigationTool::SyncSequencerSelection(const TArray<FNavigationToolViewModelWeakPtr>& InWeakSelectedItems) const
{
	const TSharedPtr<ISequencer> Sequencer = GetSequencer();
	if (!Sequencer.IsValid())
	{
		return;
	}

	FNavigationToolScopedSelection ScopedSelection(*Sequencer, ENavigationToolScopedSelectionPurpose::Sync);
	for (const FNavigationToolViewModelWeakPtr& WeakItem : InWeakSelectedItems)
	{
		if (const FNavigationToolViewModelPtr Item = WeakItem.Pin())
		{
			Item->Select(ScopedSelection);
		}
	}
}

const FNavigationToolItemProxyRegistry& FNavigationTool::GetItemProxyRegistry() const
{
	return ItemProxyRegistry;
}

FNavigationToolItemProxyRegistry& FNavigationTool::GetItemProxyRegistry()
{
	return ItemProxyRegistry;
}

void FNavigationTool::OnObjectsReplaced(const TMap<UObject*, UObject*>& InReplacementMap)
{
	ForEachItem([&InReplacementMap](const FNavigationToolViewModelPtr& InItem)
		{
			// Recursive not needed since we're calling it on all items in map anyway
			InItem->OnObjectsReplaced(InReplacementMap, /*bRecursive*/false);
		});

	for (const TSharedPtr<INavigationToolItemAction>& Action : PendingActions)
	{
		// Recursive needed since we only have direct reference to the underlying item in the action, not its children
		if (Action.IsValid())
		{
			Action->OnObjectsReplaced(InReplacementMap, true);
		}
	}

	ForEachToolView([](const TSharedPtr<FNavigationToolView>& InToolView)
		{
			InToolView->NotifyObjectsReplaced();
		});

	RequestRefresh();
}

void FNavigationTool::OnActorReplaced(AActor* const InOldActor, AActor* const InNewActor)
{
	TMap<UObject*, UObject*> ReplacementMap;
	ReplacementMap.Add(InOldActor, InNewActor);
	OnObjectsReplaced(ReplacementMap);
}

void FNavigationTool::SetToolModified()
{
	if (!bToolDirty)
	{
		bToolDirty = true;
	}
}

bool FNavigationTool::IsToolTabVisible() const
{
	return ToolTab->IsToolTabVisible();
}

void FNavigationTool::ShowHideToolTab(const bool bInVisible)
{
	ToolTab->ShowHideToolTab(bInVisible);

	if (bInVisible)
	{
		BindEvents();

		RecordUsageAnalytics(TEXT("ToolOpened"), TEXT("true"));
	}
	else
	{
		UnbindEvents();
	}
}

void FNavigationTool::ToggleToolTabVisible()
{
	ShowHideToolTab(!IsToolTabVisible());
}

void FNavigationTool::OnTabVisibilityChanged(const bool bInVisible)
{
	if (bInVisible)
	{
		BindEvents();
	
		if (GEditor)
		{
			GEditor->RegisterForUndo(this);
		}
	}
	else
	{
		UnbindEvents();

		if (GEditor)
		{
			GEditor->UnregisterForUndo(this);
		}
	}
}

void FNavigationTool::BindEvents()
{
	if (const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
	{
		Sequencer->OnChannelChangedWithTime().AddSPLambda(this, [this](const TArray<FFrameNumber>&,const FMovieSceneChannelMetaData*, UMovieSceneSection*)
			{
				OnTreeViewChanged();
			});
		Sequencer->OnMovieSceneBindingsChanged().AddSP(this, &FNavigationTool::OnTreeViewChanged);
		Sequencer->OnMovieSceneBindingsPasted().AddSPLambda(this, [this](const TArray<FMovieSceneBinding>&)
			{
				OnTreeViewChanged();
			});
		Sequencer->OnMovieSceneDataChanged().AddSPLambda(this, [this](const EMovieSceneDataChangeType)
			{
				OnTreeViewChanged();
			});

		Sequencer->OnTreeViewChanged().AddSP(this, &FNavigationTool::OnTreeViewChanged);
		Sequencer->OnEndScrubbingEvent().AddSP(this, &FNavigationTool::OnTreeViewChanged);

		if (const TSharedPtr<FSequencerEditorViewModel> ViewModel = Sequencer->GetViewModel())
		{
			if (const TSharedPtr<FSequencerSelection> SequencerSelection = ViewModel->GetSelection())
			{
				SequencerSelection->Outliner.OnChanged.AddSP(this, &FNavigationTool::OnSequencerSelectionChanged);
				SequencerSelection->TrackArea.OnChanged.AddSP(this, &FNavigationTool::OnSequencerSelectionChanged);
			}
		}
	}

	// Listen to object replacement changes
	FCoreUObjectDelegates::OnObjectsReplaced.AddSP(this, &FNavigationTool::OnObjectsReplaced);
	FEditorDelegates::OnEditorActorReplaced.AddSP(this, &FNavigationTool::OnActorReplaced);
}

void FNavigationTool::UnbindEvents()
{
	if (const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
	{
		Sequencer->OnChannelChangedWithTime().RemoveAll(this);
		Sequencer->OnMovieSceneBindingsChanged().RemoveAll(this);
		Sequencer->OnMovieSceneBindingsPasted().RemoveAll(this);
		Sequencer->OnMovieSceneDataChanged().RemoveAll(this);

		Sequencer->OnTreeViewChanged().RemoveAll(this);
		Sequencer->OnEndScrubbingEvent().RemoveAll(this);

		if (const TSharedPtr<FSequencerEditorViewModel> ViewModel = Sequencer->GetViewModel())
		{
			if (const TSharedPtr<FSequencerSelection> SequencerSelection = ViewModel->GetSelection())
			{
				SequencerSelection->Outliner.OnChanged.RemoveAll(this);
				SequencerSelection->TrackArea.OnChanged.RemoveAll(this);
			}
		}
	}

	FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);
	FEditorDelegates::OnEditorActorReplaced.RemoveAll(this);
}

void FNavigationTool::AddItem(const FNavigationToolViewModelPtr& InItem)
{
	const FNavigationToolItemId ItemId = InItem->GetItemId();

	ItemsPendingRemove.Remove(ItemId);

	if (bIteratingItemMap)
	{
		ItemsPendingAdd.Add(ItemId, InItem);
	}
	else
	{
		ItemMap.Add(ItemId, InItem);
	}
}

void FNavigationTool::RemoveItem(const FNavigationToolItemId& InItemId)
{
	ItemsPendingAdd.Remove(InItemId);

	if (bIteratingItemMap)
	{
		ItemsPendingRemove.Add(InItemId);
	}
	else
	{
		ItemMap.Remove(InItemId);
	}
}

void FNavigationTool::ForEachItem(TFunctionRef<void(const FNavigationToolViewModelPtr&)> InFunc)
{
	// iteration scope, allowing for nested for-each
	{
		TGuardValue<bool> Guard(bIteratingItemMap, true);
		for (const TPair<FNavigationToolItemId, FNavigationToolViewModelPtr>& Pair : ItemMap)
		{
			InFunc(Pair.Value);
		}
	}

	if (!bIteratingItemMap && (!ItemsPendingAdd.IsEmpty() || !ItemsPendingRemove.IsEmpty()))
	{
		for (const TPair<FNavigationToolItemId, FNavigationToolViewModelPtr>& ItemToAdd : ItemsPendingAdd)
		{
			ItemMap.Add(ItemToAdd);
		}
		ItemsPendingAdd.Empty();

		for (const FNavigationToolItemId& ItemIdToRemove : ItemsPendingRemove)
		{
			ItemMap.Remove(ItemIdToRemove);
		}
		ItemsPendingRemove.Empty();
	}
}

void FNavigationTool::OnTreeViewChanged()
{
	if (const TSharedPtr<INavigationToolView> RecentToolView = GetMostRecentToolView())
	{
		RecentToolView->RequestRefresh();
	}

	RequestRefresh();
}

void FNavigationTool::OnSequencerSelectionChanged()
{
	UNavigationToolSettings* const ToolSettings = GetMutableDefault<UNavigationToolSettings>();
	if (!ToolSettings->ShouldSyncSelectionToNavigationTool())
	{
		return;
	}

	// If any view is syncing item selection, ignore item selection or it will cause another round of selections next tick
	if (AreAllViewsSyncingItemSelection())
	{
		return;
	}

	const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	const TSharedPtr<FSequencerEditorViewModel> ViewModel = Sequencer->GetViewModel();
	if (!ViewModel.IsValid())
	{
		return;
	}

	const TSharedPtr<FSequencerSelection> SequencerSelection = ViewModel->GetSelection();
	if (!SequencerSelection.IsValid())
	{
		return;
	}

	if (!ItemsLastSelected.IsValid())
	{
		ItemsLastSelected = MakeShared<TArray<FNavigationToolViewModelWeakPtr>>();
	}

	ItemsLastSelected->Reserve(ItemsLastSelected->Num() + SequencerSelection->Outliner.Num());

	for (const FViewModelPtr OutlinerItem : SequencerSelection->Outliner)
	{
		ItemsLastSelected->Append(TryFindItems(OutlinerItem));
	}
}

TArray<FNavigationToolViewModelPtr> FNavigationTool::FindItemsFromMovieSceneObject(UObject* const InObject) const
{
	TArray<FNavigationToolViewModelPtr> OutItems;
	if (InObject)
	{
		for (const FNavigationToolViewModelPtr& Item : RootItem->GetDescendantsOfType<INavigationToolItem>())
		{
			if (Item->GetItemObject() == InObject)
			{
				OutItems.Add(Item);
			}
		}
	}
	return OutItems;
}

TArray<FNavigationToolViewModelPtr> FNavigationTool::FindItemsFromObjectGuid(const FGuid& InObjectGuid) const
{
	TArray<FNavigationToolViewModelPtr> OutItems;
	if (InObjectGuid.IsValid())
	{
		for (const TViewModelPtr<FNavigationToolBinding>& BindingItem : RootItem->GetDescendantsOfType<FNavigationToolBinding>())
		{
			if (BindingItem->GetBinding().GetObjectGuid() == InObjectGuid)
			{
				OutItems.Add(BindingItem);
			}
		}
	}
	return OutItems;
}

bool FNavigationTool::AreAllViewsSyncingItemSelection() const
{
	bool bIsSyncingItemSelection = false;

	ForEachToolView([&bIsSyncingItemSelection](const TSharedPtr<FNavigationToolView>& InView)
		{
			bIsSyncingItemSelection |= InView->IsSyncingItemSelection();
		});

	return bIsSyncingItemSelection;
}

void FNavigationTool::OnProvidersChanged(const FName InToolId
	, const TSharedRef<FNavigationToolProvider>& InProvider
	, const ENavigationToolProvidersChangeType InChangeType)
{
	if (InToolId != FNavigationToolExtender::GetToolInstanceId(*this))
	{
		return;
	}

	const TSharedPtr<ISequencer> Sequencer = GetSequencer();
	if (!Sequencer.IsValid()
		|| !InProvider->IsSequenceSupported(Sequencer->GetRootMovieSceneSequence()))
	{
		return;
	}

	RefreshGlobalFilters();

	OnTreeViewChanged();
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
