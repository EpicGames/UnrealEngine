// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorUndoClient.h"
#include "Engine/EngineTypes.h"
#include "INavigationTool.h"
#include "Items/NavigationToolItemId.h"
#include "Items/NavigationToolItemProxy.h"
#include "ItemProxies/NavigationToolItemProxyRegistry.h"
#include "ItemProxies/INavigationToolItemProxyFactory.h"
#include "NavigationToolDefines.h"
#include "SequencerCoreFwd.h"
#include "TickableEditorObject.h"
#include "UObject/WeakObjectPtrTemplates.h"

class AActor;
class FTransaction;
class FUICommandList;
class ISequencer;
class UMovieSceneSequence;
class USequencerSettings;

namespace UE::SequenceNavigator
{

class FNavigationToolBuiltInFilter;
class FNavigationToolProvider;
class FNavigationToolTab;
class FNavigationToolTreeRoot;
class FNavigationToolView;
class INavigationToolItemAction;
class INavigationToolView;
enum class ENavigationToolProvidersChangeType;

class FNavigationTool
	: public INavigationTool
	, public FTickableEditorObject
	, public FEditorUndoClient
{
public:
	FNavigationTool(const TWeakPtr<ISequencer>& InWeakSequencer);

	void Init();
	void Shutdown();

	bool CanProcessSequenceSpawn(UMovieSceneSequence* const InSequence) const;

	FOnToolLoaded OnToolLoaded;

	/** Gathers all previously existing and new Item Proxies for a given Item */
	void GetItemProxiesForItem(const FNavigationToolViewModelPtr& InItem, TArray<TSharedPtr<FNavigationToolItemProxy>>& OutItemProxies);

	/** Tries to find the Item Proxy Factory for the given Item Proxy Type Name */
	INavigationToolItemProxyFactory* GetItemProxyFactory(const FName InItemProxyTypeName) const;

	/** Returns whether the Navigation Tool is in Read-only mode */
	bool IsToolLocked() const;

	void HandleUndoRedoTransaction(const FTransaction* const InTransaction, const bool bInIsUndo);

	//~ Begin INavigationTool

	virtual void ForEachProvider(const TFunction<bool(const TSharedRef<FNavigationToolProvider>& InToolProvider)>& InPredicate) const override;

	virtual TSharedPtr<FUICommandList> GetBaseCommandList() const override;

	virtual bool IsToolTabVisible() const override;
	virtual void ShowHideToolTab(const bool bInVisible) override;
	virtual void ToggleToolTabVisible() override;

	virtual FOnToolLoaded& GetOnToolLoaded() override { return OnToolLoaded; }

	virtual TSharedPtr<ISequencer> GetSequencer() const override;

	virtual TSharedPtr<INavigationToolView> RegisterToolView(const int32 InToolViewId) override;
	virtual TSharedPtr<INavigationToolView> GetToolView(const int32 InToolViewId) const override;
	virtual TSharedPtr<INavigationToolView> GetMostRecentToolView() const override;

	virtual bool IsObjectAllowedInTool(const UObject* const InObject) const override;

	virtual void RegisterItem(const FNavigationToolViewModelPtr& InItem) override;
	virtual void UnregisterItem(const FNavigationToolItemId& InItemId) override;

	virtual void RequestRefresh() override;
	virtual void Refresh() override;

	virtual FNavigationToolViewModelWeakPtr GetTreeRoot() const override;

	virtual FNavigationToolViewModelPtr FindItem(const FNavigationToolItemId& InItemId) const override;
	virtual TArray<FNavigationToolViewModelPtr> TryFindItems(const Sequencer::FViewModelPtr& InSequencerOutlinerViewModel) const override;

	virtual void SetIgnoreNotify(const ENavigationToolIgnoreNotifyFlags InFlag, const bool bInIgnore) override;

	virtual void OnSequencerSelectionChanged() override;
	virtual TArray<FNavigationToolViewModelWeakPtr> GetSelectedItems(const bool bInNormalizeToTopLevelSelections = false) const override;
	virtual void SelectItems(const TArray<FNavigationToolViewModelWeakPtr>& InWeakItems
		, const ENavigationToolItemSelectionFlags InFlags = ENavigationToolItemSelectionFlags::SignalSelectionChange) const override;
	virtual void ClearItemSelection(const bool bInSignalSelectionChange) const override;

	virtual const FNavigationToolItemProxyRegistry& GetItemProxyRegistry() const override;
	virtual TArray<FName> GetRegisteredItemProxyTypeNames() const override;

	virtual void SetItemColor(const FNavigationToolViewModelPtr& InItem, const FColor& InColor) override;
	virtual void RemoveItemColor(const FNavigationToolViewModelPtr& InItem) override;
	virtual TOptional<FColor> FindItemColor(const FNavigationToolViewModelPtr& InItem, bool bRecurseParent = true) const override;

	virtual void EnqueueItemActions(TArray<TSharedPtr<INavigationToolItemAction>>&& InItemActions) noexcept override;

	virtual void NotifyToolItemRenamed(const FNavigationToolViewModelPtr& InItem) override;
	virtual void NotifyToolItemDeleted(const FNavigationToolViewModelPtr& InItem) override;

	//~ End INavigationTool

	FNavigationToolItemProxyRegistry& GetItemProxyRegistry();

	//~ Begin FEditorUndoClient
	virtual void PostUndo(const bool bInSuccess) override;
	virtual void PostRedo(const bool bInSuccess) override;
	//~ End FEditorUndoClient

	//~ Begin FTickableObjectBase
	virtual TStatId GetStatId() const override;
	virtual void Tick(const float InDeltaTime) override;
	//~ End FTickableObjectBase

	/** Delete a set of items in the outliner by calling their custom delete handler */
	void DeleteItems(const TArray<FNavigationToolViewModelWeakPtr>& InWeakItems);

	/** Unregisters the Navigation Tool View bound to the given id */
	void UnregisterToolView(const int32 InToolViewId);

	/** Sets the given Navigation Tool View Id as the most recent Navigation Tool View */
	void UpdateRecentToolViews(const int32 InToolViewId);

	/** Executes the given predicate for each Navigation Tool View registered */
	void ForEachToolView(const TFunction<void(const TSharedRef<FNavigationToolView>& InToolView)>& InPredicate) const;

	/** @return Number of actions that been added to the queue so far before triggering a refresh */
	int32 GetPendingItemActionCount() const;

	/** @return True if the Navigation Tool is currently in need of a refresh */
	bool NeedsRefresh() const;

	/**
	 * Replaces the Item's Id in the Item Map. This can be due to an object item changing it's object
	 * (e.g. a bp component getting destroyed and recreated, the item should be the same but the underlying component will not be)
	 */
	void NotifyItemIdChanged(const FNavigationToolItemId& OldId, const FNavigationToolViewModelPtr& InItem);

	/** Gets the closest item to all the given items while also being their common ancestor */
	static FNavigationToolViewModelPtr FindLowestCommonAncestor(const TArray<FNavigationToolViewModelPtr>& Items);

	/**
	 * Sort the given array of items based on their ordering in the Navigation Tool
	 * @see FNavigationTool::CompareToolItemOrder
	 */
	static void SortItems(TArray<FNavigationToolViewModelWeakPtr>& WeakItems, const bool bInReverseOrder = false);

	/** Normalizes the given Items by removing selected items that have their parent item also in the selection */
	static void NormalizeToTopLevelSelections(TArray<FNavigationToolViewModelWeakPtr>& WeakItems);

	/** Have the given Selected Items sync to the USelection Instances of Mode Tools */
	void SyncSequencerSelection(const TArray<FNavigationToolViewModelWeakPtr>& InWeakSelectedItems) const;

	/** Called when the engine replaces an object. A common example is when a BP Component is destroyed, and replaced */
	void OnObjectsReplaced(const TMap<UObject*, UObject*>& InReplacementMap);

	/** Called when the engine replaces an Actor */
	void OnActorReplaced(AActor* const InOldActor, AActor* const InNewActor);

	/** Marks the Navigation Tool dirty. This triggers INavigationToolProvider::OnToolModified on next tick */
	void SetToolModified();

	/** If ShouldApplyDefaultColumnView(), applies the active providers default column view */
	void RefreshColumnView();

	bool DoesGlobalFilterExist(const FName InFilterId);
	void RefreshGlobalFilters();

private:
	friend class FNavigationToolView;
	friend class FNavigationToolToolbarMenu;

	void OnTabVisibilityChanged(const bool bInVisible);

	void BindEvents();
	void UnbindEvents();

	void AddItem(const FNavigationToolViewModelPtr& InItem);
	void RemoveItem(const FNavigationToolItemId& InItemId);
	void ForEachItem(TFunctionRef<void(const FNavigationToolViewModelPtr&)> InFunc);

	USequencerSettings* GetSequencerSettings() const;

	void OnTreeViewChanged();

	TArray<FNavigationToolViewModelPtr> FindItemsFromMovieSceneObject(UObject* const InObject) const;
	TArray<FNavigationToolViewModelPtr> FindItemsFromObjectGuid(const FGuid& InObjectGuid) const;

	bool AreAllViewsSyncingItemSelection() const;

	void OnProvidersChanged(const FName InToolId
		, const TSharedRef<FNavigationToolProvider>& InProvider
		, const ENavigationToolProvidersChangeType InChangeType);

	TWeakPtr<ISequencer> WeakSequencer;

	TSharedRef<FNavigationToolTab> ToolTab;

	/** The root of all the items in the outliner */
	Sequencer::TViewModelPtr<FNavigationToolTreeRoot> RootItem;

	TSharedPtr<FUICommandList> BaseCommandList;

	/** The map of the registered items */
	TMap<FNavigationToolItemId, FNavigationToolViewModelPtr> ItemMap;

	TMap<FNavigationToolItemId, FNavigationToolViewModelPtr> ItemsPendingAdd;

	TSet<FNavigationToolItemId> ItemsPendingRemove;

	/** Navigation Tool's Item Proxy Factory Registry Instance. This takes precedence over the Module's Factory Registry */
	FNavigationToolItemProxyRegistry ItemProxyRegistry;
	
	/** The current pending actions before refresh is called */
	TArray<TSharedPtr<INavigationToolItemAction>> PendingActions;

	/** The list of items pending selection processing, filled in when the Sequencer selection changes */
	TSharedPtr<TArray<FNavigationToolViewModelWeakPtr>> ItemsLastSelected;

	/** The map of registered outliner views */
	TMap<int32, TSharedPtr<FNavigationToolView>> ToolViews;

	/** List of Navigation Tool View Ids in order from least recent to most recent (i.e. Index 0 is least recent) */
	TArray<int32> RecentToolViews;

	/** The current events to ignore and not handle automatically */
	ENavigationToolIgnoreNotifyFlags IgnoreNotifyFlags = ENavigationToolIgnoreNotifyFlags::None;

	/** Flag indicating whether the Navigation Tool has been changed this tick and should call INavigationToolProvider::OnToolModified next tick */
	bool bToolDirty = false;
	
	/** Flag indicating Refreshing is taking place */
	bool bRefreshing = false;

	/** Flag indicating that a refresh must take place next tick */
	bool bRefreshRequested = false;

	/** Flag indicating that the Item Map is iterating */
	bool bIteratingItemMap = false;

	/** Built in "global" item type filters for all views */
	TArray<TSharedPtr<FNavigationToolBuiltInFilter>> GlobalFilters;
};

} // namespace UE::SequenceNavigator
