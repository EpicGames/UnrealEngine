// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavigationToolCommands.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "NavigationToolCommands"

namespace UE::SequenceNavigator
{

FNavigationToolCommands::FNavigationToolCommands()
	: TCommands<FNavigationToolCommands>(TEXT("SequenceNavigator")
	, LOCTEXT("SequenceNavigator", "Sequence Navigator")
	, NAME_None
	, FAppStyle::GetAppStyleSetName())
{
}

void FNavigationToolCommands::RegisterCommands()
{
	FUICommandInfo::MakeCommandInfo(AsShared()
		, OpenToolSettings
		, TEXT("OpenToolSettings")
		, LOCTEXT("OpenToolSettings", "Sequence Navigator Settings...")
		, LOCTEXT("OpenToolSettingsTooltip", "Opens the Sequence Navigator editor settings")
		, FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("Icons.Settings"))
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(ToggleToolTabVisible
		, "Sequence Navigator"
		, "Toggle the visibility of the Sequence Navigator"
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(Refresh
		, "Refresh"
		, "Refreshes the outliner view"
		, EUserInterfaceActionType::Button
		, FInputChord(EKeys::F5));

	UI_COMMAND(SelectAllChildren
		, "Select All Children"
		, "Selects all the children (recursively) of each selection"
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(SelectImmediateChildren
		, "Select Immediate Children"
		, "Selects only the immediate children of each selection"
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(SelectParent
		, "Select Parent"
		, "Selects the parent item of each selection"
		, EUserInterfaceActionType::Button
		, FInputChord(EModifierKey::Shift, EKeys::Left));

	UI_COMMAND(SelectFirstChild
		, "Select First Child"
		, "Selects the first child item of each selection"
		, EUserInterfaceActionType::Button
		, FInputChord(EModifierKey::Shift, EKeys::Right));

	UI_COMMAND(SelectPreviousSibling
		, "Select Previous Sibling"
		, "Selects the previous sibling of each selection"
		, EUserInterfaceActionType::Button
		, FInputChord(EModifierKey::Shift, EKeys::Up));

	UI_COMMAND(SelectNextSibling
		, "Select Next Sibling"
		, "Selects the next sibling item of each selection"
		, EUserInterfaceActionType::Button
		, FInputChord(EModifierKey::Shift, EKeys::Down));

	UI_COMMAND(ExpandAll
		, "Expand All"
		, "Expands all items in outliner"
		, EUserInterfaceActionType::Button
		, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::RightBracket));

	UI_COMMAND(CollapseAll
		, "Collapse All"
		, "Collapses all items in outliner"
		, EUserInterfaceActionType::Button
		, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::LeftBracket));

	UI_COMMAND(ScrollNextSelectionIntoView
		, "Scroll to Next"
		, "Scrolls the next selection into view"
		, EUserInterfaceActionType::Button
		, FInputChord(EModifierKey::Alt, EKeys::N));

	UI_COMMAND(ToggleMutedHierarchy
		, "Muted Hierarchy"
		, "Show the parent of the shown items, even if the parents are filtered out"
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(ToggleAutoExpandToSelection
		, "Auto Expand to Selection"
		, "Auto expand the hierarchy to show the item when selected"
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(ToggleShortNames
		, "Short Names"
		, "Shortens child item names to exclude parent item names"
		, EUserInterfaceActionType::ToggleButton
		, FInputChord());

	UI_COMMAND(ResetVisibleColumnSizes
		, "Reset Visible Column Sizes"
		, "Resets the size of all visible columns to their defaults"
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(SaveCurrentColumnView
		, "Save Current Column View..."
		, "Save the current column visible set as a preset column view"
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(FocusSingleSelection
		, "Focus Sequence in Sequencer"
		, "Sets this sequence as the root sequence to focus on in the Sequencer"
		, EUserInterfaceActionType::Button
		, FInputChord());

	UI_COMMAND(FocusInContentBrowser
		, "Focus in Content Browser"
		, "Focus this sequence in the Content Browser"
		, EUserInterfaceActionType::Button
		, FInputChord());
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
