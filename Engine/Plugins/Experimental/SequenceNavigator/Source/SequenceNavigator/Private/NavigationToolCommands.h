// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"

namespace UE::SequenceNavigator
{

class FNavigationToolCommands
	: public TCommands<FNavigationToolCommands>
{
public:
	FNavigationToolCommands();

	virtual void RegisterCommands() override;

	TSharedPtr<FUICommandInfo> OpenToolSettings;

	TSharedPtr<FUICommandInfo> ToggleToolTabVisible;

	TSharedPtr<FUICommandInfo> Refresh;

	TSharedPtr<FUICommandInfo> SelectAllChildren;

	TSharedPtr<FUICommandInfo> SelectImmediateChildren;

	TSharedPtr<FUICommandInfo> SelectParent;

	TSharedPtr<FUICommandInfo> SelectFirstChild;

	TSharedPtr<FUICommandInfo> SelectPreviousSibling;

	TSharedPtr<FUICommandInfo> SelectNextSibling;

	TSharedPtr<FUICommandInfo> ExpandAll;

	TSharedPtr<FUICommandInfo> CollapseAll;

	TSharedPtr<FUICommandInfo> ExpandSelection;

	TSharedPtr<FUICommandInfo> CollapseSelection;

	TSharedPtr<FUICommandInfo> ScrollNextSelectionIntoView;

	TSharedPtr<FUICommandInfo> ToggleMutedHierarchy;

	TSharedPtr<FUICommandInfo> ToggleAutoExpandToSelection;

	TSharedPtr<FUICommandInfo> ToggleShortNames;

	TSharedPtr<FUICommandInfo> ResetVisibleColumnSizes;

	TSharedPtr<FUICommandInfo> SaveCurrentColumnView;

	TSharedPtr<FUICommandInfo> FocusSingleSelection;

	TSharedPtr<FUICommandInfo> FocusInContentBrowser;
};

} // namespace UE::SequenceNavigator
