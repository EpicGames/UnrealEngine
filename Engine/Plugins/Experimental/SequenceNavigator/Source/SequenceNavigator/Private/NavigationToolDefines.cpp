// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavigationToolDefines.h"
#include "Menus/NavigationToolItemContextMenu.h"
#include "Menus/NavigationToolToolbarMenu.h"

namespace UE::SequenceNavigator
{

FName GetToolBarMenuName()
{
	return FNavigationToolToolbarMenu::GetMenuName();
}

FName GetItemContextMenuName()
{
	return FNavigationToolItemContextMenu::GetMenuName();
}

} // namespace UE::SequenceNavigator
