// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavigationToolExtender.h"
#include "Customization/INavigationToolIconCustomization.h"
#include "Features/IModularFeatures.h"
#include "INavigationTool.h"
#include "ISequencer.h"
#include "Items/NavigationToolComponent.h"
#include "Modules/ModuleManager.h"
#include "NavigationTool.h"
#include "NavigationToolCommands.h"
#include "NavigationToolStyle.h"
#include "Providers/NavigationToolProvider.h"
#include "SequenceNavigatorLog.h"
#include "SequencerSettings.h"
#include "SequencerToolMenuContext.h"

#define LOCTEXT_NAMESPACE "NavigationToolExtender"

namespace UE::SequenceNavigator
{

FNavigationToolExtender& FNavigationToolExtender::Get()
{
	static FNavigationToolExtender Instance;
	return Instance;
}

FNavigationToolExtender::FNavigationToolExtender()
{
	IModularFeatures::Get().RegisterModularFeature(GetModularFeatureName(), this);

	ISequencerModule& SequencerModule = FModuleManager::LoadModuleChecked<ISequencerModule>(TEXT("Sequencer"));

	SequencerCreatedHandle = SequencerModule.RegisterOnSequencerCreated(
		FOnSequencerCreated::FDelegate::CreateRaw(this, &FNavigationToolExtender::OnSequencerCreated));

	// @TODO: register item proxies?
	//ItemProxyRegistry.RegisterItemProxyWithDefaultFactory<FNavigationToolComponentProxy, 10>();
}

FNavigationToolExtender::~FNavigationToolExtender()
{
	IModularFeatures::Get().UnregisterModularFeature(GetModularFeatureName(), this);

	if (FModuleManager::Get().IsModuleLoaded(TEXT("Sequencer")))
	{
		ISequencerModule& SequencerModule = FModuleManager::LoadModuleChecked<ISequencerModule>(TEXT("Sequencer"));

		SequencerModule.UnregisterOnSequencerCreated(SequencerCreatedHandle);
	}

	for (const TPair<FName, FNavigationToolInstance>& ToolInstance : ToolInstances)
	{
		RemoveSequencerToolBarExtension(ToolInstance.Value);
	}
}

FNavigationToolExtender::FNavigationToolInstance* FNavigationToolExtender::FindOrAddToolInstance_Internal(
	const TSharedRef<ISequencer>& InSequencer)
{
	const FName NewToolId = GetToolInstanceId(*InSequencer);
	if (NewToolId.IsNone())
	{
		return nullptr;
	}

	FNavigationToolInstance& ToolInstance = ToolInstances.FindOrAdd(NewToolId);

	ToolInstance.ToolId = NewToolId;
	ToolInstance.WeakSequencer = InSequencer;

	if (!ToolInstance.ActivateSequenceHandle.IsValid())
	{
		ToolInstance.ActivateSequenceHandle = InSequencer->OnActivateSequence().AddLambda(
			[this, NewToolId](FMovieSceneSequenceIDRef InSequenceId)
			{
				OnSequencerActivated(NewToolId, InSequenceId);
			});
	}

	if (!ToolInstance.SequencerClosedHandle.IsValid())
	{
		ToolInstance.SequencerClosedHandle = InSequencer->OnCloseEvent().AddLambda(
			[this, NewToolId](const TSharedRef<ISequencer> InSequencer)
			{
				OnSequencerClosed(NewToolId, InSequencer);
			});
	}

	if (!ToolInstance.Instance.IsValid())
	{
		const TSharedPtr<FNavigationTool> NewInstance = MakeShared<FNavigationTool>(InSequencer);
		NewInstance->Init();
		ToolInstance.Instance = NewInstance;
	}

	return &ToolInstance;
}

void FNavigationToolExtender::OnSequencerCreated(const TSharedRef<ISequencer> InSequencer)
{
	const FName NewToolId = GetToolInstanceId(*InSequencer);
	if (NewToolId.IsNone())
	{
		return;
	}

	if (!AnyProviderSupportsSequencer(NewToolId, *InSequencer))
	{
		return;
	}

	FNavigationToolInstance* const NewToolInstance = FindOrAddToolInstance_Internal(InSequencer);
	if (!NewToolInstance || !NewToolInstance->Instance.IsValid())
	{
		return;
	}

	FNavigationTool& ToolRef = *NewToolInstance->Instance;

	for (const TSharedRef<FNavigationToolProvider>& Provider : NewToolInstance->Providers)
	{
		Provider->Activate(ToolRef);
	}

	AddSequencerToolBarExtension(*NewToolInstance);
}

void FNavigationToolExtender::OnSequencerActivated(const FName InToolId, FMovieSceneSequenceIDRef InSequenceId)
{
	// Nothing to do here. May still be used for something, otherwise remove in the future
	/*if (GIsTransacting)
	{
		return;
	}

	FNavigationToolInstance* const ToolInstance = ToolInstances.Find(InToolId);
	if (!ToolInstance || !ToolInstance->WeakSequencer.IsValid())
	{
		return;
	}

	// Only care about root sequence changes
	const TSharedPtr<ISequencer> Sequencer = ToolInstance->WeakSequencer.Pin();
	if (!Sequencer.IsValid() || InSequenceId != Sequencer->GetRootTemplateID())
	{
		return;
	}*/
}

void FNavigationToolExtender::OnSequencerClosed(const FName InToolId, const TSharedRef<ISequencer> InSequencer)
{
	FNavigationToolInstance* const ToolInstance = ToolInstances.Find(InToolId);
	if (!ToolInstance || !ToolInstance->WeakSequencer.IsValid())
	{
		return;
	}

	const TSharedPtr<ISequencer> Sequencer = ToolInstance->WeakSequencer.Pin();
	if (!Sequencer.IsValid() || InSequencer != Sequencer)
	{
		return;
	}

	for (const TSharedRef<FNavigationToolProvider>& Provider : ToolInstance->Providers)
	{
		Provider->Deactivate(*ToolInstance->Instance);
	}

	InSequencer->OnCloseEvent().Remove(ToolInstance->SequencerClosedHandle);
	InSequencer->OnActivateSequence().Remove(ToolInstance->ActivateSequenceHandle);
	ToolInstance->SequencerClosedHandle.Reset();
	ToolInstance->ActivateSequenceHandle.Reset();

	if (ToolInstance->Instance.IsValid())
	{
		ToolInstance->Instance->Shutdown();
		ToolInstance->Instance.Reset();
	}

	ToolInstance->WeakSequencer.Reset();
}

FName FNavigationToolExtender::GetModularFeatureName()
{
	static FName FeatureName = TEXT("NavigationTool");
	return FeatureName;
}

FName FNavigationToolExtender::GetToolInstanceId(const ISequencer& InSequencer)
{
	const USequencerSettings* const SequencerSettings = InSequencer.GetSequencerSettings();
	return SequencerSettings ? SequencerSettings->GetFName() : NAME_None;
}

FName FNavigationToolExtender::GetToolInstanceId(const INavigationTool& InTool)
{
	if (const TSharedPtr<ISequencer> Sequencer = InTool.GetSequencer())
	{
		return GetToolInstanceId(*Sequencer);
	}
	return NAME_None;
}

TSharedPtr<INavigationTool> FNavigationToolExtender::FindNavigationTool(const TSharedRef<ISequencer>& InSequencer)
{
	const FName ToolId = GetToolInstanceId(*InSequencer);
	if (ToolId.IsNone())
	{
		return nullptr;
	}

	FNavigationToolInstance* const ToolInstance = Get().ToolInstances.Find(ToolId);
	if (!ToolInstance)
	{
		return nullptr;
	}

	return ToolInstance->Instance;
}

bool FNavigationToolExtender::RegisterToolProvider(const TSharedRef<ISequencer>& InSequencer
	, const TSharedRef<FNavigationToolProvider>& InProvider)
{
	const FName ToolId = GetToolInstanceId(*InSequencer);

	if (ToolId.IsNone())
	{
		UE_LOGF(LogSequenceNavigator, Error, "Invalid tool instance name: %ls"
			, *ToolId.ToString());
		return false;
	}

	const FName NewProviderId = InProvider->GetIdentifier();

	if (NewProviderId.IsNone())
	{
		UE_LOGF(LogSequenceNavigator, Error, "Invalid provider name: %ls"
			, *NewProviderId.ToString());
		return false;
	}

	if (const TSharedPtr<FNavigationToolProvider> ExistingProvider = FindToolProvider(ToolId, NewProviderId))
	{
		UE_LOGF(LogSequenceNavigator, Warning, "Provider already registered: %ls"
			, *NewProviderId.ToString());
		return true;
	}

	FNavigationToolExtender& This = Get();

	FNavigationToolInstance* const ToolInstance = This.FindOrAddToolInstance_Internal(InSequencer);
	if (!ToolInstance)
	{
		UE_LOGF(LogSequenceNavigator, Error, "Tool instance could not be created for provider: %ls"
			, *NewProviderId.ToString());
		return true;
	}

	ToolInstance->Providers.Add(InProvider);

	if (InProvider->IsSequenceSupported(InSequencer->GetRootMovieSceneSequence()))
	{
		InProvider->Activate(*ToolInstance->Instance);

		AddSequencerToolBarExtension(*ToolInstance);
	}

	UE_LOGF(LogSequenceNavigator, Log, "Tool instance '%ls' provider '%ls' registered"
		, *ToolId.ToString(), *NewProviderId.ToString());

	This.ProvidersChangedDelegate.Broadcast(ToolId, InProvider, ENavigationToolProvidersChangeType::Add);

	return true;
}

bool FNavigationToolExtender::UnregisterToolProvider(const FName InToolId, const FName InProviderId)
{
	FNavigationToolExtender& This = Get();

	FNavigationToolInstance* const ToolInstance = This.ToolInstances.Find(InToolId);
	if (!ToolInstance)
	{
		return false;
	}

	bool bProviderRemoved = false;

	for (auto It = ToolInstance->Providers.CreateIterator(); It; ++It)
	{
		const TSharedRef<FNavigationToolProvider>& Provider = *It;
		if (Provider->GetIdentifier() == InProviderId)
		{
			if (ToolInstance->Instance.IsValid())
			{
				if (const TSharedPtr<ISequencer> Sequencer = ToolInstance->WeakSequencer.Pin())
				{
					if (Provider->IsSequenceSupported(Sequencer->GetRootMovieSceneSequence()))
					{
						Provider->Deactivate(*ToolInstance->Instance);
					}
				}
			}

			UE_LOGF(LogSequenceNavigator, Log, "Provider unregistered: %ls"
				, *Provider->GetIdentifier().ToString());

			This.ProvidersChangedDelegate.Broadcast(InToolId, Provider, ENavigationToolProvidersChangeType::Remove);

			It.RemoveCurrent();
			bProviderRemoved = true;

			break;
		}
	}

	if (bProviderRemoved && ToolInstance->WeakSequencer.IsValid())
	{
		if (!AnyProviderSupportsSequencer(InToolId, *ToolInstance->WeakSequencer.Pin()))
		{
			RemoveSequencerToolBarExtension(*ToolInstance);
		}
	}

	return false;
}

TSet<TSharedRef<FNavigationToolProvider>>* FNavigationToolExtender::FindToolProviders(const FName InToolId)
{
	if (FNavigationToolInstance* const ToolInstance = Get().ToolInstances.Find(InToolId))
	{
		return &ToolInstance->Providers;
	}
	return nullptr;
}

TSharedPtr<FNavigationToolProvider> FNavigationToolExtender::FindToolProvider(const FName InToolId, const FName InProviderId)
{
	if (FNavigationToolInstance* const ToolInstance = Get().ToolInstances.Find(InToolId))
	{
		for (const TSharedRef<FNavigationToolProvider>& Provider : ToolInstance->Providers)
		{
			if (Provider->GetIdentifier() == InProviderId)
			{
				return Provider;
			}
		}
	}
	return nullptr;
}

bool FNavigationToolExtender::FindToolProviders(const FName InToolId
	, TSet<TSharedRef<FNavigationToolProvider>>& OutProviders)
{
	if (TSet<TSharedRef<FNavigationToolProvider>>* const Providers = FindToolProviders(InToolId))
	{
		OutProviders.Append(*Providers);
		return true;
	}
	return false;
}

void FNavigationToolExtender::ForEachProvider(const TFunction<bool(const FName InToolId
	, const TSharedRef<FNavigationToolProvider>& InProvider)>& InPredicate)
{
	for (const TPair<FName, FNavigationToolInstance>& InstancePair : Get().ToolInstances)
	{
		for (const TSharedRef<FNavigationToolProvider>& Provider : InstancePair.Value.Providers)
		{
			// Stop processing if the function returns false
			if (!InPredicate(InstancePair.Key, Provider))
			{
				return;
			}
		}
	}
}

void FNavigationToolExtender::ForEachToolProvider(const FName InToolId
	, const TFunction<bool(const TSharedRef<FNavigationToolProvider>& InProvider)>& InPredicate)
{
	TSet<TSharedRef<FNavigationToolProvider>> const* Providers = Get().FindToolProviders(InToolId);
	if (!Providers)
	{
		return;
	}

	for (const TSharedRef<FNavigationToolProvider>& Provider : *Providers)
	{
		// Stop processing if the function returns false
		if (!InPredicate(Provider))
		{
			return;
		}
	}
}

void FNavigationToolExtender::AddSequencerToolBarExtension(const FNavigationToolInstance& InToolInstance)
{
	if (!InToolInstance.Instance.IsValid() || !InToolInstance.WeakSequencer.IsValid())
	{
		return;
	}

	if (!FModuleManager::Get().IsModuleLoaded(TEXT("ToolMenus")))
	{
		return;
	}

	UToolMenus* const ToolMenus = UToolMenus::Get();
	if (!ToolMenus)
	{
		return;
	}

	UToolMenu* const ToolMenu = ToolMenus->ExtendMenu(TEXT("Sequencer.MainToolBar"));
	if (!ToolMenu)
	{
		return;
	}

	const FName ToolId = InToolInstance.ToolId;

	FToolMenuSection& NavigationToolSection = ToolMenu->FindOrAddSection(TEXT("NavigationTool")
		, LOCTEXT("SequenceNavigatorSectionDisplayName", "Sequence Navigator")
		, FToolMenuInsert(TEXT("CurveEditor"), EToolMenuInsertType::After));

	NavigationToolSection.AddDynamicEntry(TEXT("NavigationTool")
		, FNewToolMenuSectionDelegate::CreateLambda([ToolId, ToolInstance = InToolInstance](FToolMenuSection& InSection)
		{
			USequencerToolMenuContext* const ContextObject = InSection.FindContext<USequencerToolMenuContext>();
			if (!ContextObject)
			{
				return;
			}

			const TSharedPtr<ISequencer> Sequencer = ContextObject->WeakSequencer.Pin();
			if (Sequencer.IsValid() && !AnyProviderSupportsSequencer(ToolId, *Sequencer.Get()))
			{
				return;
			}

			FToolMenuEntry ToggleTabVisibleEntry = FToolMenuEntry::InitToolBarButton(
				FNavigationToolCommands::Get().ToggleToolTabVisible,
				TAttribute<FText>(), TAttribute<FText>(),
				FSlateIcon(FNavigationToolStyle::Get().GetStyleSetName(), TEXT("Icon.ToolBar")));
			ToggleTabVisibleEntry.SetCommandList(ToolInstance.Instance->GetBaseCommandList());
			ToggleTabVisibleEntry.StyleNameOverride = TEXT("SequencerToolbar");
			InSection.AddEntry(ToggleTabVisibleEntry);
		}));
}

void FNavigationToolExtender::RemoveSequencerToolBarExtension(const FNavigationToolInstance& InToolInstance)
{
	if (!FModuleManager::Get().IsModuleLoaded(TEXT("ToolMenus")))
	{
		return;
	}

	UToolMenus* const ToolMenus = UToolMenus::Get();
	if (!ToolMenus)
	{
		return;
	}

	if (UToolMenu* const ToolMenu = ToolMenus->FindMenu(TEXT("Sequencer.MainToolBar")))
	{
		ToolMenu->RemoveSection(TEXT("NavigationTool"));
	}
}

bool FNavigationToolExtender::AnyProviderSupportsSequencer(const FName InToolId, const ISequencer& InSequencer)
{
	UMovieSceneSequence* const RootSequence = InSequencer.GetRootMovieSceneSequence();
	if (!RootSequence)
	{
		return false;
	}

	bool bSupportsSequencer = false;

	ForEachToolProvider(InToolId, [RootSequence, &bSupportsSequencer]
		(const TSharedRef<FNavigationToolProvider>& InProvider)
		{
			if (InProvider->IsSequenceSupported(RootSequence))
			{
				bSupportsSequencer = true;
				return false;
			}
			return true; // Continue processing providers
		});

	return bSupportsSequencer;
}

FSlateIcon FNavigationToolExtender::FindOverrideIcon(const FNavigationToolViewModelPtr& InItem)
{
	if (const TSharedPtr<INavigationToolIconCustomization> CustomizationToUse = Get().GetCustomizationForItem(InItem))
	{
		return CustomizationToUse->GetOverrideIcon(InItem);
	}
	return FSlateIcon();
}

void FNavigationToolExtender::RegisterOverridenIcon_Internal(const Sequencer::FViewModelTypeID& InItemTypeId, const TSharedRef<INavigationToolIconCustomization>& InIconCustomization)
{
	FIconCustomizationKey Key;
	Key.ItemTypeId = InItemTypeId.GetTypeID();
	Key.CustomizationSpecializationIdentifier = InIconCustomization->GetToolItemIdentifier();

	if (!IconRegistry.Contains(Key))
	{
		IconRegistry.Add(Key, InIconCustomization);
	}
}

void FNavigationToolExtender::UnregisterOverriddenIcon_Internal(const Sequencer::FViewModelTypeID& InItemTypeId, const FName& InSpecializationIdentifier)
{
	FIconCustomizationKey Key;
	Key.ItemTypeId = InItemTypeId.GetTypeID();
	Key.CustomizationSpecializationIdentifier = InSpecializationIdentifier;

	IconRegistry.Remove(Key);
}

TSharedPtr<INavigationToolIconCustomization> FNavigationToolExtender::GetCustomizationForItem(
	const FNavigationToolViewModelPtr& InItem) const
{
	if (!InItem.IsValid())
	{
		return nullptr;
	}

	// There is always going to be just one icon customization that supports the item since it checks specifically for the SupportCustomization
	for (const TPair<FIconCustomizationKey, TSharedPtr<INavigationToolIconCustomization>>& IconCustomization : IconRegistry)
	{
		if (IconCustomization.Key.ItemTypeId == InItem.AsModel()->GetTypeTable().GetTypeID()
			&& IconCustomization.Value->HasOverrideIcon(InItem))
		{
			return IconCustomization.Value;
		}
	}

	return nullptr;
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
