// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavigationToolScopedSelection.h"
#include "ISequencer.h"
#include "MVVM/ObjectBindingModelStorageExtension.h"
#include "MVVM/SectionModelStorageExtension.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/TrackModelStorageExtension.h"
#include "MVVM/ViewModels/ObjectBindingModel.h"
#include "MVVM/ViewModels/SectionModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/TrackModel.h"
#include "MVVM/Views/SOutlinerView.h"
#include "NavigationToolSettings.h"

namespace UE::SequenceNavigator
{

using namespace Sequencer;

FNavigationToolScopedSelection::FNavigationToolScopedSelection(ISequencer& InSequencer
	, ENavigationToolScopedSelectionPurpose InPurpose)
	: Sequencer(InSequencer)
	, Purpose(InPurpose)
{
}

FNavigationToolScopedSelection::~FNavigationToolScopedSelection()
{
	if (Purpose == ENavigationToolScopedSelectionPurpose::Sync)
	{
		SyncSelections();
	}
}

void FNavigationToolScopedSelection::Select(const FGuid& InObjectGuid)
{
	if (!ensureMsgf(Purpose == ENavigationToolScopedSelectionPurpose::Sync
		, TEXT("Scope is trying to Select, but it's not a Sync Scope.")))
	{
		return;
	}

	if (!InObjectGuid.IsValid())
	{
		return;
	}

	const UObject* const SpawnedObject = Sequencer.FindSpawnedObjectOrTemplate(InObjectGuid);

	bool bIsAlreadyInSet = false;
	ObjectsSet.Add(SpawnedObject, &bIsAlreadyInSet);
	if (bIsAlreadyInSet)
	{
		return;
	}

	SelectedObjectGuids.Add(InObjectGuid);
}

void FNavigationToolScopedSelection::Select(UMovieSceneSection* const InSection)
{
	if (!ensureMsgf(Purpose == ENavigationToolScopedSelectionPurpose::Sync
		, TEXT("Scope is trying to Select, but it's not a Sync Scope.")))
	{
		return;
	}

	if (!InSection)
	{
		return;
	}

	bool bIsAlreadyInSet = false;
	ObjectsSet.Add(InSection, &bIsAlreadyInSet);
	if (bIsAlreadyInSet)
	{
		return;
	}

	SelectedSections.Add(InSection);
}

void FNavigationToolScopedSelection::Select(UMovieSceneTrack* const InTrack)
{
	if (!ensureMsgf(Purpose == ENavigationToolScopedSelectionPurpose::Sync
		, TEXT("Scope is trying to Select, but it's not a Sync Scope.")))
	{
		return;
	}

	if (!InTrack)
	{
		return;
	}

	bool bIsAlreadyInSet = false;
	ObjectsSet.Add(InTrack, &bIsAlreadyInSet);
	if (bIsAlreadyInSet)
	{
		return;
	}

	SelectedTracks.Add(InTrack);
}

void FNavigationToolScopedSelection::Select(UMovieSceneSequence* const InSequence
	, const int32 InMarkedFrameIndex)
{
	if (!ensureMsgf(Purpose == ENavigationToolScopedSelectionPurpose::Sync
		, TEXT("Scope is trying to Select, but it's not a Sync Scope.")))
	{
		return;
	}

	if (!InSequence || InMarkedFrameIndex == INDEX_NONE)
	{
		return;
	}

	SelectedMarkedFrames.FindOrAdd(InSequence).Add(InMarkedFrameIndex);
}

bool FNavigationToolScopedSelection::IsSelected(const UObject* const InObject) const
{
	return ObjectsSet.Contains(InObject);
}

bool FNavigationToolScopedSelection::IsSelected(const FGuid& InObjectGuid) const
{
	return SelectedObjectGuids.Contains(InObjectGuid);
}

bool FNavigationToolScopedSelection::IsSelected(UMovieSceneSection* const InSection) const
{
	return SelectedSections.Contains(InSection);
}

bool FNavigationToolScopedSelection::IsSelected(UMovieSceneTrack* const InTrack) const
{
	return SelectedTracks.Contains(InTrack);
}

bool FNavigationToolScopedSelection::IsSelected(UMovieSceneSequence* const InSequence
	, const int32 InMarkedFrameIndex) const
{
	if (InSequence && SelectedMarkedFrames.Contains(InSequence))
	{
		return SelectedMarkedFrames[InSequence].Contains(InMarkedFrameIndex);
	}
	return false;
}

void FNavigationToolScopedSelection::SyncSelections()
{
	const TSharedPtr<FSequencerEditorViewModel> ViewModel = Sequencer.GetViewModel();
	if (!ViewModel.IsValid())
	{
		return;
	}

	const TSharedPtr<FSequencerSelection> Selection = ViewModel->GetSelection();
	if (!Selection.IsValid())
	{
		return;
	}

	const FViewModelPtr RootViewModel = ViewModel->GetRootModel();
	if (!Selection.IsValid())
	{
		return;
	}

	const FModifierKeysState ModifierKeys = FSlateApplication::Get().GetModifierKeys();
	UNavigationToolSettings* const ToolSettings = GetMutableDefault<UNavigationToolSettings>();
	const bool bPreviouslySyncingSelection = ToolSettings->ShouldSyncSelectionToNavigationTool();

	if (!ModifierKeys.IsAltDown())
	{
		ToolSettings->SetSyncSelectionToNavigationTool(/*bInSync=*/false, /*bInSaveConfig=*/false);
	}

	const TSharedPtr<SOutlinerView> OutlinerView = Sequencer.GetOutlinerViewWidget();

	Sequencer.EmptySelection();

	for (const FGuid& ObjectGuid : SelectedObjectGuids)
	{
		if (FObjectBindingModelStorageExtension* const ObjectBindingModelStorage = RootViewModel->CastDynamic<FObjectBindingModelStorageExtension>())
		{
			if (const TSharedPtr<FObjectBindingModel> ObjectBindingModel = ObjectBindingModelStorage->FindModelForObjectBinding(ObjectGuid))
			{
				Selection->Outliner.Select(ObjectBindingModel);

				if (OutlinerView.IsValid())
				{
					OutlinerView->RequestScrollIntoView(ObjectBindingModel);
				}
			}
		}
	}

	for (UMovieSceneTrack* const Track : SelectedTracks)
	{
		if (FTrackModelStorageExtension* const TrackModelStorage = RootViewModel->CastDynamic<FTrackModelStorageExtension>())
		{
			if (const TSharedPtr<FTrackModel> TrackModel = TrackModelStorage->FindModelForTrack(Track))
			{
				Selection->Outliner.Select(TrackModel);

				if (OutlinerView.IsValid())
				{
					OutlinerView->RequestScrollIntoView(TrackModel);
				}
			}
		}
	}

	for (UMovieSceneSection* const Section : SelectedSections)
	{
		if (FSectionModelStorageExtension* const SectionModelStorage = RootViewModel->CastDynamic<FSectionModelStorageExtension>())
		{
			if (const TSharedPtr<FSectionModel> SectionModel = SectionModelStorage->FindModelForSection(Section))
			{
				Selection->TrackArea.Select(SectionModel);

				if (const TViewModelPtr<IOutlinerExtension> LinkedOutlinerItem = SectionModel->GetLinkedOutlinerItem())
				{
					if (OutlinerView.IsValid())
					{
						OutlinerView->RequestScrollIntoView(LinkedOutlinerItem);
					}
				}
			}
		}
	}

	if (!ModifierKeys.IsAltDown())
	{
		ToolSettings->SetSyncSelectionToNavigationTool(bPreviouslySyncingSelection, /*bInSaveConfig=*/false);
	}
}

ISequencer& FNavigationToolScopedSelection::GetSequencer() const
{
	return Sequencer;
}

} // namespace UE::SequenceNavigator
