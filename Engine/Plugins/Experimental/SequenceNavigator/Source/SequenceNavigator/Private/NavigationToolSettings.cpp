// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavigationToolSettings.h"
#include "ISettingsModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NavigationToolSettings)

UNavigationToolSettings::UNavigationToolSettings()
{
	CategoryName = TEXT("Sequencer");
	SectionName = TEXT("Sequence Navigator");
}

void UNavigationToolSettings::OpenEditorSettings()
{
	if (ISettingsModule* const SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>(TEXT("Settings")))
	{
		if (const UNavigationToolSettings* const Settings = GetDefault<UNavigationToolSettings>())
		{
			SettingsModule->ShowViewer(Settings->GetContainerName(), Settings->GetCategoryName(), Settings->GetSectionName());
		}
	}
}

bool UNavigationToolSettings::ShouldUseMutedHierarchy() const
{
	return bUseMutedHierarchy;
}

void UNavigationToolSettings::SetUseMutedHierarchy(const bool bInUseMutedHierarchy)
{
	if (bUseMutedHierarchy != bInUseMutedHierarchy)
	{
		bUseMutedHierarchy = bInUseMutedHierarchy;
		SaveConfig();
	}
}

bool UNavigationToolSettings::ShouldAutoExpandToSelection() const
{
	return bAutoExpandToSelection;
}

void UNavigationToolSettings::SetAutoExpandToSelection(const bool bInAutoExpandToSelection)
{
	if (bAutoExpandToSelection != bInAutoExpandToSelection)
	{
		bAutoExpandToSelection = bInAutoExpandToSelection;
		SaveConfig();
	}
}

bool UNavigationToolSettings::ShouldAlwaysShowLockState() const
{
	return bAlwaysShowLockState;
}

void UNavigationToolSettings::SetAlwaysShowLockState(const bool bInAlwaysShowLockState)
{
	if (bAlwaysShowLockState != bInAlwaysShowLockState)
	{
		bAlwaysShowLockState = bInAlwaysShowLockState;
		SaveConfig();
	}
}

void UNavigationToolSettings::ToggleViewModeSupport(ENavigationToolItemViewMode& InOutViewMode, const ENavigationToolItemViewMode InFlags)
{
	if (EnumHasAnyFlags(InOutViewMode, InFlags))
	{
		EnumRemoveFlags(InOutViewMode, InFlags);
	}
	else
	{
		EnumAddFlags(InOutViewMode, InFlags);
	}
}

void UNavigationToolSettings::ToggleItemDefaultViewModeSupport(const ENavigationToolItemViewMode InFlags)
{
	ToggleViewModeSupport(ItemDefaultViewMode, InFlags);
}

void UNavigationToolSettings::ToggleItemProxyViewModeSupport(const ENavigationToolItemViewMode InFlags)
{
	ToggleViewModeSupport(ItemProxyViewMode, InFlags);
}

bool UNavigationToolSettings::ShouldUseShortNames() const
{
	return bUseShortNames;
}

void UNavigationToolSettings::SetUseShortNames(const bool bInUseShortNames)
{
	if (bUseShortNames != bInUseShortNames)
	{
		bUseShortNames = bInUseShortNames;
		SaveConfig();
	}
}

bool UNavigationToolSettings::ShouldApplyDefaultColumnView() const
{
	return bApplyDefaultColumnView;
}

void UNavigationToolSettings::SetApplyDefaultColumnView(const bool bInApplyDefaultColumnView)
{
	if (bApplyDefaultColumnView != bInApplyDefaultColumnView)
	{
		bApplyDefaultColumnView = bInApplyDefaultColumnView;
		SaveConfig();
	}
}

ENavigationToolItemViewMode UNavigationToolSettings::GetItemDefaultViewMode() const
{
	return ItemDefaultViewMode;
}

ENavigationToolItemViewMode UNavigationToolSettings::GetItemProxyViewMode() const
{
	return ItemProxyViewMode;
}

FNavigationToolColumnView* UNavigationToolSettings::FindCustomColumnView(const FText& InColumnViewName)
{
	for (FNavigationToolColumnView& ColumnView : CustomColumnViews)
	{
		if (ColumnView.ViewName.EqualTo(InColumnViewName))
		{
			return &ColumnView;
		}
	}
	return nullptr;
}

void UNavigationToolSettings::SetBuiltInFilterEnabled(const FName InFilterName, const bool bInEnabled)
{
	if (InFilterName.IsNone())
	{
		return;
	}

	if (bInEnabled)
	{
		bool bAlreadyExists = false;
		EnabledBuiltInFilters.Add(InFilterName, &bAlreadyExists);
		if (!bAlreadyExists)
		{
			SaveConfig();
		}
	}
	else
	{
		if (EnabledBuiltInFilters.Contains(InFilterName))
		{
			EnabledBuiltInFilters.Remove(InFilterName);
			SaveConfig();
		}
	}
}

FSequencerFilterBarConfig& UNavigationToolSettings::FindOrAddFilterBar(const FName InIdentifier, const bool bInSaveConfig)
{
	FSequencerFilterBarConfig& FilterBarSettings = FilterBars.FindOrAdd(InIdentifier);

	if (bInSaveConfig)
	{
		SaveConfig();
	}

	return FilterBarSettings;
}

FSequencerFilterBarConfig* UNavigationToolSettings::FindFilterBar(const FName InIdentifier)
{
	return FilterBars.Find(InIdentifier);
}

bool UNavigationToolSettings::RemoveFilterBar(const FName InIdentifier)
{
	const int32 RemovedCount = FilterBars.Remove(InIdentifier);
	const bool bRemoveSuccess = RemovedCount > 0;

	if (bRemoveSuccess)
	{
		SaveConfig();
	}

	return bRemoveSuccess;
}

bool UNavigationToolSettings::ShouldAutoExpandNodesOnFilterPass() const
{
	return bAutoExpandNodesOnFilterPass;
}

void UNavigationToolSettings::SetAutoExpandNodesOnFilterPass(const bool bInAutoExpand)
{
	if (bAutoExpandNodesOnFilterPass != bInAutoExpand)
	{
		bAutoExpandNodesOnFilterPass = bInAutoExpand;
		SaveConfig();
	}
}

bool UNavigationToolSettings::ShouldUseFilterSubmenusForCategories() const
{
	return bUseFilterSubmenusForCategories;
}

void UNavigationToolSettings::SetUseFilterSubmenusForCategories(const bool bInUseFilterSubmenusForCategories)
{
	if (bUseFilterSubmenusForCategories != bInUseFilterSubmenusForCategories)
	{
		bUseFilterSubmenusForCategories = bInUseFilterSubmenusForCategories;
		SaveConfig();
	}
}

bool UNavigationToolSettings::IsFilterBarVisible() const
{
	return bFilterBarVisible;
}

void UNavigationToolSettings::SetFilterBarVisible(const bool bInVisible)
{
	if (bFilterBarVisible != bInVisible)
	{
		bFilterBarVisible = bInVisible;
		SaveConfig();
	}
}

EFilterBarLayout UNavigationToolSettings::GetFilterBarLayout() const
{
	return LastFilterBarLayout;
}

void UNavigationToolSettings::SetFilterBarLayout(const EFilterBarLayout InLayout)
{
	if (LastFilterBarLayout != InLayout)
	{
		LastFilterBarLayout = InLayout;
		SaveConfig();
	}
}

float UNavigationToolSettings::GetLastFilterBarSizeCoefficient() const
{
	return FMath::Max<float>(LastFilterBarSizeCoefficient, 0.05f);
}

void UNavigationToolSettings::SetLastFilterBarSizeCoefficient(const float bInSizeCoefficient, const bool bInSaveConfig)
{
	if (LastFilterBarSizeCoefficient != bInSizeCoefficient)
	{
		LastFilterBarSizeCoefficient = bInSizeCoefficient;

		if (bInSaveConfig)
		{
			SaveConfig();
		}
	}
}

bool UNavigationToolSettings::ShouldSyncSelectionToNavigationTool() const
{
	return bSyncSelectionToNavigationTool;
}

void UNavigationToolSettings::SetSyncSelectionToNavigationTool(const bool bInSync, const bool bInSaveConfig)
{
	if (bSyncSelectionToNavigationTool != bInSync)
	{
		bSyncSelectionToNavigationTool = bInSync;
		if (bInSaveConfig)
		{
			SaveConfig();
		}
	}
}

bool UNavigationToolSettings::ShouldSyncSelectionToSequencer() const
{
	return bSyncSelectionToSequencer;
}

void UNavigationToolSettings::SetSyncSelectionToSequencer(const bool bInSync, const bool bInSaveConfig)
{
	if (bSyncSelectionToSequencer != bInSync)
	{
		bSyncSelectionToSequencer = bInSync;
		if (bInSaveConfig)
		{
			SaveConfig();
		}
	}
}
