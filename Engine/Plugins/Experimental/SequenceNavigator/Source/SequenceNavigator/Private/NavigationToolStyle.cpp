// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavigationToolStyle.h"
#include "Brushes/SlateImageBrush.h"
#include "Brushes/SlateRoundedBoxBrush.h"
#include "Fonts/SlateFontInfo.h"
#include "Interfaces/IPluginManager.h"
#include "Layout/Margin.h"
#include "Misc/Paths.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/StyleColors.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"

FNavigationToolStyle::FNavigationToolStyle()
	: FSlateStyleSet(TEXT("NavigationToolStyle"))
{
	SetParentStyleName(FAppStyle::GetAppStyleSetName());

	const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME);
	check(Plugin.IsValid());

	FSlateStyleSet::SetContentRoot(Plugin->GetBaseDir() / TEXT("Resources"));
	FSlateStyleSet::SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));

	static const FVector2D Icon16x16(16.0f, 16.0f);
	static const FVector2D Icon20x20(20.0f, 20.0f);

	Set("Icon.ToolBar", new IMAGE_BRUSH_SVG("Icons/SequenceNavigator_20", Icon20x20));
	Set("Icon.Tab", new IMAGE_BRUSH_SVG("Icons/SequenceNavigator_16", Icon16x16));

	const FTableRowStyle& AlternatingTableRowStyle = FAppStyle::GetWidgetStyle<FTableRowStyle>(TEXT("TableView.AlternatingRow"));
	Set("TableViewRow", FTableRowStyle(AlternatingTableRowStyle));

	FSpinBoxStyle FrameTimeSpinStyle = FAppStyle::GetWidgetStyle<FSpinBoxStyle>(TEXT("SpinBox"));
	FrameTimeSpinStyle.ForegroundColor = FStyleColors::Foreground;
	Set("SpinBox", FrameTimeSpinStyle);

	FSpinBoxStyle StartTimeSpinStyle = FrameTimeSpinStyle
		.SetForegroundColor(FStyleColors::AccentGreen);
	Set("SpinBox.InTime", StartTimeSpinStyle);

	FSpinBoxStyle EndTimeSpinStyle = FrameTimeSpinStyle
		.SetForegroundColor(FStyleColors::AccentRed);
	Set("SpinBox.OutTime", EndTimeSpinStyle);

	FEditableTextBoxStyle NonEditableTextBoxStyle = FEditableTextBoxStyle()
		.SetPadding(FMargin(4.f))
		.SetForegroundColor(FSlateColor::UseSubduedForeground())
		.SetBackgroundImageNormal(CORE_BOX_BRUSH("Graph/CommonWidgets/TextBox", FMargin(4.0f/16.0f)))
		.SetBackgroundImageHovered(CORE_BOX_BRUSH("Graph/CommonWidgets/TextBox_Hovered", FMargin(4.0f/16.0f)))
		.SetBackgroundImageFocused(CORE_BOX_BRUSH("Graph/CommonWidgets/TextBox_Hovered", FMargin(4.0f/16.0f)))
		.SetBackgroundImageReadOnly(FSlateRoundedBoxBrush(FStyleColors::Background, 4.0f, FStyleColors::InputOutline, 1.0f));
	Set("NonEditableTextBox", NonEditableTextBoxStyle);

	Set("Item.Marker.Icon", new CORE_IMAGE_BRUSH_SVG("Sequencer/Marker_16", Icon16x16));

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FNavigationToolStyle::~FNavigationToolStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}
