// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavigationToolTab.h"
#include "Framework/Docking/TabManager.h"
#include "INavigationTool.h"
#include "INavigationToolView.h"
#include "ISequencer.h"
#include "NavigationToolStyle.h"
#include "SequencerSettings.h"
#include "Toolkits/IToolkitHost.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

#define LOCTEXT_NAMESPACE "NavigationToolTab"

namespace UE::SequenceNavigator
{

FNavigationToolTab::FNavigationToolTab(INavigationTool& InOwnerTool)
	: OwnerTool(InOwnerTool)
{
}

void FNavigationToolTab::Init()
{
	bShuttingDown = false;

	RegisterToolTab();

	USequencerSettings* const SequencerSettings = GetSequencerSettings();
    if (SequencerSettings && SequencerSettings->IsNavigationToolVisible())
    {
    	if (const TSharedPtr<FTabManager> TabManager = GetTabManager())
    	{
    		TabManager->TryInvokeTab(NavigationToolTabId);
    	}
    }
}

void FNavigationToolTab::Shutdown()
{
	bShuttingDown = true;

	CloseToolTab();
	UnregisterToolTab();
}

bool FNavigationToolTab::ShouldShowToolTab() const
{
	if (USequencerSettings* const SequencerSettings = GetSequencerSettings())
	{
		return SequencerSettings->IsNavigationToolVisible();
	}
	return false;
}

bool FNavigationToolTab::IsToolTabVisible() const
{
	if (const TSharedPtr<FTabManager> TabManager = GetTabManager())
	{
		const TSharedPtr<SDockTab> LiveTab = TabManager->FindExistingLiveTab(NavigationToolTabId);
		return LiveTab.IsValid();
	}
	return false;
}

void FNavigationToolTab::ShowHideToolTab(const bool bInVisible)
{
	if (bInVisible)
	{
		if (const TSharedPtr<FTabManager> TabManager = GetTabManager())
		{
			TabManager->TryInvokeTab(NavigationToolTabId);
		}
	}
	else
	{
		CloseToolTab();
	}
}

void FNavigationToolTab::ToggleToolTabVisible()
{
	ShowHideToolTab(!IsToolTabVisible());
}

FName FNavigationToolTab::StaticToolTabId(const ISequencer& InSequencer)
{
	const FString DefaultTabId = TEXT("NavigationTool");

	if (const USequencerSettings* const SequencerSettings = InSequencer.GetSequencerSettings())
	{
		const FString TabName = DefaultTabId + TEXT(".") + SequencerSettings->GetName();
		return *TabName;
	}

	return *DefaultTabId;
}

TSharedPtr<IToolkitHost> FNavigationToolTab::GetToolkitHost() const
{
	if (const TSharedPtr<ISequencer> SequencerHost = OwnerTool.GetSequencer())
	{
		return SequencerHost->GetToolkitHost();
	}
	return nullptr;
}

TSharedPtr<FTabManager> FNavigationToolTab::GetTabManager() const
{
	if (const TSharedPtr<IToolkitHost> ToolkitHost = GetToolkitHost())
	{
		return ToolkitHost->GetTabManager();
	}
	return nullptr;
}

USequencerSettings* FNavigationToolTab::GetSequencerSettings() const
{
	if (const TSharedPtr<ISequencer> SequencerHost = OwnerTool.GetSequencer())
	{
		return SequencerHost->GetSequencerSettings();
	}
	return nullptr;
}

void FNavigationToolTab::RegisterToolTab()
{
	if (NavigationToolTabId != NAME_None)
	{
		return;
	}

	const TSharedPtr<ISequencer> Sequencer = OwnerTool.GetSequencer();
	if (!Sequencer.IsValid())
	{
		return;
	}

	const TSharedPtr<FTabManager> TabManager = GetTabManager();
	if (!TabManager.IsValid())
	{
		return;
	}

	if (TabManager->HasTabSpawner(NavigationToolTabId))
	{
		return;
	}

	NavigationToolTabId = StaticToolTabId(*Sequencer);

	TabManager->RegisterTabSpawner(NavigationToolTabId, FOnSpawnTab::CreateSP(this, &FNavigationToolTab::SpawnToolTab))
		.SetGroup(WorkspaceMenu::GetMenuStructure().GetLevelEditorCinematicsCategory())
		.SetDisplayName(LOCTEXT("NavigationToolTab", "Sequence Nav"))
		.SetIcon(FSlateIcon(FNavigationToolStyle::Get().GetStyleSetName(), TEXT("Icon.Tab")));
}

void FNavigationToolTab::UnregisterToolTab()
{
	if (const TSharedPtr<FTabManager> TabManager = GetTabManager())
	{
		TabManager->UnregisterTabSpawner(NavigationToolTabId);
	}
}

TSharedRef<SDockTab> FNavigationToolTab::SpawnToolTab(const FSpawnTabArgs& InArgs)
{
	const TSharedPtr<INavigationToolView> RecentToolView = OwnerTool.GetMostRecentToolView();
	check(RecentToolView.IsValid());

	const TSharedPtr<SWidget> ToolWidget = RecentToolView->GetToolWidget();
	check(ToolWidget.IsValid());

	const TSharedRef<SDockTab> ToolTab = SNew(SDockTab)
		.OnTabClosed_Lambda([this](const TSharedRef<SDockTab> InDockTab)
			{
				USequencerSettings* const Settings = GetSequencerSettings();
				if (Settings && !bShuttingDown)
				{
					Settings->SetNavigationToolVisible(false);
				}

				VisibilityChangedDelegate.Broadcast(false);
			})
		[
			ToolWidget.ToSharedRef()
		];

	if (USequencerSettings* const SequencerSettings = GetSequencerSettings())
	{
		SequencerSettings->SetNavigationToolVisible(true);
	}

	VisibilityChangedDelegate.Broadcast(true);

	OwnerTool.Refresh();

	return ToolTab;
}

void FNavigationToolTab::CloseToolTab()
{
	const TSharedPtr<FTabManager> TabManager = GetTabManager();
	if (!TabManager.IsValid())
	{
		return;
	}

	const TSharedPtr<SDockTab> LiveTab = TabManager->FindExistingLiveTab(NavigationToolTabId);
	if (!LiveTab.IsValid())
	{
		return;
	}

	LiveTab->RequestCloseTab();
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
