// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavigationToolView.h"
#include "Columns/NavigationToolColumn.h"
#include "Columns/NavigationToolColumnExtender.h"
#include "ContentBrowserModule.h"
#include "DragDropOps/NavigationToolItemDragDropOp.h"
#include "Filters/Filters/NavigationToolBuiltInFilter.h"
#include "Filters/Filters/NavigationToolFilterBase.h"
#include "Filters/NavigationToolFilterBar.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IContentBrowserSingleton.h"
#include "INavigationTool.h"
#include "ISequencer.h"
#include "Items/NavigationToolActor.h"
#include "Items/NavigationToolComponent.h"
#include "Items/NavigationToolSequence.h"
#include "Items/NavigationToolTreeRoot.h"
#include "LevelSequence.h"
#include "Menus/NavigationToolItemContextMenu.h"
#include "Misc/MessageDialog.h"
#include "NavigationTool.h"
#include "NavigationToolCommands.h"
#include "NavigationToolExtender.h"
#include "NavigationToolSettings.h"
#include "Providers/NavigationToolProvider.h"
#include "SequenceNavigatorLog.h"
#include "Styling/SlateTypes.h"
#include "Utils/NavigationToolMiscUtils.h"
#include "Widgets/ModalTextInputDialog.h"
#include "Widgets/SNavigationToolView.h"
#include "Widgets/SNavigationToolTreeView.h"
#include "Widgets/SNullWidget.h"

#define LOCTEXT_NAMESPACE "NavigationToolView"

namespace UE::SequenceNavigator
{

using namespace Sequencer;

FNavigationToolView::FNavigationToolView(FPrivateToken)
	: WeakToolSettings(GetMutableDefault<UNavigationToolSettings>())
	, ItemContextMenu(MakeShared<FNavigationToolItemContextMenu>())
{
}

FNavigationToolView::~FNavigationToolView()
{
	if (UObjectInitialized())
	{
		if (UNavigationToolSettings* const ToolSettings = WeakToolSettings.Get())
		{
			ToolSettings->OnSettingChanged().RemoveAll(this);
		}
	}
}

void FNavigationToolView::Init(const TSharedRef<FNavigationTool>& InTool, const TSharedPtr<FUICommandList>& InBaseCommandList)
{
	const TSharedPtr<ISequencer> Sequencer = InTool->GetSequencer();
	if (!Sequencer.IsValid())
	{
		return;
	}

	WeakTool = InTool;

	BindCommands(InBaseCommandList);

	FilterBar = MakeShared<FNavigationToolFilterBar>(*InTool);
	FilterBar->Init();
	FilterBar->BindCommands(GetBaseCommandList());
	
	const auto IssueRebuild = [this]
	{
		if (const TSharedPtr<SNavigationToolView> ToolWidget = StaticCastSharedPtr<SNavigationToolView>(GetToolWidget()))
		{
			ToolWidget->RebuildWidget();
		}
	};
	FilterBar->OnVisibilityChanged().AddSPLambda(this, [IssueRebuild](auto){ IssueRebuild(); });
	FilterBar->OnLayoutChanged().AddSPLambda(this, [IssueRebuild](auto){ IssueRebuild(); });
	FilterBar->OnActivationStateChanged().AddSPLambda(this, [IssueRebuild]{ IssueRebuild(); });

	if (UNavigationToolSettings* const ToolSettings = WeakToolSettings.Get())
	{
		ToolSettings->OnSettingChanged().AddSP(this, &FNavigationToolView::OnToolSettingsChanged);
	}

	ToolViewWidget = SNew(SNavigationToolView, SharedThis(this));

	UpdateRecentViews();
}

void FNavigationToolView::CreateColumns(const TSharedRef<FNavigationToolProvider>& InProvider)
{
	const TSharedPtr<FNavigationTool> Tool = WeakTool.Pin();
	if (!Tool.IsValid())
	{
		return;
	}

	FNavigationToolColumnExtender ColumnExtender;
	InProvider->OnExtendColumns(ColumnExtender);

	// Sort and re-cache columns
	const TArray<TSharedPtr<FNavigationToolColumn>>& ColumnsToAdd = ColumnExtender.GetColumns();

	for (const TSharedPtr<FNavigationToolColumn>& Column : ColumnsToAdd)
	{
		const FName ColumnId = Column->GetColumnId();
		if (!Columns.Contains(ColumnId))
		{
			Columns.Add(ColumnId, Column);
		}
	}

	if (ToolViewWidget.IsValid())
	{
		ToolViewWidget->ReconstructColumns();
	}
}

void FNavigationToolView::CreateDefaultColumnViews(const TSharedRef<FNavigationToolProvider>& InProvider)
{
	if (UNavigationToolSettings* const ToolSettings = WeakToolSettings.Get())
	{
		InProvider->OnExtendColumnViews(ToolSettings->GetCustomColumnViews());
		ToolSettings->SaveConfig();
	}
}

TSharedRef<FNavigationToolView> FNavigationToolView::CreateInstance(const int32 InToolViewId
	, const TSharedRef<FNavigationTool>& InTool
	, const TSharedPtr<FUICommandList>& InBaseCommandList)
{
	const TSharedRef<FNavigationToolView> Instance = MakeShared<FNavigationToolView>(FPrivateToken{});
	Instance->ToolViewId = InToolViewId;
	Instance->Init(InTool, InBaseCommandList);
	return Instance;
}

void FNavigationToolView::PostLoad()
{
	if (ToolViewWidget.IsValid())
	{
		ToolViewWidget->ReconstructColumns();
	}
}

void FNavigationToolView::OnToolSettingsChanged(UObject* const InObject, FPropertyChangedEvent& InPropertyChangedEvent)
{
	RefreshTool(false);
}

void FNavigationToolView::Tick(const float InDeltaTime)
{
	if (bRefreshRequested)
	{
		bRefreshRequested = false;
		Refresh();
	}

	for (const TPair<FName, TSharedPtr<INavigationToolColumn>>& Pair : Columns)
	{
		if (const TSharedPtr<INavigationToolColumn>& Column = Pair.Value)
		{
			Column->Tick(InDeltaTime);
		}
	}

	// Check if we have pending items to rename and we are not currently renaming an item
	if (bRenamingItems && WeakItemsRemainingRename.Num() > 0 && !WeakCurrentItemRenaming.Pin().IsValid())
	{
		WeakCurrentItemRenaming = WeakItemsRemainingRename[0];
		WeakItemsRemainingRename.RemoveAt(0);

		if (const FNavigationToolViewModelPtr CurrentItemRenaming = WeakCurrentItemRenaming.Pin())
		{
			CurrentItemRenaming->OnRenameAction().AddSP(this, &FNavigationToolView::OnItemRenameAction);
			CurrentItemRenaming->OnRenameAction().Broadcast(ENavigationToolRenameAction::Requested, SharedThis(this));
		}
	}

	if (bRequestedRename)
	{
		bRequestedRename = false;
		RenameSelected();
	}
}

void FNavigationToolView::BindCommands(const TSharedPtr<FUICommandList>& InBaseCommandList)
{
	const FGenericCommands& GenericCommands = FGenericCommands::Get();
	const FNavigationToolCommands& ToolCommands = FNavigationToolCommands::Get();

	ViewCommandList = MakeShared<FUICommandList>();

	if (InBaseCommandList.IsValid())
	{
		InBaseCommandList->Append(ViewCommandList.ToSharedRef());
	}

	ViewCommandList->MapAction(ToolCommands.OpenToolSettings
		, FExecuteAction::CreateStatic(&UNavigationToolSettings::OpenEditorSettings));

	ViewCommandList->MapAction(ToolCommands.Refresh
		, FExecuteAction::CreateSP(this, &FNavigationToolView::RefreshTool, true));

	ViewCommandList->MapAction(GenericCommands.Rename
		, FExecuteAction::CreateSP(this, &FNavigationToolView::RenameSelected)
		, FCanExecuteAction::CreateSP(this, &FNavigationToolView::CanRenameSelected));

	ViewCommandList->MapAction(GenericCommands.Delete
		, FExecuteAction::CreateSP(this, &FNavigationToolView::DeleteSelected)
		, FCanExecuteAction::CreateSP(this, &FNavigationToolView::CanDeleteSelected));

	ViewCommandList->MapAction(GenericCommands.Duplicate
		, FExecuteAction::CreateSP(this, &FNavigationToolView::DuplicateSelected)
		, FCanExecuteAction::CreateSP(this, &FNavigationToolView::CanDuplicateSelected));

	ViewCommandList->MapAction(ToolCommands.SelectAllChildren
		, FExecuteAction::CreateSP(this, &FNavigationToolView::SelectChildren, true)
		, FCanExecuteAction::CreateSP(this, &FNavigationToolView::CanSelectChildren));

	ViewCommandList->MapAction(ToolCommands.SelectImmediateChildren
		, FExecuteAction::CreateSP(this, &FNavigationToolView::SelectChildren, false)
		, FCanExecuteAction::CreateSP(this, &FNavigationToolView::CanSelectChildren));

	ViewCommandList->MapAction(ToolCommands.SelectParent
		, FExecuteAction::CreateSP(this, &FNavigationToolView::SelectParent)
		, FCanExecuteAction::CreateSP(this, &FNavigationToolView::CanSelectParent));

	ViewCommandList->MapAction(ToolCommands.SelectFirstChild
		, FExecuteAction::CreateSP(this, &FNavigationToolView::SelectFirstChild)
		, FCanExecuteAction::CreateSP(this, &FNavigationToolView::CanSelectFirstChild));

	ViewCommandList->MapAction(ToolCommands.SelectNextSibling
		, FExecuteAction::CreateSP(this, &FNavigationToolView::SelectSibling, +1)
		, FCanExecuteAction::CreateSP(this, &FNavigationToolView::CanSelectSibling));

	ViewCommandList->MapAction(ToolCommands.SelectPreviousSibling
		, FExecuteAction::CreateSP(this, &FNavigationToolView::SelectSibling, -1)
		, FCanExecuteAction::CreateSP(this, &FNavigationToolView::CanSelectSibling));

	ViewCommandList->MapAction(ToolCommands.ExpandAll
		, FExecuteAction::CreateSP(this, &FNavigationToolView::ExpandAll)
		, FCanExecuteAction::CreateSP(this, &FNavigationToolView::CanExpandAll));

	ViewCommandList->MapAction(ToolCommands.CollapseAll
		, FExecuteAction::CreateSP(this, &FNavigationToolView::CollapseAll)
		, FCanExecuteAction::CreateSP(this, &FNavigationToolView::CanCollapseAll));

	ViewCommandList->MapAction(ToolCommands.ScrollNextSelectionIntoView
		, FExecuteAction::CreateSP(this, &FNavigationToolView::ScrollNextIntoView)
		, FCanExecuteAction::CreateSP(this, &FNavigationToolView::CanScrollNextIntoView));

	ViewCommandList->MapAction(ToolCommands.ToggleMutedHierarchy
		, FExecuteAction::CreateSP(this, &FNavigationToolView::ToggleMutedHierarchy)
		, FCanExecuteAction::CreateSP(this, &FNavigationToolView::CanToggleMutedHierarchy)
		, FIsActionChecked::CreateSP(this, &FNavigationToolView::IsMutedHierarchyActive));

	ViewCommandList->MapAction(ToolCommands.ToggleAutoExpandToSelection
		, FExecuteAction::CreateSP(this, &FNavigationToolView::ToggleAutoExpandToSelection)
		, FCanExecuteAction::CreateSP(this, &FNavigationToolView::CanToggleAutoExpandToSelection)
		, FIsActionChecked::CreateSP(this, &FNavigationToolView::ShouldAutoExpandToSelection));

	ViewCommandList->MapAction(ToolCommands.ToggleShortNames
		, FExecuteAction::CreateSP(this, &FNavigationToolView::ToggleUseShortNames)
		, FCanExecuteAction::CreateSP(this, &FNavigationToolView::CanToggleUseShortNames)
		, FIsActionChecked::CreateSP(this, &FNavigationToolView::ShouldUseShortNames));

	ViewCommandList->MapAction(ToolCommands.ResetVisibleColumnSizes
		, FExecuteAction::CreateSP(this, &FNavigationToolView::ResetVisibleColumnSizes)
		, FCanExecuteAction::CreateSP(this, &FNavigationToolView::CanResetAllColumnSizes));

	ViewCommandList->MapAction(ToolCommands.SaveCurrentColumnView
		, FExecuteAction::CreateSP(this, &FNavigationToolView::SaveNewCustomColumnView));

	ViewCommandList->MapAction(ToolCommands.FocusSingleSelection
		, FExecuteAction::CreateSP(this, &FNavigationToolView::FocusSingleSelection)
		, FCanExecuteAction::CreateSP(this, &FNavigationToolView::CanFocusSingleSelection));

	ViewCommandList->MapAction(ToolCommands.FocusInContentBrowser
		, FExecuteAction::CreateSP(this, &FNavigationToolView::FocusInContentBrowser)
		, FCanExecuteAction::CreateSP(this, &FNavigationToolView::CanFocusInContentBrowser));
}

TSharedPtr<FUICommandList> FNavigationToolView::GetBaseCommandList() const
{
	if (const TSharedPtr<FNavigationTool> Tool = WeakTool.Pin())
	{
		return Tool->GetBaseCommandList();
	}
	return nullptr;
}

void FNavigationToolView::UpdateRecentViews()
{
	if (const TSharedPtr<FNavigationTool> Tool = WeakTool.Pin())
	{
		Tool->UpdateRecentToolViews(ToolViewId);
	}
}

bool FNavigationToolView::IsMostRecentToolView() const
{
	return WeakTool.IsValid() && WeakTool.Pin()->GetMostRecentToolView().Get() == this;
}

TSharedPtr<ISequencer> FNavigationToolView::GetSequencer() const
{
	if (const TSharedPtr<INavigationTool> Tool = GetOwnerTool())
	{
		return Tool->GetSequencer();
	}
	return nullptr;
}

TSharedPtr<INavigationTool> FNavigationToolView::GetOwnerTool() const
{
	return WeakTool.Pin();
}

TSharedPtr<SWidget> FNavigationToolView::GetToolWidget() const
{
	return ToolViewWidget;
}

TSharedPtr<SWidget> FNavigationToolView::CreateItemContextMenu()
{
	return ItemContextMenu->CreateMenu(SharedThis(this), WeakSelectedItems);
}

bool FNavigationToolView::ShouldShowColumnByDefault(const TSharedPtr<INavigationToolColumn>& InColumn) const
{
	if (!InColumn.IsValid())
	{
		return false;
	}

	const TSharedPtr<FNavigationTool> Tool = WeakTool.Pin();
	if (!Tool.IsValid())
	{
		return false;
	}

	bool bShouldShow = IsColumnVisible(InColumn);

	bShouldShow |= InColumn->ShouldShowColumnByDefault();

	return bShouldShow;
}

void FNavigationToolView::RequestRefresh()
{
	bRefreshRequested = true;
}

void FNavigationToolView::Refresh()
{
	// Filter items before doing anything else so we can reliably use the filter data cache.
	// For example, in cases where a FNavigationToolAddItem is executed and a new item is added
	// to the tree, UpdateRootVisibleItems() below uses the filter data to show/hide items.
	bFilterUpdateRequested = true;
	UpdateFilters();

	UpdateRootVisibleItems();

	UpdateItemExpansions();
	
	if (ToolViewWidget.IsValid())
	{
		ToolViewWidget->RequestTreeRefresh();
	}
	
	OnToolViewRefreshed.Broadcast();
}

void FNavigationToolView::SetKeyboardFocus()
{
	if (ToolViewWidget.IsValid())
	{
		ToolViewWidget->SetKeyboardFocus();
	}
}

void FNavigationToolView::UpdateRootVisibleItems()
{
	WeakRootVisibleItems.Reset();
	WeakReadOnlyItems.Reset();
	
	if (WeakTool.IsValid())
	{
		GetChildrenOfItem(WeakTool.Pin()->GetTreeRoot(), WeakRootVisibleItems);
	}
}

void FNavigationToolView::UpdateItemExpansions()
{
	for (const FNavigationToolViewModelWeakPtr& WeakRootVisibleItem : WeakRootVisibleItems)
	{
		const FNavigationToolViewModelPtr RootVisibleItem = WeakRootVisibleItem.Pin();
		if (!RootVisibleItem.IsValid())
		{
			continue;
		}

		for (const FNavigationToolViewModelPtr& Item : RootVisibleItem.AsModel()->GetDescendantsOfType<INavigationToolItem>())
		{
			const ENavigationToolItemFlags ItemFlags = GetViewItemFlags(Item);
			const bool bHasExpandedFlag = EnumHasAnyFlags(ItemFlags, ENavigationToolItemFlags::Expanded);
			SetItemExpansion(Item, bHasExpandedFlag);
		}
	}

	if (ToolViewWidget.IsValid())
	{
		for (const FNavigationToolViewModelWeakPtr& WeakRootVisibleItem : WeakRootVisibleItems)
		{
			const FNavigationToolViewModelPtr RootVisibleItem = WeakRootVisibleItem.Pin();
			if (!RootVisibleItem.IsValid())
			{
				continue;
			}

			for (const FNavigationToolViewModelPtr& Item : RootVisibleItem.AsModel()->GetDescendantsOfType<INavigationToolItem>())
			{
				ToolViewWidget->UpdateItemExpansions(Item);
			}
		}
	}

	/*TArray<FNavigationToolViewModelWeakPtr> Items = WeakRootVisibleItems;

	while (Items.Num() > 0)
	{
		if (const FNavigationToolViewModelPtr Item = Items.Pop().Pin())
		{
			const ENavigationToolItemFlags ItemFlags = GetViewItemFlags(Item);
			const bool bHasExpandedFlag = EnumHasAnyFlags(ItemFlags, ENavigationToolItemFlags::Expanded);
			SetItemExpansion(Item, bHasExpandedFlag);
			Items.Append(Item->GetChildren());
		}
	}

	Items = WeakRootVisibleItems;

	while (Items.Num() > 0)
	{
		if (const FNavigationToolViewModelPtr Item = Items.Pop().Pin())
		{
			if (ToolViewWidget.IsValid())
			{
				ToolViewWidget->UpdateItemExpansions(Item);
			}
			Items.Append(Item->GetChildren());
		}
	}*/
}

void FNavigationToolView::NotifyObjectsReplaced()
{
	if (ToolViewWidget.IsValid())
	{
		ToolViewWidget->Invalidate(EInvalidateWidgetReason::Paint);	
	}
}

FNavigationToolViewModelPtr FNavigationToolView::GetRootItem() const
{
	if (WeakTool.IsValid())
	{
		return WeakTool.Pin()->GetTreeRoot().Pin();
	}
	return nullptr;
}

const TArray<FNavigationToolViewModelWeakPtr>& FNavigationToolView::GetRootVisibleItems() const
{
	return WeakRootVisibleItems;
}

void FNavigationToolView::SaveViewItemFlags(const FNavigationToolViewModelWeakPtr& InWeakItem
	, const ENavigationToolItemFlags InFlags)
{
	const FNavigationToolViewModelPtr Item = InWeakItem.Pin();
	if (!Item.IsValid())
	{
		return;
	}

	const TSharedPtr<FNavigationToolProvider> Provider = Item->GetProvider();
	if (!Provider.IsValid())
	{
		return;
	}

	const TSharedPtr<FNavigationTool> Tool = WeakTool.Pin();
	if (!Tool.IsValid())
	{
		return;
	}

	FNavigationToolSaveState* const SaveState = Provider->GetSaveState(*Tool);
	if (!SaveState)
	{
		UE_LOGF(LogSequenceNavigator, Warning, "SaveViewItemFlags(): Save state is NULL!");
		return;
	}

	if (!SaveState->ToolViewSaveStates.IsValidIndex(ToolViewId))
	{
		UE_LOGF(LogSequenceNavigator, Warning, "SaveViewItemFlags(): Invalid tool view Id: %d", ToolViewId);
		return;
	}

	if (InFlags == ENavigationToolItemFlags::None)
	{
		SaveState->ToolViewSaveStates[ToolViewId].ViewItemFlags.Remove(Item->GetItemId().GetStringId());
	}
	else
	{
		SaveState->ToolViewSaveStates[ToolViewId].ViewItemFlags.Add(Item->GetItemId().GetStringId(), InFlags);
	}
}

ENavigationToolItemFlags FNavigationToolView::GetViewItemFlags(const FNavigationToolViewModelWeakPtr& InWeakItem) const
{
	const FNavigationToolViewModelPtr Item = InWeakItem.Pin();
	if (!Item.IsValid())
	{
		return ENavigationToolItemFlags::None;
	}

	const TSharedPtr<FNavigationToolProvider> Provider = Item->GetProvider();
	if (!Provider.IsValid())
	{
		return ENavigationToolItemFlags::None;
	}

	const TSharedPtr<FNavigationTool> Tool = WeakTool.Pin();
	if (!Tool.IsValid())
	{
		return ENavigationToolItemFlags::None;
	}

	FNavigationToolViewSaveState* const ViewSaveState = Provider->GetViewSaveState(*Tool, ToolViewId);
	if (!ViewSaveState)
	{
		return ENavigationToolItemFlags::None;
	}

	if (const ENavigationToolItemFlags* const OverrideFlags = ViewSaveState->ViewItemFlags.Find(Item->GetItemId().GetStringId()))
	{
		return *OverrideFlags;
	}

	return ENavigationToolItemFlags::None;
}

void FNavigationToolView::GetChildrenOfItem(const FNavigationToolViewModelWeakPtr InWeakItem
	, TArray<FNavigationToolViewModelWeakPtr>& OutWeakChildren) const
{
	static const TSet<FNavigationToolViewModelWeakPtr> EmptySet;
	GetChildrenOfItem(InWeakItem, OutWeakChildren, ENavigationToolItemViewMode::ItemTree, EmptySet);
}

void FNavigationToolView::GetChildrenOfItem(const FNavigationToolViewModelWeakPtr InWeakItem
	, TArray<FNavigationToolViewModelWeakPtr>& OutWeakChildren
	, const ENavigationToolItemViewMode InViewMode
	, const TSet<FNavigationToolViewModelWeakPtr>& InWeakRecursionDisallowedItems) const
{
	const FNavigationToolViewModelPtr Item = InWeakItem.Pin();
	if (!Item.IsValid())
	{
		return;
	}

	const TSharedPtr<FNavigationToolProvider> Provider = Item->GetProvider();
	if (!Provider.IsValid() && Item->GetItemId() != FNavigationToolItemId::RootId)
	{
		UE_LOGF(LogTemp, Warning, "Sequence Navigator Item Id \"%ls\" has no provider, but is a root item!"
			, *Item->GetItemId().GetStringId());
		return;
	}

	for (const FNavigationToolViewModelWeakPtr& WeakChildItem : Item->GetChildren())
	{
		if (!WeakChildItem.Pin().IsValid())
		{
			continue;
		}

		if (ShouldShowItem(WeakChildItem, true, InViewMode))
		{
			// If the current item is visible in outliner, add it to the children
			OutWeakChildren.Add(WeakChildItem);
		}
		else if (!InWeakRecursionDisallowedItems.Contains(WeakChildItem))
		{
			TArray<FNavigationToolViewModelWeakPtr> WeakGrandChildren;

			// For Muted Hierarchy to be in effect, not only does it have to be on
			// but also the item should be shown (without counting the filter pass)
			const bool bShouldUseMutedHierarchy = WeakToolSettings.IsValid() ? WeakToolSettings.Get()->ShouldUseMutedHierarchy() : false;
			const bool bShouldShowItemWithoutFilters = ShouldShowItem(WeakChildItem, false, InViewMode);
			const bool bShouldMuteItem = bShouldUseMutedHierarchy && bShouldShowItemWithoutFilters;

			// If Muted Hierarchy, there might be ONLY grand children that are just visible in other view modes, 
			// so instead of just filtering out the child item, check that there are no grand children from other view modes passing filter tests
			// If it's NOT muted hierarchy, just get the grand children visible in the requested view mode, as this ChildItem is guaranteed to be hidden
			const ENavigationToolItemViewMode ViewModeToUse = bShouldMuteItem ? ENavigationToolItemViewMode::All : InViewMode;

			GetChildrenOfItem(WeakChildItem, WeakGrandChildren, ViewModeToUse, InWeakRecursionDisallowedItems);

			if (!WeakGrandChildren.IsEmpty())
			{
				if (bShouldMuteItem)
				{
					WeakReadOnlyItems.Add(WeakChildItem);	
					OutWeakChildren.Add(WeakChildItem);
				}
				else
				{
					// We can append them knowing that the ViewMode to use is the one passed in and there's no
					// child that leaked from another view mode
					ensure(ViewModeToUse == InViewMode);
					OutWeakChildren.Append(WeakGrandChildren);
				}
			}
		}
	}
}

FLinearColor FNavigationToolView::GetItemBrushColor(const FNavigationToolViewModelPtr InItem) const
{
	if (InItem.IsValid())
	{
		FLinearColor OutColor = InItem->GetItemTintColor();

		// If NextSelectedItemIntoView is valid, it means we're scrolling items into view with Next/Previous, 
		// so Make everything that's not the Current Item a bit more translucent to make the Current Item stand out
		if (WeakSortedSelectedItems.IsValidIndex(NextSelectedItemIntoView)
			&& WeakSortedSelectedItems[NextSelectedItemIntoView] != InItem)
		{
			OutColor.A *= 0.5f;
		}

		return OutColor;
	}

	return FStyleColors::White.GetSpecifiedColor();
}

TArray<FNavigationToolViewModelWeakPtr> FNavigationToolView::GetSelectedItems() const
{
	return WeakSelectedItems;
}

int32 FNavigationToolView::GetViewSelectedItemCount() const
{
	return WeakSelectedItems.Num();
}

int32 FNavigationToolView::CalculateVisibleItemCount() const
{
	TArray<FNavigationToolViewModelWeakPtr> WeakRemainingItems = WeakRootVisibleItems;

	int32 VisibleItemCount = WeakRemainingItems.Num();

	while (WeakRemainingItems.Num() > 0)
	{
		if (const FNavigationToolViewModelPtr Item = WeakRemainingItems.Pop().Pin())
		{
			TArray<FNavigationToolViewModelWeakPtr> WeakChildItems;
			GetChildrenOfItem(Item, WeakChildItems);

			VisibleItemCount += WeakChildItems.Num();

			WeakRemainingItems.Append(MoveTemp(WeakChildItems));
		}
	}

	// Remove the read only items as they are filtered out items that are still shown because of hierarchy viz
	VisibleItemCount -= WeakReadOnlyItems.Num();

	return VisibleItemCount;
}

void FNavigationToolView::SelectItems(TArray<FNavigationToolViewModelWeakPtr> InWeakItems
	, const ENavigationToolItemSelectionFlags InFlags)
{
	const TSet<FNavigationToolViewModelWeakPtr> WeakItemSet(InWeakItems);

	TSet<FNavigationToolViewModelWeakPtr> WeakSeenItems;
	WeakSeenItems.Reserve(InWeakItems.Num());

	// Remove duplicate items
	/*for (TSet<FNavigationToolViewModelWeakPtr>::TIterator Iter(WeakItemSet); Iter; ++Iter)
	{
		if (SeenItems.Contains(*Iter))
		{
			Iter.RemoveCurrent();
		}
		else
		{
			SeenItems.Add(*Iter);
		}
	}*/

	// Add the children of the items given
	if (EnumHasAnyFlags(InFlags, ENavigationToolItemSelectionFlags::IncludeChildren))
	{
		TArray<FNavigationToolViewModelWeakPtr> WeakRemainingChildItems(InWeakItems);
		while (WeakRemainingChildItems.Num() > 0)
		{
			if (FNavigationToolViewModelPtr ChildItem = WeakRemainingChildItems.Pop().Pin())
			{
				TArray<FNavigationToolViewModelWeakPtr> WeakChildren;
				GetChildrenOfItem(ChildItem, WeakChildren);

				InWeakItems.Append(WeakChildren);
				WeakRemainingChildItems.Append(WeakChildren);
			}
		}
	}

	if (EnumHasAnyFlags(InFlags, ENavigationToolItemSelectionFlags::AppendToCurrentSelection))
	{
		// Remove all repeated items to avoid duplicated entries
		WeakSelectedItems.RemoveAll([&WeakSeenItems](const FNavigationToolViewModelWeakPtr& InWeakItem)
			{
				return WeakSeenItems.Contains(InWeakItem);
			});
		TArray<FNavigationToolViewModelWeakPtr> Items = MoveTemp(WeakSelectedItems);
		Items.Append(MoveTemp(InWeakItems));
		InWeakItems = MoveTemp(Items);
	}

	if (!InWeakItems.IsEmpty() && EnumHasAnyFlags(InFlags, ENavigationToolItemSelectionFlags::ScrollIntoView))
	{
		ScrollItemIntoView(InWeakItems[0]);
	}

	const bool bSignalSelectionChange = EnumHasAnyFlags(InFlags, ENavigationToolItemSelectionFlags::SignalSelectionChange);
	SetItemSelectionImpl(MoveTemp(InWeakItems), bSignalSelectionChange);
}

void FNavigationToolView::ClearItemSelection(const bool bInSignalSelectionChange)
{
	SetItemSelectionImpl({}, bInSignalSelectionChange);
}

void FNavigationToolView::SetItemSelectionImpl(TArray<FNavigationToolViewModelWeakPtr>&& InWeakItems, const bool bInSignalSelectionChange)
{
	/*if (ShouldAutoExpandToSelection())
	{
		for (const FNavigationToolViewModelPtr& Item : InItems)
		{
			SetParentItemExpansions(Item, true);
		}
	}*/

	WeakSelectedItems = MoveTemp(InWeakItems);

	if (ToolViewWidget.IsValid())
	{
		ToolViewWidget->SetItemSelection(WeakSelectedItems, bInSignalSelectionChange);
	}
	else if (bInSignalSelectionChange)
	{
		NotifyItemSelectionChanged(WeakSelectedItems, nullptr, true);
	}

	Refresh();
}

void FNavigationToolView::NotifyItemSelectionChanged(const TArray<FNavigationToolViewModelWeakPtr>& InWeakSelectedItems
	, const FNavigationToolViewModelPtr& InItem
	, const bool bInUpdateModeTools)
{
	if (bSyncingItemSelection)
	{
		return;
	}
	TGuardValue<bool> Guard(bSyncingItemSelection, true);

	WeakSelectedItems = InWeakSelectedItems;
	WeakSortedSelectedItems = InWeakSelectedItems;
	NextSelectedItemIntoView = -1;

	FNavigationTool::SortItems(WeakSortedSelectedItems);

	// If we have pending items remaining but we switched selection via navigation, treat it as "I want to rename this too"
	if (bRenamingItems && InItem.IsValid() && InItem != WeakCurrentItemRenaming)
	{
		bRequestedRename = true;
	}

	const TSharedPtr<FNavigationTool> Tool = WeakTool.Pin();
	if (bInUpdateModeTools && Tool.IsValid())
	{
		if (UNavigationToolSettings* const ToolSettings = WeakToolSettings.Get())
		{
			if (ToolSettings->ShouldSyncSelectionToSequencer())
			{
				Tool->SyncSequencerSelection(WeakSelectedItems);
			}
		}

		Tool->SelectItems(WeakSelectedItems, ENavigationToolItemSelectionFlags::None);
	}
}

bool FNavigationToolView::IsItemReadOnly(const FNavigationToolViewModelWeakPtr& InWeakItem) const
{
	return WeakReadOnlyItems.Contains(InWeakItem);
}

bool FNavigationToolView::CanSelectItem(const FNavigationToolViewModelWeakPtr& InWeakItem) const
{
	const FNavigationToolViewModelPtr Item = InWeakItem.Pin();
	const bool bIsSelectable = Item.IsValid() && Item->IsSelectable();
	return bIsSelectable && !IsItemReadOnly(InWeakItem);
}

bool FNavigationToolView::IsItemSelected(const FNavigationToolViewModelWeakPtr& InWeakItem) const
{
	return WeakSelectedItems.Contains(InWeakItem);
}

bool FNavigationToolView::IsItemExpanded(const FNavigationToolViewModelWeakPtr& InWeakItem, const bool bInUseFilter) const
{
	// Don't continue if item should be hidden in view.
	// The tree view still calls OnItemExpansionChanged even if it doesn't contain the item
	// so this preemptive check is needed.
	if (!ShouldShowItem(InWeakItem, bInUseFilter, ENavigationToolItemViewMode::ItemTree))
	{
		return false;
	}

	if (ToolViewWidget.IsValid())
	{
		return ToolViewWidget->IsItemExpanded(InWeakItem);
	}

	return false;
}

void FNavigationToolView::SetItemExpansion(const FNavigationToolViewModelWeakPtr& InWeakItem
	, const bool bInExpand, const bool bInUseFilter)
{
	// Don't continue if the item should be hidden in view.
	// The tree view still calls OnItemExpansionChanged even if it doesn't contain the item
	// so this preemptive check is needed
	if (!ShouldShowItem(InWeakItem, bInUseFilter, ENavigationToolItemViewMode::ItemTree))
	{
		return;
	}

	if (ToolViewWidget.IsValid())
	{
		ToolViewWidget->SetItemExpansion(InWeakItem, bInExpand);
	}
	else
	{
		OnItemExpansionChanged(InWeakItem, bInExpand);
	}
}

void FNavigationToolView::SetItemExpansionRecursive(const FNavigationToolViewModelWeakPtr InWeakItem, const bool bInExpand)
{
	const FNavigationToolViewModelPtr Item = InWeakItem.Pin();
	if (!Item.IsValid())
	{
		return;
	}

	SetItemExpansion(InWeakItem, bInExpand, false);

	for (const FNavigationToolViewModelWeakPtr& WeakChildItem : Item->GetChildren())
	{
		SetItemExpansionRecursive(WeakChildItem, bInExpand);
	}
}

void FNavigationToolView::SetParentItemExpansions(const FNavigationToolViewModelWeakPtr& InWeakItem, const bool bInExpand)
{
	const FNavigationToolViewModelPtr Item = InWeakItem.Pin();
	if (!Item.IsValid())
	{
		return;
	}

	TArray<FNavigationToolViewModelPtr> ItemsToExpand;

	// Don't auto expand at all if there's a parent preventing it
	FNavigationToolViewModelPtr ParentItem = Item->GetParent();
	while (ParentItem.IsValid())
	{
		if (!ParentItem->CanAutoExpand())
		{
			return;
		}
		ItemsToExpand.Add(ParentItem);
		ParentItem = ParentItem->GetParent();
	}

	for (const FNavigationToolViewModelPtr& ItemToExpand : ItemsToExpand)
	{
		SetItemExpansion(ItemToExpand, bInExpand);
	}
}

void FNavigationToolView::OnItemExpansionChanged(const FNavigationToolViewModelWeakPtr InWeakItem, const bool bInIsExpanded)
{
	const FNavigationToolViewModelPtr Item = InWeakItem.Pin();
	if (!Item.IsValid())
	{
		return;
	}

	const ENavigationToolItemFlags CurrentFlags = GetViewItemFlags(InWeakItem);
	
	ENavigationToolItemFlags TargetFlags = CurrentFlags;
	
	if (bInIsExpanded)
	{
		TargetFlags |= ENavigationToolItemFlags::Expanded;
	}
	else
	{
		TargetFlags &= ~ENavigationToolItemFlags::Expanded;
	}

	SaveViewItemFlags(InWeakItem, TargetFlags);

	if (CurrentFlags != TargetFlags)
	{
		Item->OnExpansionChanged().Broadcast(SharedThis(this), bInIsExpanded);
	}
}

bool FNavigationToolView::ShouldShowItem(const FNavigationToolViewModelWeakPtr& InWeakItem
	, const bool bInUseFilters, const ENavigationToolItemViewMode InViewMode) const
{
	const FNavigationToolViewModelPtr Item = InWeakItem.Pin();
	if (!Item.IsValid())
	{
		return false;
	}

	if (Item.AsModel()->IsA<FNavigationToolTreeRoot>())
	{
		return true;
	}

	const TSharedPtr<FNavigationTool> Tool = WeakTool.Pin();
	if (!Tool.IsValid())
	{
		return false;
	}

	if (!Item->IsAllowedInTool())
	{
		//UE_LOGF(LogTemp, Warning, "Sequence Navigator Item Id \"%ls\" is hidden: Not Allowed In Tool", *InItem->GetItemId().GetStringId());
		return false;
	}

	if (!Item->IsViewModeSupported(InViewMode, *this))
	{
		//UE_LOGF(LogTemp, Warning, "Sequence Navigator Item Id \"%ls\" is hidden: View Mode Not Supported", *InItem->GetItemId().GetStringId());
		return false;
	}

	// Allow providers to determine whether the item should be hidden
	bool bProviderShouldHideItem = false;
	Tool->ForEachProvider([&bProviderShouldHideItem, &Item]
		(const TSharedRef<FNavigationToolProvider>& InToolProvider)
		{
			if (InToolProvider->ShouldHideItem(Item))
			{
				bProviderShouldHideItem = true;
				return false;
			}
			return true;
		});
	if (bProviderShouldHideItem)
	{
		//UE_LOGF(LogTemp, Warning, "Sequence Navigator Item Id \"%ls\" is hidden: Provider Should Hide Item", *InItem->GetItemId().GetStringId());
		return false;
	}

	// Extra pass for Non-Item Proxies that are parented under an Item Proxy
	// Hiding an Item Proxy Type should affect all the rest of the items below it
	if (!Item.AsModel()->IsA<FNavigationToolItemProxy>())
	{
		FNavigationToolViewModelPtr ItemParent = Item->GetParent();
		while (ItemParent.IsValid())
		{
			if (ItemParent.AsModel()->IsA<FNavigationToolItemProxy>())
			{
				// Stop at the first Item Proxy parent found
				break;
			}
			ItemParent = ItemParent->GetParent();
		}
	}

	/** All global filters must fail to hide the item */
	bool bGlobalFilterOut = false;

	for (const TSharedPtr<FNavigationToolBuiltInFilter>& GlobalFilter : Tool->GlobalFilters)
	{
		if (!GlobalFilter->IsActive() && GlobalFilter->PassesFilter(Item))
		{
			bGlobalFilterOut = true;
			break;
		}
	}

	if (bGlobalFilterOut)
	{
		return false;
	}

	if (bInUseFilters && FilterBar->GetFilterData().IsFilteredOut(Item))
	{
		//UE_LOGF(LogTemp, Warning, "Sequence Navigator Item Id \"%ls\" is hidden: Filtered Out", *InItem->GetItemId().GetStringId());
		return false;
	}

	return true;
}

int32 FNavigationToolView::GetVisibleChildIndex(const FNavigationToolViewModelPtr& InParentItem, const FNavigationToolViewModelPtr& InChildItem) const
{
	if (InParentItem.IsValid())
	{
		TArray<FNavigationToolViewModelWeakPtr> WeakChildren;
		GetChildrenOfItem(InParentItem, WeakChildren);
		return WeakChildren.Find(InChildItem);
	}
	return INDEX_NONE;
}

FNavigationToolViewModelPtr FNavigationToolView::GetVisibleChildAt(const FNavigationToolViewModelPtr& InParentItem, int32 InChildIndex) const
{
	if (InParentItem.IsValid())
	{
		TArray<FNavigationToolViewModelWeakPtr> WeakChildren;
		GetChildrenOfItem(InParentItem, WeakChildren);
		if (WeakChildren.IsValidIndex(InChildIndex))
		{
			return WeakChildren[InChildIndex].Pin();
		}
	}
	return nullptr;
}

bool FNavigationToolView::IsToolLocked() const
{
	return WeakTool.IsValid() && WeakTool.Pin()->IsToolLocked();
}

void FNavigationToolView::ShowColumn(const TSharedPtr<INavigationToolColumn>& InColumn)
{
	const FName ColumnId = InColumn->GetColumnId();
	ShowColumnById(ColumnId);
}

void FNavigationToolView::ShowColumnById(const FName& InColumnId)
{
	if (ToolViewWidget.IsValid())
	{
		ToolViewWidget->ShowHideColumn(InColumnId, true);
	}

	SaveColumnState(InColumnId);
}

void FNavigationToolView::HideColumn(const TSharedPtr<INavigationToolColumn>& InColumn)
{
	const FName ColumnId = InColumn->GetColumnId();

	if (ToolViewWidget.IsValid())
	{
		ToolViewWidget->ShowHideColumn(ColumnId, false);
	}

	SaveColumnState(ColumnId);
}

bool FNavigationToolView::IsColumnVisible(const TSharedPtr<INavigationToolColumn>& InColumn) const
{
	const TSharedPtr<FNavigationTool> Tool = WeakTool.Pin();
	if (!Tool.IsValid())
	{
		return false;
	}

	bool bShouldShow = false;

	Tool->ForEachProvider([this, &InColumn, &Tool, &bShouldShow]
		(const TSharedRef<FNavigationToolProvider>& InProvider)
		{
			if (FNavigationToolViewSaveState* const ViewSaveState = InProvider->GetViewSaveState(*Tool, ToolViewId))
			{
				const FName ColumnId = InColumn->GetColumnId();
				if (const FNavigationToolViewColumnSaveState* const FoundColumnState = ViewSaveState->ColumnsState.Find(ColumnId))
				{
					bShouldShow |= FoundColumnState->bVisible;
				}
			}
			return true;
		});

	return bShouldShow;
}

ENavigationToolItemViewMode FNavigationToolView::GetItemDefaultViewMode() const
{
	if (UNavigationToolSettings* const ToolSettings = WeakToolSettings.Get())
	{
		return ToolSettings->GetItemDefaultViewMode();
	}
	return ENavigationToolItemViewMode::None;
}

ENavigationToolItemViewMode FNavigationToolView::GetItemProxyViewMode() const
{
	if (UNavigationToolSettings* const ToolSettings = WeakToolSettings.Get())
	{
		return ToolSettings->GetItemProxyViewMode();
	}
	return ENavigationToolItemViewMode::None;
}

void FNavigationToolView::ToggleViewModeSupport(ENavigationToolItemViewMode& InOutViewMode, ENavigationToolItemViewMode InFlags)
{
	if (UNavigationToolSettings* const ToolSettings = WeakToolSettings.Get())
	{
		ToolSettings->ToggleViewModeSupport(InOutViewMode, InFlags);
	}

	Refresh();
}

void FNavigationToolView::ToggleItemDefaultViewModeSupport(ENavigationToolItemViewMode InFlags)
{
	if (UNavigationToolSettings* const ToolSettings = WeakToolSettings.Get())
	{
		ToolSettings->ToggleItemDefaultViewModeSupport(InFlags);
	}

	Refresh();
}

void FNavigationToolView::ToggleItemProxyViewModeSupport(ENavigationToolItemViewMode InFlags)
{
	if (UNavigationToolSettings* const ToolSettings = WeakToolSettings.Get())
	{
		ToolSettings->ToggleItemProxyViewModeSupport(InFlags);
	}

	Refresh();
}

ECheckBoxState FNavigationToolView::GetViewModeCheckState(ENavigationToolItemViewMode InViewMode, ENavigationToolItemViewMode InFlags) const
{
	const ENavigationToolItemViewMode Result = InViewMode & InFlags;

	if (Result == InFlags)
	{
		return ECheckBoxState::Checked;
	}
	
	if (Result != ENavigationToolItemViewMode::None)
	{
		return ECheckBoxState::Undetermined;
	}
	
	return ECheckBoxState::Unchecked;
}

ECheckBoxState FNavigationToolView::GetItemDefaultViewModeCheckState(ENavigationToolItemViewMode InFlags) const
{
	if (UNavigationToolSettings* const ToolSettings = WeakToolSettings.Get())
	{
		return GetViewModeCheckState(ToolSettings->GetItemDefaultViewMode(), InFlags);
	}
	return ECheckBoxState::Unchecked;
}

ECheckBoxState FNavigationToolView::GetItemProxyViewModeCheckState(ENavigationToolItemViewMode InFlags) const
{
	if (UNavigationToolSettings* const ToolSettings = WeakToolSettings.Get())
	{
		return GetViewModeCheckState(ToolSettings->GetItemProxyViewMode(), InFlags);
	}
	return ECheckBoxState::Unchecked;
}

void FNavigationToolView::ToggleMutedHierarchy()
{
	if (UNavigationToolSettings* const ToolSettings = WeakToolSettings.Get())
	{
		ToolSettings->SetUseMutedHierarchy(!ToolSettings->ShouldUseMutedHierarchy());
	}

	Refresh();
}

bool FNavigationToolView::IsMutedHierarchyActive() const
{
	if (UNavigationToolSettings* const ToolSettings = WeakToolSettings.Get())
	{
		return ToolSettings->ShouldUseMutedHierarchy();
	}
	return false;
}

void FNavigationToolView::ToggleAutoExpandToSelection()
{
	if (UNavigationToolSettings* const ToolSettings = WeakToolSettings.Get())
	{
		ToolSettings->SetAutoExpandToSelection(!ToolSettings->ShouldAutoExpandToSelection());
	}

	Refresh();
}

bool FNavigationToolView::ShouldAutoExpandToSelection() const
{
	if (UNavigationToolSettings* const ToolSettings = WeakToolSettings.Get())
	{
		return ToolSettings->ShouldAutoExpandToSelection();
	}
	return false;
}

void FNavigationToolView::ToggleUseShortNames()
{
	if (UNavigationToolSettings* const ToolSettings = WeakToolSettings.Get())
	{
		ToolSettings->SetUseShortNames(!ToolSettings->ShouldUseShortNames());
	}

	Refresh();
}

bool FNavigationToolView::ShouldUseShortNames() const
{
	if (UNavigationToolSettings* const ToolSettings = WeakToolSettings.Get())
	{
		return ToolSettings->ShouldUseShortNames();
	}
	return false;
}

void FNavigationToolView::ToggleShowItemFilters()
{
	//Note: Not Marking Navigation Tool Instance as Modified because this is not saved.
	bShowItemFilters = !bShowItemFilters;
}

void FNavigationToolView::ToggleShowItemColumns()
{
	bShowItemColumns = !bShowItemColumns;
}

void FNavigationToolView::SetItemTypeHidden(const FName InItemTypeName, const bool bInHidden)
{
	if (IsItemTypeHidden(InItemTypeName) != bInHidden)
	{
		if (bInHidden)
		{
			HiddenItemTypes.Add(InItemTypeName);
		}
		else
		{
			HiddenItemTypes.Remove(InItemTypeName);
		}
		RequestRefresh();
	}	
}

void FNavigationToolView::ToggleHideItemTypes(const FName InItemTypeName)
{
	SetItemTypeHidden(InItemTypeName, !IsItemTypeHidden(InItemTypeName));
}

ECheckBoxState FNavigationToolView::GetToggleHideItemTypesState(const FName InItemTypeName) const
{
	return IsItemTypeHidden(InItemTypeName) ? ECheckBoxState::Unchecked : ECheckBoxState::Checked;
}

bool FNavigationToolView::IsItemTypeHidden(const FName InItemTypeName) const
{
	return HiddenItemTypes.Contains(InItemTypeName);
}

bool FNavigationToolView::IsItemTypeHidden(const FNavigationToolViewModelPtr& InItem) const
{
	return IsItemTypeHidden(InItem.AsModel()->GetTypeTable().GetTypeName());
}

void FNavigationToolView::OnDragEnter(const FDragDropEvent& InDragDropEvent
	, const FNavigationToolViewModelWeakPtr InWeakTargetItem)
{
	bool bDragIntoTreeRoot = false;

	const FNavigationToolViewModelPtr TargetItem = InWeakTargetItem.Pin();
	if (!TargetItem.IsValid() && WeakTool.IsValid())
	{
		const FNavigationToolViewModelPtr TreeRoot = WeakTool.Pin()->GetTreeRoot().Pin();
		const bool bCanAcceptDrop = TreeRoot.IsValid()
			&& TreeRoot->CanAcceptDrop(InDragDropEvent, EItemDropZone::OntoItem).IsSet();
		SetDragIntoTreeRoot(bCanAcceptDrop);
	}

	SetDragIntoTreeRoot(bDragIntoTreeRoot);

	/*const FNavigationToolViewModelPtr TargetItem = InWeakTargetItem.Pin();
	if (!TargetItem.IsValid() && WeakTool.IsValid())
	{
		const TViewModelPtr<FNavigationToolTreeRoot> TreeRoot = WeakTool.Pin()->GetTreeRoot();
		const bool bCanAcceptDrop = TreeRoot->CanAcceptDrop(InDragDropEvent, EItemDropZone::OntoItem).IsSet();
		SetDragIntoTreeRoot(bCanAcceptDrop);
	}
	else
	{
		SetDragIntoTreeRoot(false);
	}*/
}

void FNavigationToolView::OnDragLeave(const FDragDropEvent& InDragDropEvent
	, const FNavigationToolViewModelWeakPtr InWeakTargetItem)
{
	// If drag left an item, set the drag into tree root to false (this will set it back to false if a valid item receives DragEnter)
	SetDragIntoTreeRoot(InWeakTargetItem.Pin().IsValid());
}

FReply FNavigationToolView::OnDragDetected(const FGeometry& InGeometry
	, const FPointerEvent& InPointerEvent
	, const FNavigationToolViewModelWeakPtr InWeakTargetItem)
{
	if (IsToolLocked())
	{
		return FReply::Unhandled();
	}

	// Only select the target if it hasn't already been selected
	if (!IsItemSelected(InWeakTargetItem.ImplicitPin()))
	{
		const ENavigationToolItemSelectionFlags SelectionFlags = InPointerEvent.IsControlDown()
			? ENavigationToolItemSelectionFlags::AppendToCurrentSelection
			: ENavigationToolItemSelectionFlags::None;

		SelectItems({ InWeakTargetItem.ImplicitPin() }, SelectionFlags);
	}

	// Get all selected items that are in a state where they can be selected again (i.e. not read only)
	TArray<FNavigationToolViewModelWeakPtr> WeakItems = GetSelectedItems();
	WeakItems.RemoveAll([this](const FNavigationToolViewModelWeakPtr& InWeakItem) -> bool
		{
			return !CanSelectItem(InWeakItem);
		});

	if (WeakItems.Num() > 0)
	{
		const ENavigationToolDragDropActionType ActionType = InPointerEvent.IsAltDown()
			? ENavigationToolDragDropActionType::Copy
			: ENavigationToolDragDropActionType::Move;

		const TSharedRef<FNavigationToolItemDragDropOp> DragDropOp =
			FNavigationToolItemDragDropOp::New(WeakItems, SharedThis(this), ActionType);
		return FReply::Handled().BeginDragDrop(DragDropOp);
	}

	return FReply::Unhandled();
}

FReply FNavigationToolView::OnDrop(const FDragDropEvent& InDragDropEvent
	, const EItemDropZone InDropZone
	, const FNavigationToolViewModelWeakPtr InWeakTargetItem)
{
	SetDragIntoTreeRoot(false);

	const FNavigationToolViewModelPtr TargetItem = InWeakTargetItem.Pin();
	if (TargetItem.IsValid())
	{
		return TargetItem->AcceptDrop(InDragDropEvent, InDropZone);
	}

	FNavigationToolViewModelPtr TreeRoot;

	if (WeakTool.IsValid())
	{
		TreeRoot = WeakTool.Pin()->GetTreeRoot().Pin();
	}

	if (TreeRoot.IsValid() && TreeRoot->CanAcceptDrop(InDragDropEvent, EItemDropZone::OntoItem))
	{
		return TreeRoot->AcceptDrop(InDragDropEvent, EItemDropZone::OntoItem);
	}
	
	return FReply::Unhandled();
}

TOptional<EItemDropZone> FNavigationToolView::OnCanDrop(const FDragDropEvent& InDragDropEvent
	, const EItemDropZone InDropZone
	, const FNavigationToolViewModelWeakPtr InWeakTargetItem) const
{
	const FNavigationToolViewModelPtr TargetItem = InWeakTargetItem.Pin();
	if (TargetItem.IsValid() && !IsToolLocked() && CanSelectItem(TargetItem))
	{
		return TargetItem->CanAcceptDrop(InDragDropEvent, InDropZone);
	}
	return TOptional<EItemDropZone>();
}

void FNavigationToolView::SetDragIntoTreeRoot(const bool bInIsDraggingIntoTreeRoot)
{
	if (ToolViewWidget.IsValid())
	{
		ToolViewWidget->SetTreeBorderVisibility(bInIsDraggingIntoTreeRoot);
	}
}

void FNavigationToolView::RenameSelected()
{
	const TSharedPtr<FNavigationTool> Tool = WeakTool.Pin();
	if (!Tool.IsValid())
	{
		return;
	}

	TArray<FNavigationToolViewModelWeakPtr> WeakItems = GetSelectedItems();

	if (WeakItems.IsEmpty())
	{
		return;
	}

	// Assume we have an item currently renaming
	ResetRenaming();

	// Remove items that are invalid or can't be renamed
	WeakItems.RemoveAll([](const FNavigationToolViewModelWeakPtr& InWeakItem)
		{
			const FNavigationToolViewModelPtr Item = InWeakItem.Pin();
			if (!Item.IsValid())
			{
				return true;
			}

			if (const TViewModelPtr<IRenameableExtension> RenameableExtension = Item.ImplicitCast())
			{
				return !RenameableExtension->CanRename();
			}

			return true;
		});

	WeakItemsRemainingRename = MoveTemp(WeakItems);

	if (WeakItemsRemainingRename.Num() > 0)
	{
		FNavigationTool::SortItems(WeakItemsRemainingRename);
		bRenamingItems = true;
	}
}

void FNavigationToolView::ResetRenaming()
{
	if (const FNavigationToolViewModelPtr CurrentItemRenaming = WeakCurrentItemRenaming.Pin())
	{
		CurrentItemRenaming->OnRenameAction().RemoveAll(this);
		WeakCurrentItemRenaming = nullptr;
	}

	if (WeakItemsRemainingRename.IsEmpty())
	{
		bRenamingItems = false;
	}
}

void FNavigationToolView::OnItemRenameAction(const ENavigationToolRenameAction InRenameAction, const TSharedPtr<INavigationToolView>& InToolView)
{
	if (InToolView.Get() != this)
	{
		return;
	}
	
	switch (InRenameAction)
	{
	case ENavigationToolRenameAction::None:
		break;
	
	case ENavigationToolRenameAction::Requested:
		break;
	
	case ENavigationToolRenameAction::Cancelled:
		WeakItemsRemainingRename.Reset();
		ResetRenaming();
		break;
	
	case ENavigationToolRenameAction::Completed:
		ResetRenaming();
		break;
	
	default:
		break;
	}
}

bool FNavigationToolView::CanRenameSelected() const
{
	for (const FNavigationToolViewModelWeakPtr& WeakItem : WeakSelectedItems)
	{
		const FNavigationToolViewModelPtr Item = WeakItem.Pin();
		if (!Item.IsValid())
		{
			return false;
		}

		if (const TViewModelPtr<IRenameableExtension> RenameableExtension = Item.ImplicitCast())
		{
			if (!RenameableExtension->CanRename())
			{
				return false;
			}
		}
	}

	return WeakSelectedItems.Num() > 0;
}

void FNavigationToolView::DeleteSelected()
{
	if (const TSharedPtr<FNavigationTool> Tool = WeakTool.Pin())
	{
		TArray<FNavigationToolViewModelWeakPtr> WeakItems = GetSelectedItems();

		WeakItems.RemoveAll([](const FNavigationToolViewModelWeakPtr& InWeakItem)
			{
				if (const FNavigationToolViewModelPtr Item = InWeakItem.Pin())
				{
					return !Item.IsValid() || !Item->CanDelete();
				}
				return false;
			});

		if (WeakItems.IsEmpty())
		{
			return;
		}

		Tool->DeleteItems(WeakItems);
	}
}

bool FNavigationToolView::CanDeleteSelected() const
{
	for (const FNavigationToolViewModelWeakPtr& WeakItem : GetSelectedItems())
	{
		if (const FNavigationToolViewModelPtr Item = WeakItem.Pin())
		{
			if (Item->CanDelete())
			{
				return true;
			}
		}
	}
	return false;
}

void FNavigationToolView::DuplicateSelected()
{
	// Disabled for now until (if) we decide to support this
	/*if (const TSharedPtr<FNavigationTool> Tool = WeakTool.Pin())
	{
		Tool->DuplicateItems(GetSelectedItems(), nullptr, TOptional<EItemDropZone>());
	}*/
}

bool FNavigationToolView::CanDuplicateSelected() const
{
	// Disabled for now until (if) we decide to support this
	/*for (const FNavigationToolViewModelPtr& Item : GetSelectedItems())
	{
		if (Item && Item->CanDuplicate())
		{
			return true;
		}
	}*/
	return false;
}

void FNavigationToolView::SelectChildren(const bool bInIsRecursive)
{
	TArray<FNavigationToolViewModelWeakPtr> WeakItemsToSelect;
	TArray<FNavigationToolViewModelWeakPtr> WeakRemainingItems = GetSelectedItems();

	while (WeakRemainingItems.Num() > 0)
	{
		// Note: Pop here will affect the order of children in selection
		const FNavigationToolViewModelPtr ParentItem = WeakRemainingItems.Pop().Pin();
		if (!ParentItem.IsValid())
		{
			continue;
		}

		TArray<FNavigationToolViewModelWeakPtr> WeakChildItems;
		GetChildrenOfItem(ParentItem, WeakChildItems);
		if (bInIsRecursive)
		{
			WeakRemainingItems.Append(WeakChildItems);
		}
		WeakItemsToSelect.Append(WeakChildItems);
	}

	SelectItems(WeakItemsToSelect, ENavigationToolItemSelectionFlags::AppendToCurrentSelection
		| ENavigationToolItemSelectionFlags::SignalSelectionChange);
}

bool FNavigationToolView::CanSelectChildren() const
{
	return GetViewSelectedItemCount() > 0;
}

void FNavigationToolView::SelectParent()
{
	const TSet<FNavigationToolViewModelWeakPtr> WeakItems(GetSelectedItems());

	TSet<FNavigationToolViewModelWeakPtr> WeakParentItemsToSelect;
	WeakParentItemsToSelect.Reserve(WeakItems.Num());

	const FNavigationToolViewModelPtr RootItem = GetRootItem();
	
	//Add only Valid Parents that are not Root and are not part of the Original Selection!
	for (const FNavigationToolViewModelWeakPtr& WeakItem : WeakItems)
	{
		const FNavigationToolViewModelPtr Item = WeakItem.Pin();
		if (!Item.IsValid())
		{
			continue;
		}

		const FNavigationToolViewModelPtr ParentItem = Item->GetParent();
		if (ParentItem.IsValid()
			&& ParentItem != RootItem
			&& !WeakItems.Contains(ParentItem))
		{
			WeakParentItemsToSelect.Add(ParentItem);
		}
	}

	SortAndSelectItems(WeakParentItemsToSelect.Array());
}

bool FNavigationToolView::CanSelectParent() const
{
	return GetViewSelectedItemCount() == 1;
}

void FNavigationToolView::SelectFirstChild()
{
	const TSet<FNavigationToolViewModelWeakPtr> WeakItems(GetSelectedItems());

	TSet<FNavigationToolViewModelWeakPtr> WeakFirstChildItemsToSelect;
	WeakFirstChildItemsToSelect.Reserve(WeakItems.Num());

	for (const FNavigationToolViewModelWeakPtr& WeakItem : WeakItems)
	{
		const FNavigationToolViewModelPtr Item = WeakItem.Pin();
		if (!Item.IsValid())
		{
			continue;
		}

		const FNavigationToolViewModelPtr FirstChildItem = GetVisibleChildAt(Item, 0);

		// Don't select Component items! (Component items on selection also select their owner actor items, which can cause undesired issues)
		if (FirstChildItem.IsValid() && !FirstChildItem.AsModel()->IsA<FNavigationToolComponent>())
		{
			WeakFirstChildItemsToSelect.Add(FirstChildItem);
		}
	}

	SortAndSelectItems(WeakFirstChildItemsToSelect.Array());
}

bool FNavigationToolView::CanSelectFirstChild() const
{
	return GetViewSelectedItemCount() == 1;
}

void FNavigationToolView::SelectSibling(const int32 InDeltaIndex)
{
	const TArray<FNavigationToolViewModelWeakPtr> WeakItems = GetSelectedItems();

	TSet<FNavigationToolViewModelWeakPtr> WeakSiblingItemsToSelect;
	WeakSiblingItemsToSelect.Reserve(WeakItems.Num());

	for (const FNavigationToolViewModelWeakPtr& WeakItem : WeakItems)
	{
		const FNavigationToolViewModelPtr Item = WeakItem.Pin();
		if (!Item.IsValid())
		{
			continue;
		}

		const FNavigationToolViewModelPtr ParentItem = Item->GetParent();
		if (!ParentItem.IsValid())
		{
			continue;
		}

		const int32 ItemIndex = GetVisibleChildIndex(ParentItem, Item);
		const int32 TargetIndex = ItemIndex + InDeltaIndex;

		// Don't try to Normalize Index, if it's Invalid, we won't cycle and just skip that selection
		const FNavigationToolViewModelPtr SiblingToSelect = GetVisibleChildAt(ParentItem, TargetIndex);

		// Don't select Component items! (Component items on selection also select their owner actor items, which can cause undesired issues)
		if (SiblingToSelect.IsValid() && !SiblingToSelect.AsModel()->IsA<FNavigationToolComponent>())
		{
			WeakSiblingItemsToSelect.Add(SiblingToSelect);
		}
	}

	SortAndSelectItems(WeakSiblingItemsToSelect.Array());
}

bool FNavigationToolView::CanSelectSibling() const
{
	return GetViewSelectedItemCount() == 1;
}

void FNavigationToolView::ExpandAll()
{
	for (const FNavigationToolViewModelWeakPtr& WeakItem : WeakRootVisibleItems)
	{
		SetItemExpansionRecursive(WeakItem.Pin(), true);
	}
}

bool FNavigationToolView::CanExpandAll() const
{
	return true;
}

void FNavigationToolView::CollapseAll()
{
	for (const FNavigationToolViewModelWeakPtr& WeakItem : WeakRootVisibleItems)
	{
		SetItemExpansionRecursive(WeakItem.Pin(), false);
	}
}

bool FNavigationToolView::CanCollapseAll() const
{
	return true;
}

void FNavigationToolView::ScrollNextIntoView()
{
	ScrollDeltaIndexIntoView(+1);
}

void FNavigationToolView::ScrollPrevIntoView()
{
	ScrollDeltaIndexIntoView(-1);
}

bool FNavigationToolView::CanScrollNextIntoView() const
{
	return GetViewSelectedItemCount() > 0;
}

void FNavigationToolView::ScrollDeltaIndexIntoView(const int32 InDeltaIndex)
{
	const int32 SelectedItemCount = WeakSortedSelectedItems.Num();
	if (WeakSortedSelectedItems.Num() > 0)
	{
		const int32 TargetIndex  = NextSelectedItemIntoView + InDeltaIndex;
		NextSelectedItemIntoView = TargetIndex % SelectedItemCount;
		if (NextSelectedItemIntoView < 0)
		{
			NextSelectedItemIntoView += SelectedItemCount;
		}
		ScrollItemIntoView(WeakSortedSelectedItems[NextSelectedItemIntoView].Pin());
	}
}

void FNavigationToolView::ScrollItemIntoView(const FNavigationToolViewModelWeakPtr& InWeakItem)
{
	const FNavigationToolViewModelPtr Item = InWeakItem.Pin();
	if (!Item.IsValid())
	{
		return;
	}

	SetParentItemExpansions(InWeakItem, true);

	if (ToolViewWidget.IsValid() && ToolViewWidget->GetTreeView())
	{
		ToolViewWidget->GetTreeView()->FocusOnItem(Item);
		ToolViewWidget->ScrollItemIntoView(InWeakItem);
	}
}

void FNavigationToolView::SortAndSelectItems(TArray<FNavigationToolViewModelWeakPtr> InWeakItemsToSelect)
{
	if (InWeakItemsToSelect.IsEmpty())
	{
		return;
	}

	FNavigationTool::SortItems(InWeakItemsToSelect);

	SelectItems(InWeakItemsToSelect
		, ENavigationToolItemSelectionFlags::SignalSelectionChange
		| ENavigationToolItemSelectionFlags::ScrollIntoView);
}

void FNavigationToolView::RefreshTool(const bool bInImmediateRefresh)
{
	if (const TSharedPtr<INavigationTool> Tool = GetOwnerTool())
	{
		if (bInImmediateRefresh)
		{
			Tool->Refresh();
		}
		else
		{
			Tool->RequestRefresh();
		}
	}
}

void FNavigationToolView::EnsureToolViewCount(const int32 InToolViewId) const
{
	const TSharedPtr<FNavigationTool> Tool = WeakTool.Pin();
	if (!Tool.IsValid())
	{
		return;
	}

	FNavigationTool& ToolRef = *Tool;

	ToolRef.ForEachProvider([&ToolRef, InToolViewId]
		(const TSharedRef<FNavigationToolProvider>& InToolProvider)
		{
			InToolProvider->EnsureToolViewCount(ToolRef, InToolViewId);
			return true;
		});
}

void FNavigationToolView::SaveViewState(const TSharedRef<FNavigationToolProvider>& InProvider)
{
	const TSharedPtr<FNavigationTool> Tool = WeakTool.Pin();
	if (!Tool.IsValid())
	{
		return;
	}

	const FNavigationTool& ToolRef = *Tool;

	EnsureToolViewCount(ToolViewId);

	if (FNavigationToolViewSaveState* const ViewSaveState = InProvider->GetViewSaveState(ToolRef, ToolViewId))
	{
		// Save view state filters
		ViewSaveState->ActiveItemFilters.Reset();
		for (const TSharedRef<FNavigationToolFilter>& ActiveItemFilter : FilterBar->GetActiveFilters())
		{
			ViewSaveState->ActiveItemFilters.Add(*ActiveItemFilter->GetDisplayName().ToString());
		}

		SaveColumnState();
		SaveToolViewItems(*ViewSaveState);
	}
}

void FNavigationToolView::LoadViewState(const TSharedRef<FNavigationToolProvider>& InProvider)
{
	const TSharedPtr<FNavigationTool> Tool = WeakTool.Pin();
	if (!Tool.IsValid())
	{
		return;
	}

	const FNavigationTool& ToolRef = *Tool;

	EnsureToolViewCount(ToolViewId);

	// Disable all filters before load
	FilterBar->EnableAllFilters(false, {});

	if (FNavigationToolViewSaveState* const ViewSaveState = InProvider->GetViewSaveState(ToolRef, ToolViewId))
	{
		LoadFilterState(*ViewSaveState, false, false);
		LoadToolViewItems(*ViewSaveState);
	}
	else
	{
		UE_LOGF(LogSequenceNavigator, Warning, "FNavigationToolView::LoadViewState(): Save state is NULL!");
	}

	PostLoad();

	FilterBar->RequestFilterUpdate();
}

void FNavigationToolView::SaveColumnState(const FName InColumnId)
{
	if (!ToolViewWidget.IsValid())
	{
		return;
	}

	const TSharedPtr<FNavigationTool> Tool = WeakTool.Pin();
	if (!Tool.IsValid())
	{
		return;
	}

	const FNavigationTool& ToolRef = *Tool;

	// Save all columns unless a specific column is specified
	if (InColumnId.IsNone())
	{
		// Save each column to their respective providers save data
		// Note: Some columns may have multiple providers
		for (const TTuple<FName, TSharedPtr<INavigationToolColumn>>& Column : Columns)
		{
			if (!Column.Value.IsValid())
			{
				continue;
			}

			const FName ColumnId = Column.Value->GetColumnId();

			Tool->ForEachProvider([this, &ColumnId, &ToolRef]
				(const TSharedRef<FNavigationToolProvider>& InProvider)
				{
					if (FNavigationToolViewSaveState* const ViewSaveState = InProvider->GetViewSaveState(ToolRef, ToolViewId))
					{
						FNavigationToolViewColumnSaveState& FoundColumnState = ViewSaveState->ColumnsState.FindOrAdd(ColumnId);
						ToolViewWidget->GenerateColumnState(ColumnId, FoundColumnState);
					}
					return true;
				});
		}
	}
	else if (ensure(Columns.Contains(InColumnId)))
	{
		Tool->ForEachProvider([this, &InColumnId, &ToolRef]
			(const TSharedRef<FNavigationToolProvider>& InProvider)
			{
				// Save the specific column to its providers save data
				const TSharedPtr<INavigationToolColumn>& Column = Columns[InColumnId];
				if (FNavigationToolViewSaveState* const ViewSaveState = InProvider->GetViewSaveState(ToolRef, ToolViewId))
				{
					FNavigationToolViewColumnSaveState& FoundColumnState = ViewSaveState->ColumnsState.FindOrAdd(InColumnId);
					ToolViewWidget->GenerateColumnState(InColumnId, FoundColumnState);
				}
				return true;
			});
	}
}

void FNavigationToolView::SaveFilterState(FNavigationToolViewSaveState& OutViewSaveState)
{
	OutViewSaveState.ActiveItemFilters.Reset();

	for (const TSharedRef<FNavigationToolFilter>& ActiveItemFilter : FilterBar->GetActiveFilters())
	{
		OutViewSaveState.ActiveItemFilters.Add(*ActiveItemFilter->GetDisplayName().ToString());
	}
}

void FNavigationToolView::LoadFilterState(const FNavigationToolViewSaveState& InViewSaveState
	, const bool bInDisableAllFilters
	, const bool bInRequestFilterUpdate)
{
	if (bInDisableAllFilters)
	{
		FilterBar->EnableAllFilters(false, {});
	}

	for (const FName ActiveItemFilterName : InViewSaveState.ActiveItemFilters)
	{
		FilterBar->SetFilterActiveByDisplayName(ActiveItemFilterName.ToString(), true, false);
	}

	if (bInRequestFilterUpdate)
	{
		FilterBar->RequestFilterUpdate();
	}
}

void FNavigationToolView::SaveToolViewItems(FNavigationToolViewSaveState& OutViewSaveState)
{
	const TSharedPtr<FNavigationTool> OwnerTool = WeakTool.Pin();
	if (!OwnerTool.IsValid())
	{
		return;
	}

	const FNavigationToolViewModelPtr TreeRoot = OwnerTool->GetTreeRoot().Pin();
	if (!TreeRoot.IsValid())
	{
		return;
	}

	TArray<FNavigationToolViewModelWeakPtr> WeakItemsToSave = TreeRoot->GetChildren();

	OutViewSaveState.ViewItemFlags.Reset();

	while (WeakItemsToSave.Num() > 0)
	{
		const FNavigationToolViewModelPtr ItemToSave = WeakItemsToSave.Pop().Pin();
		if (!ItemToSave.IsValid())
		{
			continue;
		}

		// Iteratively also save children
		WeakItemsToSave.Append(ItemToSave->GetChildren());

		const FNavigationToolItemId ItemId = ItemToSave->GetItemId();
		const FString StringId = ItemId.GetStringId();

		// Save item state flags
		if (const ENavigationToolItemFlags* const ItemFlags = OutViewSaveState.ViewItemFlags.Find(StringId))
		{
			OutViewSaveState.ViewItemFlags.Add(StringId, *ItemFlags);
		}
		else
		{
			OutViewSaveState.ViewItemFlags.Remove(StringId);
		}
	}
}

void FNavigationToolView::LoadToolViewItems(FNavigationToolViewSaveState& InViewSaveState)
{
	const TSharedPtr<FNavigationTool> OwnerTool = WeakTool.Pin();
	if (!OwnerTool.IsValid())
	{
		return;
	}

	const FNavigationToolViewModelPtr TreeRoot = OwnerTool->GetTreeRoot().Pin();
	if (!TreeRoot.IsValid())
	{
		return;
	}

	TArray<FNavigationToolViewModelWeakPtr> WeakItemsToLoad = TreeRoot->GetChildren();

	while (WeakItemsToLoad.Num() > 0)
	{
		const FNavigationToolViewModelPtr ItemToLoad = WeakItemsToLoad.Pop().Pin();
		if (!ItemToLoad.IsValid())
		{
			continue;
		}

		// Iteratively also load children
		WeakItemsToLoad.Append(ItemToLoad->GetChildren());

		const FNavigationToolItemId ItemId = ItemToLoad->GetItemId();
		const FString StringId = ItemId.GetStringId();

		// Load item flags
		if (const ENavigationToolItemFlags* const ItemFlags = InViewSaveState.ViewItemFlags.Find(StringId))
		{
			InViewSaveState.ViewItemFlags.Add(StringId, *ItemFlags);
		}
		else
		{
			InViewSaveState.ViewItemFlags.Remove(StringId);
		}
	}
}

TSharedRef<SWidget> FNavigationToolView::GetColumnMenuContent(const FName InColumnId)
{
	FMenuBuilder MenuBuilder(true, ViewCommandList);

	const FNavigationToolCommands& ToolCommands = FNavigationToolCommands::Get();

	MenuBuilder.AddMenuEntry(
		LOCTEXT("ResetColumnSize", "Reset Column Size"), 
		LOCTEXT("ResetColumnSizeTooltip", "Resets the size of this column to the default"), 
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateRaw(this, &FNavigationToolView::ResetColumnSize, InColumnId),
			FCanExecuteAction::CreateRaw(this, &FNavigationToolView::CanResetColumnSize, InColumnId)));

	MenuBuilder.AddMenuSeparator();

	MenuBuilder.AddMenuEntry(ToolCommands.ResetVisibleColumnSizes);

	return MenuBuilder.MakeWidget();
}

void FNavigationToolView::ResetColumnSize(const FName InColumnId)
{
	if (!ToolViewWidget.IsValid())
	{
		return;
	}

	if (!Columns.Contains(InColumnId) || !ToolViewWidget->IsColumnVisible(InColumnId))
	{
		return;
	}

	const TSharedPtr<FNavigationTool> Tool = WeakTool.Pin();
	if (!Tool.IsValid())
	{
		return;
	}

	const FNavigationTool& ToolRef = *Tool;

	Tool->ForEachProvider([this, &InColumnId, &ToolRef]
		(const TSharedRef<FNavigationToolProvider>& InProvider)
		{
			if (FNavigationToolViewSaveState* const ViewSaveState = InProvider->GetViewSaveState(ToolRef, ToolViewId))
			{
				const float DefaultSize = Columns[InColumnId]->GetFillWidth();
				if (DefaultSize > 0.f)
				{
					ViewSaveState->ColumnsState.FindOrAdd(InColumnId).Size = DefaultSize;

					ToolViewWidget->SetColumnWidth(InColumnId, DefaultSize);

					ToolViewWidget->GenerateColumnState(InColumnId, ViewSaveState->ColumnsState.FindOrAdd(InColumnId));
				}
			}
			return true;
		});
}

bool FNavigationToolView::CanResetColumnSize(const FName InColumnId) const
{
	if (!Columns.Contains(InColumnId) || !ToolViewWidget->IsColumnVisible(InColumnId))
	{
		return false;
	}

	const TSharedPtr<FNavigationTool> Tool = WeakTool.Pin();
	if (!Tool.IsValid())
	{
		return false;
	}

	const FNavigationTool& ToolRef = *Tool;

	bool bCanReset = false;

	Tool->ForEachProvider([this, &InColumnId, &ToolRef, &bCanReset]
		(const TSharedRef<FNavigationToolProvider>& InProvider)
		{
			const FNavigationToolViewSaveState* const ViewSaveState = InProvider->GetViewSaveState(ToolRef, ToolViewId);
			if (!ViewSaveState || !ViewSaveState->ColumnsState.Contains(InColumnId))
			{
				return true;
			}

			const float DefaultSize = Columns[InColumnId]->GetFillWidth();
			if (DefaultSize <= 0.f)
			{
				return true;
			}

			bCanReset |= ViewSaveState->ColumnsState[InColumnId].Size != DefaultSize;

			return true;
		});

	return bCanReset;
}

void FNavigationToolView::ResetVisibleColumnSizes()
{
	if (!ToolViewWidget.IsValid())
	{
		return;
	}

	const TSharedPtr<FNavigationTool> Tool = WeakTool.Pin();
	if (!Tool.IsValid())
	{
		return;
	}

	const FNavigationTool& ToolRef = *Tool;

	for (const TPair<FName, TSharedPtr<INavigationToolColumn>>& Pair : Columns)
	{
		const FName& ColumnId = Pair.Key;

		if (!ToolViewWidget->IsColumnVisible(ColumnId))
		{
			continue;
		}

		const float DefaultSize = Columns[ColumnId]->GetFillWidth();
		if (DefaultSize <= 0.f)
		{
			continue;
		}

		Tool->ForEachProvider([this, &ColumnId, &ToolRef, DefaultSize]
			(const TSharedRef<FNavigationToolProvider>& InProvider)
			{
				if (FNavigationToolViewSaveState* const ViewSaveState = InProvider->GetViewSaveState(ToolRef, ToolViewId))
				{
					ToolViewWidget->SetColumnWidth(ColumnId, DefaultSize);
					ToolViewWidget->GenerateColumnState(ColumnId, ViewSaveState->ColumnsState.FindOrAdd(ColumnId));
				}
				return true;
			});
	}
}

bool FNavigationToolView::CanResetAllColumnSizes() const
{
	return true;
}

void FNavigationToolView::SaveNewCustomColumnView()
{
	UNavigationToolSettings* const ToolSettings = WeakToolSettings.Get();
	if (!ToolSettings)
	{
		return;
	}

	TSet<FNavigationToolColumnView>& CustomColumnViews = ToolSettings->GetCustomColumnViews();

	SaveColumnState();

	// Create a unique column view name suggestion
	auto DoesColumnViewExist = [&CustomColumnViews](const FText& InViewName)
	{
		for (const FNavigationToolColumnView& ColumnView : CustomColumnViews)
		{
			if (ColumnView.ViewName.EqualTo(InViewName))
			{
				return true;
			}
		}
		return false;
	};

	FNavigationToolColumnView NewColumnView;

	for (int32 Index = 1; Index < INT_MAX; ++Index)
	{
		NewColumnView.ViewName = FText::Format(LOCTEXT("ColumnViewName", "Column View {0}"), { Index });
		if (!DoesColumnViewExist(NewColumnView.ViewName))
		{
			break;
		}
	}

	for (const TPair<FName, TSharedPtr<INavigationToolColumn>>& Column : Columns)
	{
		const FName ColumnId = Column.Value->GetColumnId();
		if (IsColumnVisible(Column.Value))
		{
			NewColumnView.VisibleColumns.Add(ColumnId);
		}
	}

	// Prompt user for name, using the generated unique suggestion as the default name
	FModalTextInputDialog InputDialog;
	InputDialog.InputLabel = LOCTEXT("NewColumnViewName_InputLabel", "New Column View Name");
	if (!InputDialog.Open(NewColumnView.ViewName, NewColumnView.ViewName))
	{
		return;
	}

	bool bAlreadyExists = false;
	CustomColumnViews.Add(NewColumnView, &bAlreadyExists);

	if (bAlreadyExists)
	{
		FMessageDialog::Open(EAppMsgCategory::Error, EAppMsgType::Ok
			, LOCTEXT("AlreadyExistsErrorText", "Column view name already exists!"));
		return;
	}

	CustomColumnViews.Sort([](const FNavigationToolColumnView& InA, const FNavigationToolColumnView& InB)
		{
			return InA.ViewName.CompareTo(InB.ViewName) < 0;
		});

	ToolSettings->SaveConfig();
}

void FNavigationToolView::ApplyCustomColumnView(const FText InColumnViewName)
{
	if (!ToolViewWidget.IsValid() || InColumnViewName.IsEmptyOrWhitespace())
	{
		return;
	}

	UNavigationToolSettings* const ToolSettings = WeakToolSettings.Get();
	if (!ToolSettings)
	{
		return;
	}

	const FNavigationToolColumnView* const SavedColumnView = ToolSettings->FindCustomColumnView(InColumnViewName);
	if (!SavedColumnView)
	{
		return;
	}

	for (const TPair<FName, TSharedPtr<INavigationToolColumn>>& Column : Columns)
	{
		const FName ColumnId = Column.Value->GetColumnId();
		const bool bColumnVisible = SavedColumnView->VisibleColumns.Contains(ColumnId);
		ToolViewWidget->ShowHideColumn(ColumnId, bColumnVisible);
	}

	SaveColumnState();
}

bool FNavigationToolView::CanFocusSingleSelection() const
{
	if (WeakSelectedItems.Num() == 1)
	{
		if (const FNavigationToolViewModelPtr SelectedItem = WeakSelectedItems[0].Pin())
		{
			const TViewModelPtr<FNavigationToolSequence> SequenceItem = SelectedItem.ImplicitCast();
			return SequenceItem && SequenceItem->GetSequence();
		}
	}
	return false;
}

void FNavigationToolView::FocusSingleSelection()
{
	if (WeakSelectedItems.Num() != 1)
	{
		return;
	}

	const FNavigationToolViewModelPtr SelectedItem = WeakSelectedItems[0].Pin();
	if (!SelectedItem.IsValid())
	{
		return;
	}

	const TViewModelPtr<FNavigationToolSequence> SequenceItem = SelectedItem.ImplicitCast();
	if (!SequenceItem.IsValid())
	{
		return;
	}

	UMovieSceneSequence* const Sequence = SequenceItem->GetSequence();
	if (!Sequence)
	{
		return;
	}

	const TSharedPtr<INavigationTool> Tool = GetOwnerTool();
	if (!Tool.IsValid())
	{
		return;
	}

	FocusSequence(*Tool, *Sequence);
}

bool FNavigationToolView::CanFocusInContentBrowser() const
{
	if (WeakSelectedItems.Num() != 1)
	{
		return false;
	}

	const FNavigationToolViewModelPtr SelectedItem = WeakSelectedItems[0].Pin();
	if (!SelectedItem.IsValid())
	{
		return false;
	}

	const TViewModelPtr<FNavigationToolSequence> SequenceItem = SelectedItem.ImplicitCast();
	if (!SequenceItem.IsValid())
	{
		return false;
	}

	UMovieSceneSequence* const Sequence = SequenceItem->GetSequence();
	if (!Sequence)
	{
		return false;
	}

	return Cast<ULevelSequence>(Sequence) != nullptr;
}

void FNavigationToolView::FocusInContentBrowser()
{
	if (WeakSelectedItems.Num() != 1)
	{
		return;
	}

	const FNavigationToolViewModelPtr SelectedItem = WeakSelectedItems[0].Pin();
	if (!SelectedItem.IsValid())
	{
		return;
	}

	const TViewModelPtr<FNavigationToolSequence> SequenceItem = SelectedItem.ImplicitCast();
	if (!SequenceItem.IsValid())
	{
		return;
	}

	UMovieSceneSequence* const Sequence = SequenceItem->GetSequence();
	if (!Sequence)
	{
		return;
	}

	ULevelSequence* const LevelSequence = Cast<ULevelSequence>(Sequence);
	if (!LevelSequence)
	{
		return;
	}

	const TArray<UObject*> ObjectsToSync = { LevelSequence };

	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
	ContentBrowserModule.Get().SyncBrowserToAssets(ObjectsToSync);
}

bool FNavigationToolView::UpdateFilters()
{
	if (!bFilterUpdateRequested)
	{
		return false;
	}

	const TSharedPtr<INavigationTool> Tool = GetOwnerTool();
	if (!Tool.IsValid())
	{
		return false;
	}

	const FNavigationToolFilterData PreviousFilterData = FilterBar->GetFilterData();
	const FNavigationToolFilterData& FilterData = FilterBar->FilterNodes();

	//bFilteringOnNodeGroups = Sequencer.GetFocusedMovieSceneSequence()->GetMovieScene()->GetNodeGroups().HasAnyActiveFilter();
	bFilterUpdateRequested = false;

	// Return whether the new list of FilteredNodes is different than the previous list
	return PreviousFilterData != FilterData;
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
