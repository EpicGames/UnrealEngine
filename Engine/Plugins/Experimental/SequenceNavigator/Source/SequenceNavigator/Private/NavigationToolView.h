// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieScene.h"
#include "Filters/NavigationToolFilterBar.h"
#include "INavigationToolView.h"
#include "NavigationToolDefines.h"

class FDragDropEvent;
class FReply;
class FUICommandList;
class SWidget;
class UNavigationToolSettings;
enum class ECheckBoxState : uint8;
enum class EItemDropZone;
struct FGeometry;
struct FNavigationToolSaveState;
struct FPointerEvent;

namespace UE::SequenceNavigator
{

class FNavigationTool;
class FNavigationToolFilterBar;
class FNavigationToolItemContextMenu;
class FNavigationToolProvider;
class INavigationToolColumn;
class SNavigationToolView;

/**
  * A view instance of the Navigation Tool, that handles viewing a subset of the outliner items based on
  * item filters, search text, hierarchy type, etc
  */
class FNavigationToolView : public INavigationToolView
{
private:
	friend FNavigationToolSaveState;
	friend FNavigationTool;

	struct FPrivateToken { explicit FPrivateToken() = default; };

public:
	explicit FNavigationToolView(FPrivateToken);
	virtual ~FNavigationToolView() override;

	/** Creates columns from a specific provider */
	void CreateColumns(const TSharedRef<FNavigationToolProvider>& InProvider);

	void CreateDefaultColumnViews(const TSharedRef<FNavigationToolProvider>& InProvider);

	/**
	 * Creates a Navigation Tool View Instance and register it to the Navigation Tool
	 * @param InToolViewId the Id assigned to the Instance created
	 * @param InTool the Navigation Tool this Instance View belongs to
	 * @param InBaseCommandList the base command list
	 */
	static TSharedRef<FNavigationToolView> CreateInstance(const int32 InToolViewId
		, const TSharedRef<FNavigationTool>& InTool
		, const TSharedPtr<FUICommandList>& InBaseCommandList);

	void PostLoad();

	/** Called when the UNavigationToolSettings has a property change */
	void OnToolSettingsChanged(UObject* const InObject, FPropertyChangedEvent& InPropertyChangedEvent);

	void Tick(const float InDeltaTime);

	void BindCommands(const TSharedPtr<FUICommandList>& InBaseCommandList);

	TSharedPtr<FUICommandList> GetBaseCommandList() const;
	TSharedPtr<FUICommandList> GetViewCommandList() const { return ViewCommandList; }

	/** Notifies FNavigationTool so that this view instance becomes the most recent interacted outliner view */
	void UpdateRecentViews();

	/** Returns whether this view instance is the most recently interacted between the instances in FNavigationTool */
	bool IsMostRecentToolView() const;

	int32 GetToolViewId() const { return ToolViewId; }

	//~ Begin INavigationToolView
	virtual TSharedPtr<INavigationTool> GetOwnerTool() const override;
	virtual TSharedPtr<SWidget> GetToolWidget() const override;
	virtual TSharedPtr<ISequencer> GetSequencer() const override;
	virtual void RequestRefresh() override;
	virtual FOnToolViewRefreshed& GetOnToolViewRefreshed() override { return OnToolViewRefreshed; }
	virtual void SetKeyboardFocus() override;
	virtual ENavigationToolItemViewMode GetItemDefaultViewMode() const override;
	virtual ENavigationToolItemViewMode GetItemProxyViewMode() const override;
	virtual TSharedRef<SWidget> GetColumnMenuContent(const FName InColumnId) override;
	virtual void GetChildrenOfItem(const FNavigationToolViewModelWeakPtr InWeakItem
		, TArray<FNavigationToolViewModelWeakPtr>& OutWeakChildren) const override;
	virtual void GetChildrenOfItem(const FNavigationToolViewModelWeakPtr InWeakItem
		, TArray<FNavigationToolViewModelWeakPtr>& OutWeakChildren
		, const ENavigationToolItemViewMode InViewMode
		, const TSet<FNavigationToolViewModelWeakPtr>& InWeakRecursionDisallowedItems) const override;
	virtual bool IsItemReadOnly(const FNavigationToolViewModelWeakPtr& InWeakItem) const override;
	virtual bool CanSelectItem(const FNavigationToolViewModelWeakPtr& InWeakItem) const override;
	virtual void SelectItems(TArray<FNavigationToolViewModelWeakPtr> InWeakItems
		, const ENavigationToolItemSelectionFlags InFlags = ENavigationToolItemSelectionFlags::SignalSelectionChange) override;
	virtual void ClearItemSelection(const bool bInSignalSelectionChange = true) override;
	virtual bool IsItemSelected(const FNavigationToolViewModelWeakPtr& InWeakItem) const override;
	virtual TArray<FNavigationToolViewModelWeakPtr> GetSelectedItems() const override;
	/*virtual FReply OnDragDetected(const FGeometry& InGeometry
		, const FPointerEvent& InPointerEvent
		, const FNavigationToolViewModelWeakPtr InWeakTargetItem) override;
	virtual FReply OnDrop(const FDragDropEvent& InDragDropEvent
		, const EItemDropZone InDropZone
		, const FNavigationToolViewModelWeakPtr InWeakTargetItem) override;
	virtual TOptional<EItemDropZone> OnCanDrop(const FDragDropEvent& InDragDropEvent
		, const EItemDropZone InDropZone
		, const FNavigationToolViewModelWeakPtr InWeakTargetItem) const override;*/
	virtual bool IsItemExpanded(const FNavigationToolViewModelWeakPtr& InWeakItem, const bool bInUseFilter = true) const override;
	virtual void SetItemExpansion(const FNavigationToolViewModelWeakPtr& InWeakItem
		, const bool bInExpand, const bool bInUseFilter = true) override;
	virtual void SetItemExpansionRecursive(const FNavigationToolViewModelWeakPtr InWeakItem, const bool bInExpand) override;
	virtual void SetParentItemExpansions(const FNavigationToolViewModelWeakPtr& InWeakItem, const bool bInExpand) override;
	virtual bool CanExpandAll() const override;
	virtual void ExpandAll() override;
	virtual void CollapseAll() override;
	virtual bool CanCollapseAll() const override;
	//~ End INavigationToolView

	FReply OnDragDetected(const FGeometry& InGeometry
		, const FPointerEvent& InPointerEvent
		, const FNavigationToolViewModelWeakPtr InWeakTargetItem);
	FReply OnDrop(const FDragDropEvent& InDragDropEvent
		, const EItemDropZone InDropZone
		, const FNavigationToolViewModelWeakPtr InWeakTargetItem);
	TOptional<EItemDropZone> OnCanDrop(const FDragDropEvent& InDragDropEvent
		, const EItemDropZone InDropZone
		, const FNavigationToolViewModelWeakPtr InWeakTargetItem) const;

	TSharedPtr<SWidget> CreateItemContextMenu();

	/** Get the columns currently allocated in this Navigation Tool instance */
	const TMap<FName, TSharedPtr<INavigationToolColumn>>& GetColumns() const { return Columns; }

	/** Whether the given column should be shown by default. This is only used at the start when creating the columns */
	bool ShouldShowColumnByDefault(const TSharedPtr<INavigationToolColumn>& InColumn) const;

	/** Refreshes the items visible in this view and refreshes the Navigation Tool widget if it was created */
	void Refresh();

	/** Refreshes the items that will be at the top level of the tree (e.g. Actors with no parent item) */
	void UpdateRootVisibleItems();

	/** Updates the item expansions in the tree widget based on whether the Items have the expanded flag or not */
	void UpdateItemExpansions();

	/** Called when object replacement has taken place. Used to invalidate the widget for painting */
	void NotifyObjectsReplaced();

	/** The root item of the tree. Convenience function to get the RootItem from FNavigationTool */
	FNavigationToolViewModelPtr GetRootItem() const;

	/** Gets the top level items that should be visible in the tree. This can differ between views depending on filters and other factors */
	const TArray<FNavigationToolViewModelWeakPtr>& GetRootVisibleItems() const;

	void SaveViewItemFlags(const FNavigationToolViewModelWeakPtr& InWeakItem
		, const ENavigationToolItemFlags InFlags);
	
	ENavigationToolItemFlags GetViewItemFlags(const FNavigationToolViewModelWeakPtr& InWeakItem) const;

	FLinearColor GetItemBrushColor(const FNavigationToolViewModelPtr InItem) const;

	/** Gets the Currently Selected Item Count in the Tree View */
	int32 GetViewSelectedItemCount() const;

	/** Calculates the amount of Items that are visible in this Navigation Tool View */
	int32 CalculateVisibleItemCount() const;

	/** Whether Sync Selection is currently taking place */
	bool IsSyncingItemSelection() const { return bSyncingItemSelection; }

	TSharedPtr<FNavigationToolFilterBar> GetFilterBar() const { return FilterBar; }

	/** Called when Item selection has changed. This can be called by the Navigation Tool Widget or by the View itself if there's no ToolWidget */
	void NotifyItemSelectionChanged(const TArray<FNavigationToolViewModelWeakPtr>& InWeakSelectedItems
		, const FNavigationToolViewModelPtr& InItem
		, const bool bInUpdateModeTools);

	/** Return whether the given item should be visible in this Navigation Tool Instance */
	bool ShouldShowItem(const FNavigationToolViewModelWeakPtr& InWeakItem
		, const bool bInUseFilters, const ENavigationToolItemViewMode InViewMode) const;

	/** Returns the Index of the Child from the Parent's Children Visible List (Visible List can vary between Tool Instances) */
	int32 GetVisibleChildIndex(const FNavigationToolViewModelPtr& InParentItem, const FNavigationToolViewModelPtr& InChildItem) const;

	/** Returns the Child from the Parent's Children Visible List at the given Index (Visible List can vary between Tool Instances) */
	FNavigationToolViewModelPtr GetVisibleChildAt(const FNavigationToolViewModelPtr& InParentItem, const int32 InChildIndex) const;

	/** Returns whether the Tool is Locked. See FNavigationTool::IsToolLocked */
	bool IsToolLocked() const;

	/**
	 * Activate an Item Column
	 * @param InColumn the column to activate
	 */
	void ShowColumn(const TSharedPtr<INavigationToolColumn>& InColumn);

	/**
	 * Activate the Item Column with the given ID
	 * @param InColumnId the filter id associated with the Filter to activate
	 */
	void ShowColumnById(const FName& InColumnId);

	/**
	 * Deactivates the given Item Column
	 * @param InColumn the filter to remove from the active filter list
	 */
	void HideColumn(const TSharedPtr<INavigationToolColumn>& InColumn);

	/** Whether the given Column is in the Active Column List */
	virtual bool IsColumnVisible(const TSharedPtr<INavigationToolColumn>& InColumn) const override;

	void ToggleViewModeSupport(ENavigationToolItemViewMode& InOutViewMode, ENavigationToolItemViewMode InFlags);
	void ToggleItemDefaultViewModeSupport(ENavigationToolItemViewMode InFlags);
	void ToggleItemProxyViewModeSupport(ENavigationToolItemViewMode InFlags);

	ECheckBoxState GetViewModeCheckState(ENavigationToolItemViewMode InViewMode, ENavigationToolItemViewMode InFlags) const;
	ECheckBoxState GetItemDefaultViewModeCheckState(ENavigationToolItemViewMode InFlags) const;
	ECheckBoxState GetItemProxyViewModeCheckState(ENavigationToolItemViewMode InFlags) const;
	
	/**
	 * Action to turn Muted Hierarchy on or off
	 * @see bUseMutedHierarchy
	 */
	void ToggleMutedHierarchy();
	bool CanToggleMutedHierarchy() const { return true; }
	bool IsMutedHierarchyActive() const;

	/**
	 * Action to turn Auto Expand to Selection on or off
	 * @see bAutoExpandToSelection
	 */
	void ToggleAutoExpandToSelection();
	bool CanToggleAutoExpandToSelection() const { return true; }
	bool ShouldAutoExpandToSelection() const;

	void ToggleUseShortNames();
	bool CanToggleUseShortNames() const { return true; }
	bool ShouldUseShortNames() const;

	void ToggleShowItemFilters();
	bool CanToggleShowItemFilters() const { return true; }
	bool ShouldShowItemFilters() const { return bShowItemFilters; }

	void ToggleShowItemColumns();
	bool CanToggleShowItemColumns() const { return true; }
	bool ShouldShowItemColumns() const { return bShowItemColumns; }

	/** Sets whether the given item type should be hidden or not */
	void SetItemTypeHidden(const FName InItemTypeName, const bool bInHidden);
	
	/** Toggles the given Item Types to Hide/Show */
	void ToggleHideItemTypes(const FName InItemTypeName);
	ECheckBoxState GetToggleHideItemTypesState(const FName InItemTypeName) const;
	
	/** Hides the given Outliner Item Type from showing in this Outliner View (e.g. Hide Components is used) */
	template<typename InItemType, typename = typename TEnableIf<TIsDerivedFrom<InItemType, INavigationToolItem>::IsDerived>::Type>
	void HideItemType()
	{
		SetItemTypeHidden(InItemType::GetTypeTable().GetTypeName(), true);
	}
	
	/** Shows the given Outliner Item Type if it was hidden in this Outliner View (e.g. Showing Components again) */
	template<typename InItemType, typename = typename TEnableIf<TIsDerivedFrom<InItemType, INavigationToolItem>::IsDerived>::Type>
	void ExposeItemType()
	{
		SetItemTypeHidden(InItemType::GetTypeTable().GetTypeName(), false);
	}
	
	/** Whether the given Outliner Item Type is currently hidden in this Outliner View */
	bool IsItemTypeHidden(const FName InItemTypeName) const;
	bool IsItemTypeHidden(const FNavigationToolViewModelPtr& InItem) const;
	
	template<typename InItemType, typename = typename TEnableIf<TIsDerivedFrom<InItemType, INavigationToolItem>::IsDerived>::Type>
	bool IsItemTypeHidden() const
	{
		return IsItemTypeHidden(InItemType::GetStaticTypeName());
	}

	/** Called when a drag enters the Navigation Tool widgets for the given item (if null, treat as root) */
	void OnDragEnter(const FDragDropEvent& InDragDropEvent
		, const FNavigationToolViewModelWeakPtr InWeakTargetItem);

	/** Called when a drag leaves the Navigation Tool widgets for the given item (if null, treat as root) */
	void OnDragLeave(const FDragDropEvent& InDragDropEvent
		, const FNavigationToolViewModelWeakPtr InWeakTargetItem);

	/** Called when there's a drag being started/ended to the Navigation Tool widget as a whole (i.e. root) and not an individual item */
	void SetDragIntoTreeRoot(const bool bInIsDraggingIntoTreeRoot);

	void RenameSelected();
	void ResetRenaming();
	void OnItemRenameAction(const ENavigationToolRenameAction InRenameAction, const TSharedPtr<INavigationToolView>& InToolView);
	bool CanRenameSelected() const;

	void DeleteSelected();
	bool CanDeleteSelected() const;

	void DuplicateSelected();
	bool CanDuplicateSelected() const;

	void SelectChildren(const bool bInIsRecursive);
	bool CanSelectChildren() const;

	void SelectParent();
	bool CanSelectParent() const;

	void SelectFirstChild();
	bool CanSelectFirstChild() const;

	void SelectSibling(const int32 InDeltaIndex);
	bool CanSelectSibling() const;

	/** Called when the given item's expansion state (expanded/collapsed) has changed */
	void OnItemExpansionChanged(const FNavigationToolViewModelWeakPtr InWeakItem, const bool bInIsExpanded);

	void ScrollNextIntoView();
	void ScrollPrevIntoView();
	bool CanScrollNextIntoView() const;
	void ScrollDeltaIndexIntoView(const int32 InDeltaIndex);
	void ScrollItemIntoView(const FNavigationToolViewModelWeakPtr& InWeakItem);

	/** Sorts the given Items in the user-defined order and selects them, and if the Widget is valid focusing to the item in the Widget */
	void SortAndSelectItems(TArray<FNavigationToolViewModelWeakPtr> InWeakItemsToSelect);

	/** Save the current state of the Navigation Tool */
	void SaveViewState(const TSharedRef<FNavigationToolProvider>& InProvider);
	/** Load the current state of the Navigation Tool */
	void LoadViewState(const TSharedRef<FNavigationToolProvider>& InProvider);

	/**  */
	void SaveColumnState(const FName InColumnId = NAME_None);

	void ResetColumnSize(const FName InColumnId);
	bool CanResetColumnSize(const FName InColumnId) const;

	void ResetVisibleColumnSizes();
	bool CanResetAllColumnSizes() const;

	/** Display a modal dialog for saving a new custom column view */
	void SaveNewCustomColumnView();

	/** Apply a saved custom column view by name */
	void ApplyCustomColumnView(const FText InColumnViewName);

protected:
	/** Initializes the Navigation Tool Instance. Only executed once */
	void Init(const TSharedRef<FNavigationTool>& InTool, const TSharedPtr<FUICommandList>& InBaseCommandList);

	/** Sets the Selected Items to be the given Items in this List. */
	void SetItemSelectionImpl(TArray<FNavigationToolViewModelWeakPtr>&& InWeakItems, const bool bSignalSelectionChange);

	/** Triggers a Refresh on the FNavigationTool */
	void RefreshTool(const bool bInImmediateRefresh);

	void EnsureToolViewCount(const int32 InToolViewId) const;

	/**  */
	void SaveFilterState(FNavigationToolViewSaveState& OutViewSaveState);
	/**  */
	void LoadFilterState(const FNavigationToolViewSaveState& InViewSaveState
		, const bool bInDisableAllFilters = true
		, const bool bInRequestFilterUpdate = true);

	/** Save the Item State in the Navigation Tool Widget (e.g. Item Scoped Flags) */
	void SaveToolViewItems(FNavigationToolViewSaveState& OutViewSaveState);
	/** Load the Item State in the Navigation Tool Widget (e.g. Item Scoped Flags) */
	void LoadToolViewItems(FNavigationToolViewSaveState& InViewSaveState);

	bool CanFocusSingleSelection() const;
	void FocusSingleSelection();

	bool CanFocusInContentBrowser() const;
	void FocusInContentBrowser();

	bool UpdateFilters();

	/** Local Identifier of this Instance */
	int32 ToolViewId = -1;

	/** Weak pointer to the outliner this is a view of */
	TWeakPtr<FNavigationTool> WeakTool;

	/** Widget showing the View. Can be null if instanced for testing (i.e. bCreateToolWidget is false in Init) */
	TSharedPtr<SNavigationToolView> ToolViewWidget;

	/** Command List mapped to the View to handle things like Selected Items */
	TSharedPtr<FUICommandList> ViewCommandList;

	/** Settings for the tool */
	TWeakObjectPtr<UNavigationToolSettings> WeakToolSettings;

	/** Root Items from Navigation Tool visible to this Instance. i.e. a Subset of all the Items in FNavigationTool */
	TArray<FNavigationToolViewModelWeakPtr> WeakRootVisibleItems;

	/** A list of the Selected Items in this Navigation Tool View */
	TArray<FNavigationToolViewModelWeakPtr> WeakSelectedItems;

	/** Set of Items that are Currently Read Only in this Instance. Mutable as this changes on GetChildrenOfItem const func */
	mutable TSet<FNavigationToolViewModelWeakPtr> WeakReadOnlyItems;

	/** A set of the Navigation Tool Item Type Names that should be hidden */
	TSet<FName> HiddenItemTypes;

	TSharedPtr<FNavigationToolFilterBar> FilterBar;

	/** Delegate called at the end of the Navigation Tool View Refresh */
	FOnToolViewRefreshed OnToolViewRefreshed;

	/** Map of the Column Ids and the Column with that Id. This is filled in FNavigationToolView::CreateColumns */
	TMap<FName, TSharedPtr<INavigationToolColumn>> Columns;

	/** Right click context menu for items */
	TSharedPtr<FNavigationToolItemContextMenu> ItemContextMenu;

	/** The Index of the Item that will be scrolled into View */
	int32 NextSelectedItemIntoView = -1;

	/** Selected Items Sorted from Top To Bottom */
	TArray<FNavigationToolViewModelWeakPtr> WeakSortedSelectedItems;

	/** The list of items that need renaming and are waiting for their turn to be renamed */
	TArray<FNavigationToolViewModelWeakPtr> WeakItemsRemainingRename;
	
	/** The current item in the process of renaming. Null if no renaming is taking place */
	FNavigationToolViewModelWeakPtr WeakCurrentItemRenaming;
	
	/** Flag to call FNavigationToolView::Refresh next tick */
	bool bRefreshRequested = false;
	
	/** Whether Item renaming is taking place in this View */
	bool bRenamingItems = false;
	
	/** Whether Renaming Items should be processed next tick */
	bool bRequestedRename = false;

	/** Whether Items are currently being synced to what the Widget or Interface sent. Used as a re-enter guard */
	bool bSyncingItemSelection = false;
	
	/** Flag used for the Navigation Tool Widget to determine if the Item Filter Bar should open and show the Item Filters */
	bool bShowItemFilters = false;

	bool bShowItemColumns = false;

	bool bFilterUpdateRequested = false;
};

} // namespace UE::SequenceNavigator
