// Copyright Epic Games, Inc. All Rights Reserved.

#include "Columns/INavigationToolColumn.h"
#include "Extensions/IColorExtension.h"
#include "Extensions/IIdExtension.h"
#include "Extensions/IInTimeExtension.h"
#include "Extensions/IMarkerVisibilityExtension.h"
#include "Extensions/IOutTimeExtension.h"
#include "Extensions/IPlayheadExtension.h"
#include "Extensions/IRevisionControlExtension.h"
#include "ItemActions/INavigationToolItemAction.h"
#include "Items/INavigationToolItem.h"
#include "MVVM/ICastable.h"
#include "MVVM/ViewModelTypeID.h"

namespace UE::SequenceNavigator
{

UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(INavigationToolItem)
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(INavigationToolColumn)

UE_SEQUENCER_DEFINE_CASTABLE(INavigationToolItemAction)

UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(IColorExtension)
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(IIdExtension)
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(IInTimeExtension)
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(IMarkerVisibilityExtension)
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(IOutTimeExtension)
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(IPlayheadExtension)
UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(IRevisionControlExtension)

} // namespace UE::SequenceNavigator
