// Copyright Epic Games, Inc. All Rights Reserved.

#include "Providers/NavigationToolProvider.h"
#include "Filters/Filters/NavigationToolBuiltInFilterParams.h"
#include "Items/NavigationToolItemUtils.h"
#include "NavigationTool.h"
#include "NavigationToolSettings.h"
#include "NavigationToolView.h"
#include "SequenceNavigatorLog.h"

namespace UE::SequenceNavigator
{

using namespace Sequencer;

void FNavigationToolProvider::OnExtendColumnViews(TSet<FNavigationToolColumnView>& OutColumnViews)
{
	INavigationToolProvider::OnExtendColumnViews(OutColumnViews);

	ExtendedColumnViewNames.Reset(OutColumnViews.Num());
	for (const FNavigationToolColumnView& ColumnView : OutColumnViews)
	{
		ExtendedColumnViewNames.Add(ColumnView.ViewName);
	}
}

void FNavigationToolProvider::OnExtendBuiltInFilters(TArray<FNavigationToolBuiltInFilterParams>& OutFilterParams)
{
	INavigationToolProvider::OnExtendBuiltInFilters(OutFilterParams);

	ExtendedBuiltInFilterNames.Reset(OutFilterParams.Num());
	for (const FNavigationToolBuiltInFilterParams& FilterParams : OutFilterParams)
	{
		ExtendedBuiltInFilterNames.Add(FilterParams.GetFilterId());
	}
}

void FNavigationToolProvider::UpdateItemIdContexts(const INavigationTool& InTool)
{
	FNavigationToolSaveState* const SaveState = GetSaveState(InTool);
	if (!SaveState)
	{
		return;
	}

	const FString ContextString = GetIdentifier().ToString();

	// Already updated
	if (SaveState->ContextPath == ContextString)
	{
		return;
	}

	ON_SCOPE_EXIT
	{
		SaveState->ContextPath = ContextString;
	};

	auto FixItemIdMap = [&ContextPath = SaveState->ContextPath, &ContextString]<typename InValueType>(TMap<FString, InValueType>& InItemIdMap)
		{
			TMap<FString, InValueType> ItemIdMapTemp = InItemIdMap;
			for (TPair<FString, InValueType>& Pair : ItemIdMapTemp)
			{
				FSoftObjectPath ObjectPath;
				ObjectPath.SetPath(ContextPath);

				FString AssetPath = ObjectPath.GetAssetPath().ToString();
				if (AssetPath.IsEmpty() || Pair.Key.StartsWith(AssetPath))
				{
					InItemIdMap.Remove(Pair.Key);

					Pair.Key.RemoveFromStart(AssetPath);
					Pair.Key.RemoveFromStart(TEXT(":"));

					FSoftObjectPath NewPath;
					NewPath.SetPath(ContextString);
					NewPath.SetSubPathString(Pair.Key);

					InItemIdMap.Add(NewPath.ToString(), Pair.Value);
				}
			}
		};

	FixItemIdMap(SaveState->ItemColorMap);

	for (FNavigationToolViewSaveState& ViewState : SaveState->ToolViewSaveStates)
	{
		FixItemIdMap(ViewState.ViewItemFlags);
	}
}

FNavigationToolViewSaveState* FNavigationToolProvider::GetViewSaveState(const INavigationTool& InTool, const int32 InToolViewId) const
{
	FNavigationToolSaveState* const SaveState = GetSaveState(InTool);
	if (!SaveState)
	{
		UE_LOGF(LogSequenceNavigator, Warning, "GetViewSaveState(): Save state is NULL!");
		return nullptr;
	}

	if (!SaveState->ToolViewSaveStates.IsValidIndex(InToolViewId))
	{
		UE_LOGF(LogSequenceNavigator, Warning, "GetViewSaveState(): Invalid tool view Id: %d", InToolViewId);
		return nullptr;
	}

	return &SaveState->ToolViewSaveStates[InToolViewId];
}

void FNavigationToolProvider::EnsureToolViewCount(const INavigationTool& InTool, const int32 InToolViewId)
{
	FNavigationToolSaveState* const SaveState = GetSaveState(InTool);
	if (!SaveState)
	{
		UE_LOGF(LogSequenceNavigator, Warning, "EnsureToolViewCount(): Save state is NULL!");
		return;
	}

	const int32 CurrentCount = SaveState->ToolViewSaveStates.Num();
	const int32 MinViewCount = InToolViewId + 1;

	if (CurrentCount < MinViewCount)
	{
		SaveState->ToolViewSaveStates.AddDefaulted(MinViewCount - CurrentCount);
	}
}

TOptional<EItemDropZone> FNavigationToolProvider::OnToolItemCanAcceptDrop(const FDragDropEvent& InDragDropEvent
	, EItemDropZone InDropZone
	, const FNavigationToolViewModelPtr& InTargetItem) const
{
	return TOptional<EItemDropZone>();
}

FReply FNavigationToolProvider::OnToolItemAcceptDrop(const FDragDropEvent& InDragDropEvent
	, EItemDropZone InDropZone
	, const FNavigationToolViewModelPtr& InTargetItem)
{
	return FReply::Unhandled();
}

void FNavigationToolProvider::Activate(FNavigationTool& InTool)
{
	const FName ProviderId = GetIdentifier();
	UE_LOGF(LogSequenceNavigator, Log, "Activating provider: %ls", *ProviderId.ToString());

	if (const TSharedPtr<FUICommandList> BaseCommandList = InTool.GetBaseCommandList())
	{
		BindCommands(BaseCommandList.ToSharedRef());
	}

	LoadState(InTool);

	const TSharedRef<FNavigationToolProvider> SharedThisRef = SharedThis(this);

	InTool.ForEachToolView([&SharedThisRef](const TSharedRef<FNavigationToolView>& InToolView)
		{
			InToolView->CreateColumns(SharedThisRef);
			InToolView->CreateDefaultColumnViews(SharedThisRef);
		});

	InTool.RefreshColumnView();

	OnActivate();
}

void FNavigationToolProvider::Deactivate(FNavigationTool& InTool)
{
	const FName ProviderId = GetIdentifier();
	UE_LOGF(LogSequenceNavigator, Log, "Deactivating provider: %ls", *ProviderId.ToString());

	CleanupExtendedColumnViews();

	OnDeactivate();
}

void FNavigationToolProvider::CleanupExtendedColumnViews()
{
	UNavigationToolSettings* const ToolSettings = GetMutableDefault<UNavigationToolSettings>();
	if (!ToolSettings)
	{
		return;
	}

	TSet<FNavigationToolColumnView>& CustomColumnViews = ToolSettings->GetCustomColumnViews();

	for (const FText& ColumnViewName : ExtendedColumnViewNames)
	{
		CustomColumnViews.Remove(ColumnViewName);
	}

	ToolSettings->SaveConfig();
}

bool FNavigationToolProvider::IsSequenceSupported(UMovieSceneSequence* const InSequence) const
{
	if (InSequence)
	{
		return GetSupportedSequenceClasses().Contains(InSequence->GetClass());
	}
	return false;
}

void FNavigationToolProvider::SaveState(FNavigationTool& InTool)
{
	const FName ProviderId = GetIdentifier();
	UE_LOGF(LogSequenceNavigator, Log, "Saving provider state: %ls", *ProviderId.ToString());

	SaveSerializedTree(InTool, /*bInResetTree*/true);

	const TSharedRef<FNavigationToolProvider> SharedThisRef = SharedThis(this);

	InTool.ForEachToolView([&SharedThisRef](const TSharedPtr<FNavigationToolView>& InToolView)
		{
			InToolView->SaveViewState(SharedThisRef);
		});

	if (FNavigationToolSaveState* const SaveState = GetSaveState(InTool))
	{
		// Remove any saved item colors that can no longer be found
		TArray<FString> ItemIds;
		SaveState->ItemColorMap.GetKeys(ItemIds);
		for (const FString& ItemId : ItemIds)
		{
			if (!InTool.FindItem(ItemId).IsValid())
			{
				SaveState->ItemColorMap.Remove(ItemId);
			}
		}
	}

	UpdateItemIdContexts(InTool);
}

void FNavigationToolProvider::LoadState(FNavigationTool& InTool)
{
	const FNavigationToolViewModelPtr RootItem = InTool.GetTreeRoot().Pin();
	if (!RootItem.IsValid())
	{
		return;
	}

	FNavigationToolSaveState* const SaveState = GetSaveState(InTool);
	if (!SaveState)
	{
		return;
	}

	const UNavigationToolSettings* const ToolSettings = GetDefault<UNavigationToolSettings>();
	if (!ToolSettings)
	{
		return;
	}

	const FName ProviderId = GetIdentifier();
	UE_LOGF(LogSequenceNavigator, Log, "Loading provider state: %ls", *ProviderId.ToString());

	UpdateItemIdContexts(InTool);

	LoadSerializedTree(RootItem, &SaveState->SerializedTree);

	const TSharedRef<FNavigationToolProvider> SharedThisRef = SharedThis(this);

	InTool.ForEachToolView([&SharedThisRef](const TSharedPtr<FNavigationToolView>& InToolView)
		{
			InToolView->LoadViewState(SharedThisRef);
		});
}

void FNavigationToolProvider::SaveSerializedTree(FNavigationTool& InTool, const bool bInResetTree)
{
	const FNavigationToolViewModelPtr RootItem = InTool.GetTreeRoot().Pin();
	if (!RootItem.IsValid())
	{
		return;
	}

	FNavigationToolSaveState* const SaveState = GetSaveState(InTool);
	if (!SaveState)
	{
		return;
	}

	if (bInResetTree)
	{
		SaveState->SerializedTree.Reset();
	}

	SaveSerializedTreeRecursive(RootItem, SaveState->SerializedTree);
}

void FNavigationToolProvider::SaveSerializedTreeRecursive(const FNavigationToolViewModelPtr& InParentItem
	, FNavigationToolSerializedTree& InSerializedTree)
{
	const TArray<FNavigationToolViewModelWeakPtr>& WeakChildren = InParentItem->GetChildren();

	const FNavigationToolSerializedItem ParentSceneItem = InParentItem->MakeSerializedItem();

	for (int32 Index = 0; Index < WeakChildren.Num(); ++Index)
	{
		if (const FNavigationToolViewModelPtr ChildItem = WeakChildren[Index].Pin())
		{
			if (ChildItem->ShouldSort())
			{
				const FNavigationToolSerializedItem SceneItem = ChildItem->MakeSerializedItem();
				if (SceneItem.IsValid())
				{
					InSerializedTree.GetOrAddTreeNode(SceneItem, ParentSceneItem);
				}
			}
			SaveSerializedTreeRecursive(ChildItem, InSerializedTree);
		}
	}
}

void FNavigationToolProvider::LoadSerializedTree(const FNavigationToolViewModelPtr& InParentItem
	, FNavigationToolSerializedTree* const InSerializedTree)
{
	using namespace ItemUtils;

	TArray<FNavigationToolViewModelWeakPtr>& WeakChildren = InParentItem->GetChildrenMutable();
	
	TArray<FNavigationToolViewModelWeakPtr> WeakSortable;
	TArray<FNavigationToolViewModelWeakPtr> WeakUnsortable;
	SplitSortableAndUnsortableItems(WeakChildren, WeakSortable, WeakUnsortable);

	// If Scene Tree is valid, Item Sorting should be empty as this function only takes a valid Scene Tree if
	// loaded version supports Scene Trees (i.e. when Item Sorting stops being loaded in)
	if (InSerializedTree)
	{
		WeakSortable.Sort([InSerializedTree](const FNavigationToolViewModelWeakPtr& InWeakItemA, const FNavigationToolViewModelWeakPtr& InWeakItemB)
			{
				const TViewModelPtr<INavigationToolItem> ItemA = InWeakItemA.Pin();
				const TViewModelPtr<INavigationToolItem> ItemB = InWeakItemB.Pin();
				const FNavigationToolSerializedTreeNode* const NodeA = ItemA.IsValid()
					? InSerializedTree->FindTreeNode(ItemA->MakeSerializedItem()) : nullptr;
				const FNavigationToolSerializedTreeNode* const NodeB = ItemB.IsValid()
					? InSerializedTree->FindTreeNode(ItemB->MakeSerializedItem()) : nullptr;
				return FNavigationToolSerializedTree::CompareTreeItemOrder(NodeA, NodeB);
			});
	}

	WeakChildren = MoveTemp(WeakUnsortable);
	WeakChildren.Append(MoveTemp(WeakSortable));

	for (FNavigationToolViewModelWeakPtr& WeakChild : WeakChildren)
	{
		LoadSerializedTree(WeakChild.Pin(), InSerializedTree);
	}
}

} // namespace UE::SequenceNavigator
