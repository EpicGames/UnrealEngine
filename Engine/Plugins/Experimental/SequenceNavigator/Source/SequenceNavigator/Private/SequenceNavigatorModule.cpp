// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequenceNavigatorModule.h"
#include "Extensions/IColorExtension.h"
#include "Extensions/IInTimeExtension.h"
#include "Extensions/IMarkerVisibilityExtension.h"
#include "Extensions/IOutTimeExtension.h"
#include "Extensions/IPlayheadExtension.h"
#include "Extensions/IRevisionControlExtension.h"
#include "Items/INavigationToolItem.h"
#include "MVVM/ViewModelTypeID.h"
#include "NavigationToolCommands.h"
#include "SequenceNavigatorLog.h"

DEFINE_LOG_CATEGORY(LogSequenceNavigator);
/*
namespace UE::SequenceNavigator
{
	UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(INavigationToolItem)
	UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(IColorExtension)
	UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(IInTimeExtension)
	UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(IMarkerVisibilityExtension)
	UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(IOutTimeExtension)
	UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(IPlayheadExtension)
	UE_SEQUENCER_DEFINE_VIEW_MODEL_TYPE_ID(IRevisionControlExtension)
}*/

void FSequenceNavigatorModule::StartupModule()
{
	using namespace UE::SequenceNavigator;

	FNavigationToolCommands::Register();
}

void FSequenceNavigatorModule::ShutdownModule()
{
	using namespace UE::SequenceNavigator;

	FNavigationToolCommands::Unregister();
}

IMPLEMENT_MODULE(FSequenceNavigatorModule, SequenceNavigator)
