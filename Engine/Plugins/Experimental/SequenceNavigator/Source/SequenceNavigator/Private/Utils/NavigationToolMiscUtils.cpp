// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavigationToolMiscUtils.h"
#include "EngineAnalytics.h"
#include "INavigationTool.h"
#include "ISourceControlModule.h"
#include "ISourceControlProvider.h"
#include "Items/NavigationToolBinding.h"
#include "Items/NavigationToolSequence.h"
#include "Items/NavigationToolTrack.h"
#include "MovieScene.h"
#include "MVVM/ObjectBindingModelStorageExtension.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/ViewModels/ObjectBindingModel.h"
#include "MVVM/ViewModels/TrackModel.h"
#include "MVVM/Views/SOutlinerView.h"
#include "Sections/MovieSceneSubSection.h"
#include "UObject/Package.h"

namespace UE::SequenceNavigator
{

using namespace Sequencer;

void RecordUsageAnalytics(const FString& InParamName, const FString& InParamValue)
{
	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.SequenceNavigator")
			, InParamName, InParamValue);
	}
}

FSourceControlStatePtr FindSourceControlState(UPackage* const InPackage)
{
	if (!InPackage)
	{
		return nullptr;
	}

	ISourceControlModule& SourceControlModule = ISourceControlModule::Get();
	if (!SourceControlModule.IsEnabled())
	{
		return nullptr;
	}

	ISourceControlProvider& SourceControlProvider = SourceControlModule.GetProvider();
	if (!SourceControlProvider.IsAvailable())
	{
		return nullptr;
	}

	return SourceControlProvider.GetState(InPackage, EStateCacheUsage::Use);
}

const FSlateBrush* FindSourceControlStatusBrush(UPackage* const InPackage)
{
	if (const FSourceControlStatePtr SourceControlState = FindSourceControlState(InPackage))
	{
		const FSlateIcon Icon = SourceControlState->GetIcon();
		if (Icon.IsSet())
		{
			return Icon.GetIcon();
		}
	}
	return nullptr;
}

FText FindSourceControlStatusText(UPackage* const InPackage)
{
	if (const FSourceControlStatePtr SourceControlState = FindSourceControlState(InPackage))
	{
		return SourceControlState->GetDisplayTooltip();
	}
	return FText::GetEmpty();
}

void FocusSequence(const INavigationTool& InTool, UMovieSceneSequence& InSequence)
{
	const TSharedPtr<ISequencer> Sequencer = InTool.GetSequencer();
	if (!Sequencer.IsValid())
	{
		return;
	}

	if (Sequencer->GetRootMovieSceneSequence() == &InSequence)
	{
		Sequencer->ResetToNewRootSequence(InSequence);
		return;
	}

	if (Sequencer->GetFocusedMovieSceneSequence() == &InSequence)
	{
		return;
	}

	FMovieSceneEvaluationState* const EvaluationState = Sequencer->GetEvaluationState();
	if (!EvaluationState)
	{
		return;
	}

	FMovieSceneSequenceIDRef SequenceID = EvaluationState->FindSequenceId(&InSequence);
	UMovieSceneSubSection* const SubSection = Sequencer->FindSubSection(SequenceID);
	if (!SubSection)
	{
		return;
	}

	Sequencer->FocusSequenceInstance(*SubSection);
}

void FocusSequence(const INavigationTool& InTool
	, UMovieSceneSequence& InSequence
	, const FNavigationToolSequence& InSequenceItem)
{
	FocusSequence(InTool, InSequence);

	if (const TSharedPtr<ISequencer> Sequencer = InTool.GetSequencer())
	{
		if (UMovieSceneSubSection* const SubSection = InSequenceItem.GetSubSection())
		{
			Sequencer->EmptySelection();
			Sequencer->SelectSection(SubSection);
		}
	}

	RecordUsageAnalytics(TEXT("FocusItemInSequencer"), TEXT("SubSection"));
}

void FocusSequence(const INavigationTool& InTool
	, UMovieSceneSequence& InSequence
	, const FNavigationToolTrack& InTrackItem)
{
	using namespace Sequencer;

	FocusSequence(InTool, InSequence);

	if (const TSharedPtr<ISequencer> Sequencer = InTool.GetSequencer())
	{
		if (UMovieSceneTrack* const Track = InTrackItem.GetTrack())
		{
			Sequencer->EmptySelection();
			Sequencer->SelectTrack(Track);

			if (const TSharedPtr<SOutlinerView> OutlinerView = Sequencer->GetOutlinerViewWidget())
			{
				if (const TViewModelPtr<FTrackModel> TrackModel = InTrackItem.GetViewModel())
				{
					OutlinerView->ExpandCollapseNode(TrackModel, true, ETreeRecursion::NonRecursive);
					OutlinerView->RequestScrollIntoView(TrackModel);
				}
			}
		}
	}

	RecordUsageAnalytics(TEXT("FocusItemInSequencer"), TEXT("Track"));
}

void FocusSequence(const INavigationTool& InTool
	, UMovieSceneSequence& InSequence
	, const FNavigationToolBinding& InBindingItem)
{
	using namespace Sequencer;

	FocusSequence(InTool, InSequence);

	if (const TSharedPtr<ISequencer> Sequencer = InTool.GetSequencer())
	{
		SelectSequencerBindingTrack(*Sequencer, InBindingItem.GetBinding().GetObjectGuid());

		if (const TSharedPtr<SOutlinerView> OutlinerView = Sequencer->GetOutlinerViewWidget())
		{
			if (const TViewModelPtr<FObjectBindingModel> ObjectBindingViewModel = InBindingItem.GetViewModel())
			{
				OutlinerView->ExpandCollapseNode(ObjectBindingViewModel, true, ETreeRecursion::NonRecursive);
				OutlinerView->RequestScrollIntoView(ObjectBindingViewModel);
			}
		}
	}

	RecordUsageAnalytics(TEXT("FocusItemInSequencer"), TEXT("ObjectBinding"));
}

void SelectSequencerBindingTrack(const ISequencer& InSequencer, const FGuid& InObjectId)
{
	using namespace Sequencer;

	const TSharedPtr<FSequencerEditorViewModel> ViewModel = InSequencer.GetViewModel();
	if (!ViewModel.IsValid())
	{
		return;
	}

	const TSharedPtr<FSequencerSelection> Selection = ViewModel->GetSelection();
	if (!Selection.IsValid())
	{
		return;
	}

	const FViewModelPtr RootViewModel = ViewModel->GetRootModel();
	if (!Selection.IsValid())
	{
		return;
	}

	const FObjectBindingModelStorageExtension* const ObjectBindingStorage = RootViewModel->CastDynamic<FObjectBindingModelStorageExtension>();
	if (!ObjectBindingStorage)
	{
		return;
	}

	const TSharedPtr<FObjectBindingModel> ObjectBindingModel = ObjectBindingStorage->FindModelForObjectBinding(InObjectId);
	if (!ObjectBindingModel.IsValid())
	{
		return;
	}

	Selection->Empty();
	Selection->Outliner.Select(ObjectBindingModel);
}

void FocusSequence(const INavigationTool& InTool
	, UMovieSceneSequence& InSequence
	, const FMovieSceneMarkedFrame& InMarkedFrame)
{
	FocusSequence(InTool, InSequence);

	if (const TSharedPtr<ISequencer> Sequencer = InTool.GetSequencer())
	{
		Sequencer->SetGlobalTime(InMarkedFrame.FrameNumber);
	}
}

void FocusItemInSequencer(const INavigationTool& InTool, const FNavigationToolViewModelPtr& InItem)
{
	if (const TViewModelPtr<FNavigationToolSequence> SequenceItem = InItem.ImplicitCast())
	{
		if (UMovieSceneSequence* const ParentSequence = SequenceItem->GetSequence())
		{
			FocusSequence(InTool, *ParentSequence);
		}
	}
	else if (const TViewModelPtr<FNavigationToolTrack> TrackItem = InItem.ImplicitCast())
	{
		if (UMovieSceneSequence* const Sequence = TrackItem->GetSequence())
		{
			FocusSequence(InTool, *Sequence, *TrackItem);
		}
	}
	else if (const TViewModelPtr<FNavigationToolBinding> BindingItem = InItem.ImplicitCast())
	{
		if (UMovieSceneSequence* const Sequence = BindingItem->GetSequence())
		{
			FocusSequence(InTool, *Sequence, *BindingItem);
		}
	}
}

FMovieSceneSequenceID ResolveSequenceID(const ISequencer& InSequencer, UMovieSceneSequence* const InSequence)
{
	if (FMovieSceneEvaluationState* const EvaluationState = InSequencer.GetSharedPlaybackState()->FindCapability<FMovieSceneEvaluationState>())
	{
		return EvaluationState->FindSequenceId(InSequence);
	}
	return FMovieSceneSequenceID();
}

TArrayView<TWeakObjectPtr<>> ResolveBoundObjects(const ISequencer& InSequencer
	, UMovieSceneSequence* const InSequence, const FGuid& InBindingId)
{
	const TSharedRef<const MovieScene::FSharedPlaybackState> SharedPlaybackState = InSequencer.GetSharedPlaybackState();

	if (FMovieSceneEvaluationState* const EvaluationState = SharedPlaybackState->FindCapability<FMovieSceneEvaluationState>())
	{
		FMovieSceneSequenceIDRef SequenceID = EvaluationState->FindSequenceId(InSequence);
		return EvaluationState->FindBoundObjects(InBindingId, SequenceID, SharedPlaybackState);
	}

	return TArrayView<TWeakObjectPtr<>>();
}

} // namespace UE::SequenceNavigator
