// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISequencer.h"
#include "ISourceControlState.h"
#include "NavigationToolDefines.h"

class FNavigationToolBinding;
class FNavigationToolSequence;
class FNavigationToolTrack;
class FText;
class ISequencer;
class UMovieSceneSequence;
class UPackage;
struct FGuid;
struct FMovieSceneSequenceID;
struct FSlateBrush;

namespace UE::SequenceNavigator
{

class INavigationTool;

/**
 * Records usage analytics for the Sequence Navigator tool.
 * @param InParamName Parameter name string
 * @param InParamValue Parameter value string
 */
void RecordUsageAnalytics(const FString& InParamName, const FString& InParamValue);

/**
 * Find the source control state for a given package.
 * @param InPackage The package for which the source control state is being queried.
 * @return The source control state if available, otherwise nullptr.
 */
FSourceControlStatePtr FindSourceControlState(UPackage* const InPackage);

/**
 * Find the source control status brush associated with a given package.
 * @param InPackage The package for which the source control status brush is being queried.
 * @return A pointer to the slate brush representing the source control status, or nullptr if unavailable.
 */
const FSlateBrush* FindSourceControlStatusBrush(UPackage* const InPackage);

/**
 * Find the source control status text for the specified package.
 * @param InPackage The package for which the source control status text is being requested.
 * @return The tooltip text representing the source control status if available, otherwise an empty text.
 */
FText FindSourceControlStatusText(UPackage* const InPackage);

/**
 * Focuses a sub-movie scene (MovieScene within a MovieScene) in the sequencer.
 * @param InTool The Navigation Tool that manages scene sequences.
 * @param InSequence The movie scene sequence to focus on.
 */
void FocusSequence(const INavigationTool& InTool, UMovieSceneSequence& InSequence);

/**
 * Focuses a sub-movie scene (MovieScene within a MovieScene) in the sequencer.
 * @param InTool The Navigation Tool that manages scene sequences.
 * @param InSequence The movie scene sequence to focus on.
 * @param InSequenceItem Optional sequence item to select in the sequence.
 */
void FocusSequence(const INavigationTool& InTool
	, UMovieSceneSequence& InSequence
	, const FNavigationToolSequence& InSequenceItem);

/**
 * Focuses a sub-movie scene (MovieScene within a MovieScene) in the sequencer.
 * @param InTool The Navigation Tool that manages scene sequences.
 * @param InSequence The movie scene sequence to focus on.
 * @param InTrackItem Optional track item to select in the sequence.
 */
void FocusSequence(const INavigationTool& InTool
	, UMovieSceneSequence& InSequence
	, const FNavigationToolTrack& InTrackItem);

/**
 * Focuses a sub-movie scene (MovieScene within a MovieScene) in the sequencer.
 * @param InTool The Navigation Tool that manages scene sequences.
 * @param InSequence The movie scene sequence to focus on.
 * @param InBindingItem Optional binding item to select in the sequence.
 */
void FocusSequence(const INavigationTool& InTool
	, UMovieSceneSequence& InSequence
	, const FNavigationToolBinding& InBindingItem);

/** Selects an object binding track in a sequencer. */
void SelectSequencerBindingTrack(const ISequencer& InSequencer, const FGuid& InObjectId);

/**
 * Focuses a sub-movie scene (MovieScene within a MovieScene) in the sequencer.
 * @param InTool The Navigation Tool that manages scene sequences.
 * @param InSequence The movie scene sequence to focus on.
 * @param InMarkedFrame Optional marked frame to select in the sequence.
 */
void FocusSequence(const INavigationTool& InTool
	, UMovieSceneSequence& InSequence
	, const FMovieSceneMarkedFrame& InMarkedFrame);

void FocusItemInSequencer(const INavigationTool& InTool
	, const FNavigationToolViewModelPtr& InItem);

/**
 * Resolve the sequence ID for a given sequence within the provided sequencer.
 * @param InSequencer The sequencer instance to retrieve the sequence ID from.
 * @param InSequence The sequence whose ID is to be resolved.
 * @return The resolved sequence ID if found, otherwise an empty sequence ID.
 */
FMovieSceneSequenceID ResolveSequenceID(const ISequencer& InSequencer, UMovieSceneSequence* const InSequence);

/**
 * Resolves the objects bound to a specified binding ID within a sequence.
 * @param InSequencer The sequencer instance used for evaluating bindings and sequences.
 * @param InSequence The movie scene sequence to search for bindings.
 * @param InBindingId The binding ID whose bound objects are being resolved.
 * @return A view of the objects bound to the specified binding ID in the specified sequence.
 */
TArrayView<TWeakObjectPtr<>> ResolveBoundObjects(const ISequencer& InSequencer
	, UMovieSceneSequence* const InSequence, const FGuid& InBindingId);

} // namespace UE::SequenceNavigator
