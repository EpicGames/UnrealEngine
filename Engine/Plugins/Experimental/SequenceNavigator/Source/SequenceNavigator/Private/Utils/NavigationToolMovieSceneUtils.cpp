// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavigationToolMovieSceneUtils.h"
#include "ISequencer.h"
#include "Misc/FrameTime.h"
#include "MovieScene.h"
#include "Sections/MovieSceneSubSection.h"
#include "SequencerUtilities.h"

namespace UE::SequenceNavigator
{

FFrameTime ConvertToDisplayRateTime(const UMovieSceneSequence& InSequence, const FFrameTime& InTime)
{
	if (UMovieScene* const MovieScene = InSequence.GetMovieScene())
	{
		return ConvertFrameTime(InTime
			, MovieScene->GetTickResolution()
			, MovieScene->GetDisplayRate());
	}
	return 0;
}

FFrameTime ConvertToTickResolutionTime(const UMovieSceneSequence& InSequence, const FFrameTime& InTime)
{
	if (UMovieScene* const MovieScene = InSequence.GetMovieScene())
	{
		return ConvertFrameTime(InTime
			, MovieScene->GetDisplayRate()
			, MovieScene->GetTickResolution());
	}
	return 0;
}

UMovieSceneSubSection* FindSequenceSubSection(ISequencer& InSequencer, UMovieSceneSequence* const InSequence)
{
	FMovieSceneEvaluationState* const EvaluationState = InSequencer.GetEvaluationState();
	if (!EvaluationState)
	{
		return nullptr;
	}

	FMovieSceneSequenceIDRef SequenceID = EvaluationState->FindSequenceId(InSequence);

	return InSequencer.FindSubSection(SequenceID);
}

bool IsGloballyMarkedFramesForSequence(UMovieSceneSequence* const InSequence)
{
	if (!InSequence)
	{
		return false;
	}

	const UMovieScene* const MovieScene = InSequence->GetMovieScene();
	if (!MovieScene)
	{
		return false;
	}

	return MovieScene->GetGloballyShowMarkedFrames();
}

void ShowGloballyMarkedFramesForSequence(ISequencer& InSequencer, UMovieSceneSequence* const InSequence, const bool bInVisible)
{
	if (!InSequence)
	{
		return;
	}

	UMovieScene* const MovieScene = InSequence->GetMovieScene();
	if (!MovieScene)
	{
		return;
	}

	if (MovieScene->IsReadOnly())
	{
		FSequencerUtilities::ShowReadOnlyError();
		return;
	}

	InSequence->Modify();
	MovieScene->Modify();

	MovieScene->SetGloballyShowMarkedFrames(bInVisible);

	//InSequencer.InvalidateGlobalMarkedFramesCache();
}

void ModifySequenceAndMovieScene(UMovieSceneSequence* const InSequence)
{
	if (!InSequence)
	{
		return;
	}

	UMovieScene* const MovieScene = InSequence->GetMovieScene();
	if (!MovieScene)
	{
		return;
	}

	InSequence->Modify();
	MovieScene->Modify();
}

void GetSequenceSubSections(UMovieSceneSequence* const InSequence, TArray<UMovieSceneSubSection*>& OutSubSections)
{
	if (!InSequence)
	{
		return;
	}

	const UMovieScene* const MovieScene = InSequence->GetMovieScene();
	if (!MovieScene)
	{
		return;
	}

	for (UMovieSceneSection* const Section : MovieScene->GetAllSections())
	{
		if (UMovieSceneSubSection* const SubSection = Cast<UMovieSceneSubSection>(Section))
		{
			OutSubSections.Add(SubSection);
		}
	}
}

} // namespace UE::SequenceNavigator
