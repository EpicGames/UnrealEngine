// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"

class ISequencer;
class UMovieScene;
class UMovieSceneSequence;
class UMovieSceneSubSection;
struct FFrameTime;

namespace UE::SequenceNavigator
{

FFrameTime ConvertToDisplayRateTime(const UMovieSceneSequence& InSequence, const FFrameTime& InTime);
FFrameTime ConvertToTickResolutionTime(const UMovieSceneSequence& InSequence, const FFrameTime& InTime);

/**
 * Finds the subsection corresponding to a given sequence within the specified sequencer.
 * @param InSequencer The sequencer to search within.
 * @param InSequence The movie scene sequence for which the subsection is to be found.
 * @return The found movie scene subsection, or nullptr if no matching subsection is found.
 */
UMovieSceneSubSection* FindSequenceSubSection(ISequencer& InSequencer, UMovieSceneSequence* const InSequence);

/**
 * Checks if globally marked frames are enabled for the given movie scene sequence.
 * @param InSequence The movie scene sequence to check.
 * @return True if globally marked frames are enabled for the specified sequence, false otherwise.
 */
bool IsGloballyMarkedFramesForSequence(UMovieSceneSequence* const InSequence);

/**
 * Toggles the visibility of globally marked frames for the specified sequence within the provided sequencer.
 * @param InSequencer The sequencer controlling the movie scene.
 * @param InSequence The movie scene sequence for which the globally marked frame visibility is to be changed.
 * @param bInVisible Indicates whether the globally marked frames should be visible (true) or hidden (false).
 */
void ShowGloballyMarkedFramesForSequence(ISequencer& InSequencer
	, UMovieSceneSequence* const InSequence, const bool bInVisible);

/**
 * Modifies the given sequence and its associated movie scene to ensure they are marked as modified.
 * @param InSequence The movie scene sequence to modify, including its associated movie scene.
 */
void ModifySequenceAndMovieScene(UMovieSceneSequence* const InSequence);

/**
 * Retrieves all sections as subsections from the given movie scene sequence.
 * @param InSequence The movie scene sequence to retrieve subsections from.
 * @param OutSubSections The array to be populated with found movie scene subsections.
 */
void GetSequenceSubSections(UMovieSceneSequence* const InSequence, TArray<UMovieSceneSubSection*>& OutSubSections);

} // namespace UE::SequenceNavigator
