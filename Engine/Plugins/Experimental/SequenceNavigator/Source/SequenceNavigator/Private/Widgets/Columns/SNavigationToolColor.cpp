// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNavigationToolColor.h"
#include "Engine/Engine.h"
#include "Extensions/IColorExtension.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "NavigationTool.h"
#include "NavigationToolView.h"
#include "Styling/StyleColors.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SNavigationToolTreeRow.h"

#define LOCTEXT_NAMESPACE "SNavigationToolColor"

namespace UE::SequenceNavigator
{

using namespace Sequencer;

DECLARE_DELEGATE_RetVal_OneParam(FReply, FOnColorEntrySelected, FName EntryName);

class SNavigationToolColorEntry : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNavigationToolColorEntry) {}
		SLATE_EVENT(FOnColorEntrySelected, OnColorEntrySelected)
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, FName InEntryName, const FLinearColor& InEntryColor)
	{
		ColorEntryName = InEntryName;
		MenuButtonStyle = &FAppStyle::Get().GetWidgetStyle<FButtonStyle>(TEXT("Menu.Button"));

		OnColorEntrySelected = InArgs._OnColorEntrySelected;

		ChildSlot
		[
			SNew(SBox)
			.WidthOverride(120.f)
			[
				SNew(SBorder)
				.BorderImage(this, &SNavigationToolColorEntry::GetBorderImage)
				.Padding(FMargin(12.f, 1.f, 12.f, 1.f))
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.FillWidth(0.8f)
					.Padding(FMargin(0.f, 2.f, 0.f, 0.f))
					[
						SNew(STextBlock)
						.Text(FText::FromName(InEntryName))
						.ColorAndOpacity(FLinearColor::White)
					]
					+ SHorizontalBox::Slot()
					.FillWidth(0.2f)
					[
						SNew(SBorder)
						[
							SNew(SColorBlock)
							.Color(InEntryColor)
						]
					]
				]
			]
		];
	}

	const FSlateBrush* GetBorderImage() const
	{
		if (IsHovered())
		{
			return &MenuButtonStyle->Hovered;
		}
		return &MenuButtonStyle->Normal;
	}

	virtual FReply OnMouseButtonUp(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) override
	{
		if (OnColorEntrySelected.IsBound())
		{
			return OnColorEntrySelected.Execute(ColorEntryName);
		}
		return FReply::Handled();
	}

protected:
	FName ColorEntryName;
	FOnColorEntrySelected OnColorEntrySelected;
	const FButtonStyle* MenuButtonStyle = nullptr;
};

class FColorDragDropOp : public FDragDropOperation
{
public:
	DRAG_DROP_OPERATOR_TYPE(FColorDragDropOp, FDragDropOperation)

	FColor Color;

	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override
	{
		return SNullWidget::NullWidget;
	}

	static TSharedRef<FColorDragDropOp> New(const FColor& Color)
	{
		const TSharedRef<FColorDragDropOp> Operation = MakeShared<FColorDragDropOp>();
		Operation->Color = Color;
		Operation->Construct();
		return Operation;
	}
};

void SNavigationToolColor::Construct(const FArguments& InArgs
	, const FNavigationToolViewModelPtr& InItem
	, const TSharedRef<INavigationToolView>& InView
	, const TSharedRef<SNavigationToolTreeRow>& InRowWidget)
{
	WeakItem = InItem;
	WeakView = InView;
	WeakRowWidget = InRowWidget;

	WeakTool = InView->GetOwnerTool();

	SetColorAndOpacity(TAttribute<FLinearColor>(this, &SNavigationToolColor::GetStateColorAndOpacity));

	ChildSlot
	[
		SNew(SBox)
		.Padding(FMargin(6.f, 0.f, 0.f, 0.f))
		[
			SNew(SBorder)
			.Padding(0.f, 1.f)
			.BorderBackgroundColor(this, &SNavigationToolColor::GetBorderColor)
			[
				SNew(SColorBlock)
				.Color(this, &SNavigationToolColor::GetColorBlockColor)
				.Size(FVector2D(14.f, 22.f))
				.OnMouseButtonDown(this, &SNavigationToolColor::OnColorMouseButtonDown)
			]
		]
	];
}

FLinearColor SNavigationToolColor::GetColorBlockColor() const
{
	ItemColor = FColor();

	if (const FNavigationToolViewModelPtr Item = WeakItem.Pin())
	{
		if (const TViewModelPtr<IColorExtension> ColorExtension = Item.ImplicitCast())
		{
			ItemColor = ColorExtension->GetColor().Get(FColor());
		}
	}

	return ItemColor;
}

void SNavigationToolColor::RemoveItemColor() const
{
	const TSharedPtr<INavigationTool> Tool = WeakTool.Pin();
	check(Tool.IsValid());
	Tool->RemoveItemColor(WeakItem.Pin());
}

FSlateColor SNavigationToolColor::GetBorderColor() const
{
	if (IsHovered())
	{
		return FSlateColor::UseForeground();
	}
	return FSlateColor::UseSubduedForeground();
}

FLinearColor SNavigationToolColor::GetStateColorAndOpacity() const
{
	const FNavigationToolViewModelPtr Item = WeakItem.Pin();
	if (!Item.IsValid())
	{
		return FLinearColor::Transparent;
	}

	const TSharedPtr<INavigationToolView> ToolView = WeakView.Pin();
	if (!ToolView.IsValid())
	{
		return FLinearColor::Transparent;
	}

	const bool bIsSelected = ToolView->IsItemSelected(Item);

	const TSharedPtr<SNavigationToolTreeRow> Row = WeakRowWidget.Pin();

	const bool IsRowNotHovered = Row.IsValid() && !Row->IsHovered();
	const bool IsColorOptionsClosed = ColorOptions.IsValid() && !ColorOptions->IsOpen();

	// make the foreground brush transparent if it is not selected and it doesn't have an Item Color
	if (ItemColor == FColor() && IsRowNotHovered && !bIsSelected && IsColorOptionsClosed)
	{
		return FLinearColor::Transparent;
	}

	return FLinearColor::White;
}

FReply SNavigationToolColor::OnColorEntrySelected(const FColor& InColor) const
{
	if (ColorOptions.IsValid())
	{
		ColorOptions->SetIsOpen(false);
	}

	if (const TSharedPtr<INavigationTool> Tool = WeakTool.Pin())
	{
		Tool->SetItemColor(WeakItem.Pin(), InColor);
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply SNavigationToolColor::OnDragDetected(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	if (InPointerEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		return FReply::Handled().BeginDragDrop(FColorDragDropOp::New(ItemColor));
	}

	return SCompoundWidget::OnDragDetected(InGeometry, InPointerEvent);
}

void SNavigationToolColor::OnDragEnter(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent)
{
	if (const TSharedPtr<FColorDragDropOp> ColorOp = InDragDropEvent.GetOperationAs<FColorDragDropOp>())
	{
		OnColorEntrySelected(ColorOp->Color);
	}

	SCompoundWidget::OnDragEnter(InGeometry, InDragDropEvent);
}

FReply SNavigationToolColor::OnColorMouseButtonDown(const FGeometry& InGeometry, const FPointerEvent& InEvent)
{
	OpenColorPickerDialog();

	return FReply::Handled();
}

void SNavigationToolColor::OpenColorPickerDialog()
{
	const FLinearColor PreviousColor = ItemColor;

	FColorPickerArgs PickerArgs;
	PickerArgs.bUseAlpha = false;
	PickerArgs.DisplayGamma = TAttribute<float>::Create(TAttribute<float>::FGetter::CreateUObject(GEngine, &UEngine::GetDisplayGamma));
	PickerArgs.InitialColor = ItemColor;
	PickerArgs.ParentWidget = GetParentWidget();
	PickerArgs.OnColorCommitted = FOnLinearColorValueChanged::CreateLambda([this](const FLinearColor InNewColor)
		{
			if (const TViewModelPtr<IColorExtension> ColorExtension = WeakItem.ImplicitPin())
			{
				ColorExtension->SetColor(InNewColor.ToFColor(true));
			}
		});
	PickerArgs.OnColorPickerCancelled = FOnColorPickerCancelled::CreateLambda([this, PreviousColor](const FLinearColor& InNewColor)
		{
			if (const TViewModelPtr<IColorExtension> ColorExtension = WeakItem.ImplicitPin())
			{
				ColorExtension->SetColor(PreviousColor.ToFColor(true));
			}
		});

	OpenColorPicker(PickerArgs);
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
