// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NavigationToolDefines.h"
#include "Widgets/SCompoundWidget.h"

class SMenuAnchor;

namespace UE::SequenceNavigator
{

class IColorExtension;
class INavigationTool;
class INavigationToolView;
class SNavigationToolTreeRow;

class SNavigationToolColor : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNavigationToolColor) {}
	SLATE_END_ARGS()

	virtual ~SNavigationToolColor() override = default;

	void Construct(const FArguments& InArgs
		, const FNavigationToolViewModelPtr& InItem
		, const TSharedRef<INavigationToolView>& InView
		, const TSharedRef<SNavigationToolTreeRow>& InRowWidget);

	FLinearColor GetColorBlockColor() const;
	void RemoveItemColor() const;

	FSlateColor GetBorderColor() const;
	FLinearColor GetStateColorAndOpacity() const;

	FReply OnColorEntrySelected(const FColor& InColor) const;

	virtual FReply OnDragDetected(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) override;
	virtual void OnDragEnter(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent) override;

protected:
	FReply OnColorMouseButtonDown(const FGeometry& InGeometry, const FPointerEvent& InEvent);

	void OpenColorPickerDialog();

	FNavigationToolViewModelWeakPtr WeakItem;

	TWeakPtr<INavigationToolView> WeakView;

	TWeakPtr<SNavigationToolTreeRow> WeakRowWidget;

	TWeakPtr<INavigationTool> WeakTool;

	TSharedPtr<SMenuAnchor> ColorOptions;

	mutable FColor ItemColor = FColor();

	bool bIsPressed = false;
};

} // namespace UE::SequenceNavigator
