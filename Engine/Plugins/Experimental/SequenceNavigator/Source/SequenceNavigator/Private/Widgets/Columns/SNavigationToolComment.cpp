// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNavigationToolComment.h"
#include "Items/INavigationToolItem.h"
#include "Items/NavigationToolItemUtils.h"
#include "Items/NavigationToolSequence.h"
#include "NavigationToolView.h"
#include "TrackEditors/SubTrackEditorBase.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/SNavigationToolTreeRow.h"

#define LOCTEXT_NAMESPACE "SNavigationToolComment"

namespace UE::SequenceNavigator
{

void SNavigationToolComment::Construct(const FArguments& InArgs
	, const FNavigationToolViewModelPtr& InItem
	, const TSharedRef<INavigationToolView>& InView
	, const TSharedRef<SNavigationToolTreeRow>& InRowWidget)
{
	WeakItem = InItem;
	WeakView = InView;
	WeakRowWidget = InRowWidget;

	ChildSlot
		[
			SNew(SEditableTextBox)
			.Text(this, &SNavigationToolComment::GetCommentText)
			.OnTextChanged(this, &SNavigationToolComment::OnCommentTextChanged)
			.OnTextCommitted(this, &SNavigationToolComment::OnCommentTextCommitted)
		];
}

SNavigationToolComment::~SNavigationToolComment()
{
}

FString SNavigationToolComment::GetMetaDataComment() const
{
	if (const UMovieSceneMetaData* const SequenceMetaData = ItemUtils::GetSequenceItemMetaData(WeakItem.Pin()))
	{
		return SequenceMetaData->GetNotes();
	}
	return FString();
}

FText SNavigationToolComment::GetCommentText() const
{
	return FText::FromString(GetMetaDataComment());
}

void SNavigationToolComment::OnCommentTextChanged(const FText& InNewText)
{
	OnCommentTextCommitted(InNewText, ETextCommit::Default);
}

void SNavigationToolComment::OnCommentTextCommitted(const FText& InNewText, const ETextCommit::Type InCommitType)
{
	using namespace Sequencer;

	const FNavigationToolViewModelPtr Item = WeakItem.Pin();
	if (!Item.IsValid())
	{
		return;
	}

	const TViewModelPtr<FNavigationToolSequence> SequenceItem = Item.ImplicitCast();
	if (!SequenceItem.IsValid())
	{
		return;
	}

	UMovieSceneSequence* const Sequence = SequenceItem->GetSequence();
	if (!Sequence)
	{
		return;
	}

	UMovieSceneMetaData* const SequenceMetaData = FSubTrackEditorUtil::FindOrAddMetaData(Sequence);
	if (!SequenceMetaData)
	{
		return;
	}

	if (InNewText.ToString().Equals(SequenceMetaData->GetNotes(), ESearchCase::CaseSensitive))
	{
		return;
	}

	const bool bShouldTransact = (InCommitType != ETextCommit::Default);
	const FScopedTransaction Transaction(GetTransactionText(), bShouldTransact);

	Sequence->Modify();
	SequenceMetaData->SetNotes(InNewText.ToString());
}

FText SNavigationToolComment::GetTransactionText() const
{
	return LOCTEXT("SetSequenceCommentTransaction", "Set Sequence Comment");
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
