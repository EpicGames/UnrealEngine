// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/ViewModelPtr.h"
//#include "NavigationToolDefines.h"
#include "Items/NavigationToolSequence.h"
#include "Templates/UniquePtr.h"
#include "Widgets/SCompoundWidget.h"

namespace UE::SequenceNavigator
{

//class FNavigationToolSequence;
class INavigationToolView;
class SNavigationToolTreeRow;

class SNavigationToolComment : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNavigationToolComment) {}
	SLATE_END_ARGS()

	virtual ~SNavigationToolComment() override;

	void Construct(const FArguments& InArgs
		, const FNavigationToolViewModelPtr& InItem
		, const TSharedRef<INavigationToolView>& InView
		, const TSharedRef<SNavigationToolTreeRow>& InRowWidget);

protected:
	FString GetMetaDataComment() const;

	FText GetCommentText() const;
	void OnCommentTextChanged(const FText& InNewText);
	void OnCommentTextCommitted(const FText& InNewText, const ETextCommit::Type InCommitType);

	FText GetTransactionText() const;

	FNavigationToolViewModelWeakPtr WeakItem;

	TWeakPtr<INavigationToolView> WeakView;

	TWeakPtr<SNavigationToolTreeRow> WeakRowWidget;
};

} // namespace UE::SequenceNavigator
