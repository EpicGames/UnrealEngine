// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNavigationToolDeactiveState.h"
#include "Columns/NavigationToolDeactiveStateColumn.h"
#include "Editor.h"
#include "Items/INavigationToolItem.h"
#include "Items/NavigationToolItemUtils.h"
#include "MVVM/Extensions/IDeactivatableExtension.h"
#include "NavigationToolView.h"
#include "Widgets/SNavigationToolTreeRow.h"

#define LOCTEXT_NAMESPACE "SNavigationToolDeactiveState"

namespace UE::SequenceNavigator
{

using namespace Sequencer;

class FEvaluationStateDragDropOp : public FDragDropOperation
{
public:
	DRAG_DROP_OPERATOR_TYPE(FVisibilityDragDropOp, FDragDropOperation)

	/** Flag which defines whether to hide destination items or not */
	EDeactivatableState InactiveState;

	/** Undo transaction stolen from the gutter which is kept alive for the duration of the drag */
	TUniquePtr<FScopedTransaction> UndoTransaction;

	/** The widget decorator to use */
	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override
	{
		return SNullWidget::NullWidget;
	}

	/** Create a new drag and drop operation out of the specified flag */
	static TSharedRef<FEvaluationStateDragDropOp> New(const EDeactivatableState InEvaluationState, TUniquePtr<FScopedTransaction>& ScopedTransaction)
	{
		TSharedRef<FEvaluationStateDragDropOp> Operation = MakeShared<FEvaluationStateDragDropOp>();
		Operation->InactiveState = InEvaluationState;
		Operation->UndoTransaction = MoveTemp(ScopedTransaction);
		Operation->Construct();
		return Operation;
	}
};

EDeactivatableState GetItemDeactivatedState(const FNavigationToolViewModelPtr& InItem)
{
	using namespace Sequencer;
	using namespace ItemUtils;

	if (!InItem.IsValid())
	{
		return EDeactivatableState::None;
	}

	const ENavigationToolCompareState State = CompareChildrenItemState<IDeactivatableExtension>(InItem,
		[](const TViewModelPtr<IDeactivatableExtension>& InItem)
			{
				return InItem->IsDeactivated();
			},
		[](const TViewModelPtr<IDeactivatableExtension>& InItem)
			{
				return !InItem->IsDeactivated();
			});

	return static_cast<EDeactivatableState>(State);
}

void SetItemDeactivated(const FNavigationToolViewModelPtr& InItem, const bool bInInactive)
{
	if (const TViewModelPtr<IDeactivatableExtension> DeactivatableItem = InItem.ImplicitCast())
	{
		DeactivatableItem->SetIsDeactivated(bInInactive);
	}
}

void SNavigationToolDeactiveState::Construct(const FArguments& InArgs
	, const TSharedRef<FNavigationToolDeactiveStateColumn>& InColumn
	, const FNavigationToolViewModelPtr& InItem
	, const TSharedRef<INavigationToolView>& InView
	, const TSharedRef<SNavigationToolTreeRow>& InRowWidget)
{
	WeakColumn = InColumn;
	WeakItem = InItem;
	WeakView = InView;
	WeakRowWidget = InRowWidget;

	SImage::Construct(SImage::FArguments()
		.IsEnabled(this, &SNavigationToolDeactiveState::IsVisibilityWidgetEnabled)
		.ColorAndOpacity(this, &SNavigationToolDeactiveState::GetForegroundColor)
		.Image(this, &SNavigationToolDeactiveState::GetBrush));
}

FReply SNavigationToolDeactiveState::OnDragDetected(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	using namespace Sequencer;

	if (InPointerEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		const EDeactivatableState NewState = (GetInactiveState() != EDeactivatableState::None)
			? EDeactivatableState::Deactivated : EDeactivatableState::None;
		return FReply::Handled().BeginDragDrop(FEvaluationStateDragDropOp::New(NewState, UndoTransaction));
	}

	return FReply::Unhandled();
}

void SNavigationToolDeactiveState::OnDragEnter(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent)
{
	const TSharedPtr<FEvaluationStateDragDropOp> DragDropOp = InDragDropEvent.GetOperationAs<FEvaluationStateDragDropOp>();
	if (DragDropOp.IsValid())
	{
		const bool bItemInactive = (DragDropOp->InactiveState != EDeactivatableState::None);
		SetIsDeactivated(bItemInactive);
	}
}

FReply SNavigationToolDeactiveState::HandleClick()
{
	using namespace Sequencer;

	if (!IsVisibilityWidgetEnabled())
	{
		return FReply::Unhandled();
	}

	const TSharedPtr<INavigationToolView> ToolView = WeakView.Pin();
	const FNavigationToolViewModelPtr Item = WeakItem.Pin();
	const TSharedPtr<FNavigationToolDeactiveStateColumn> Column = WeakColumn.Pin();

	if (!ToolView.IsValid() || !Item.IsValid() || !Column.IsValid())
	{
		return FReply::Unhandled();
	}

	// Open an undo transaction
	UndoTransaction = MakeUnique<FScopedTransaction>(LOCTEXT("SetNavigationToolItemInactiveState", "Set Item Inactive State"));

	const bool bNewIsInactive = (GetInactiveState() != EDeactivatableState::Deactivated);

	// We operate on all the selected items if the specified item is selected
	if (ToolView->IsItemSelected(Item))
	{
		for (FNavigationToolViewModelWeakPtr& WeakSelectedItem : ToolView->GetSelectedItems())
		{
			SetItemDeactivated(WeakSelectedItem.Pin(), bNewIsInactive);
		}
	}
	else
	{
		SetIsDeactivated(bNewIsInactive);
	}

	return FReply::Handled().DetectDrag(SharedThis(this), EKeys::LeftMouseButton);
}

FReply SNavigationToolDeactiveState::OnMouseButtonDoubleClick(const FGeometry& InInGeometry, const FPointerEvent& InPointerEvent)
{
	return HandleClick();
}

FReply SNavigationToolDeactiveState::OnMouseButtonDown(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	if (InPointerEvent.GetEffectingButton() != EKeys::LeftMouseButton)
	{
		return FReply::Unhandled();
	}

	return HandleClick();
}

FReply SNavigationToolDeactiveState::OnMouseButtonUp(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	if (InPointerEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		UndoTransaction.Reset();
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SNavigationToolDeactiveState::OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent)
{
	UndoTransaction.Reset();
}

const FSlateBrush* SNavigationToolDeactiveState::GetBrush() const
{
	return FAppStyle::GetBrush(TEXT("Sequencer.Column.Mute"));
}

FSlateColor SNavigationToolDeactiveState::GetForegroundColor() const
{
	const FNavigationToolViewModelPtr Item = WeakItem.Pin();
	if (!Item.IsValid())
	{
		return FStyleColors::Transparent;
	}

	const bool bIsItemHovered = WeakRowWidget.IsValid() && WeakRowWidget.Pin()->IsHovered();
	if (bIsItemHovered || IsHovered())
	{
		switch (GetInactiveState())
		{
		case EDeactivatableState::None:
			return FStyleColors::White25;

		case EDeactivatableState::PartiallyDeactivated:
			return FStyleColors::ForegroundHover;

		case EDeactivatableState::Deactivated:
			return FStyleColors::ForegroundHover;
		}
	}
	else
	{
		switch (GetInactiveState())
		{
		case EDeactivatableState::None:
			return FStyleColors::Transparent;

		case EDeactivatableState::PartiallyDeactivated:
			return FStyleColors::White25;

		case EDeactivatableState::Deactivated:
			return FStyleColors::Foreground;
		}
	}

	return FStyleColors::Transparent;
}

EDeactivatableState SNavigationToolDeactiveState::GetInactiveState() const
{
	return GetItemDeactivatedState(WeakItem.Pin());
}

void SNavigationToolDeactiveState::SetIsDeactivated(const bool bInIsDeactivated)
{
	SetItemDeactivated(WeakItem.Pin(), bInIsDeactivated);
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
