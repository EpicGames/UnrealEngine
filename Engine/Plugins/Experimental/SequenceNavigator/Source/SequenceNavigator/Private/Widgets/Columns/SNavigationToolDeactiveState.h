// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NavigationToolDefines.h"
#include "ScopedTransaction.h"
#include "Templates/UniquePtr.h"
#include "Widgets/Images/SImage.h"

namespace UE::Sequencer
{
	class IDeactivatableExtension;
	enum class EDeactivatableState;
}

namespace UE::SequenceNavigator
{

class FNavigationToolItem;
class FNavigationToolDeactiveStateColumn;
class INavigationToolView;
class SNavigationToolTreeRow;

/** Widget responsible for managing the visibility for a single item */
class SNavigationToolDeactiveState : public SImage
{
public:
	SLATE_BEGIN_ARGS(SNavigationToolDeactiveState) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs
		, const TSharedRef<FNavigationToolDeactiveStateColumn>& InColumn
		, const FNavigationToolViewModelPtr& InItem
		, const TSharedRef<INavigationToolView>& InView
		, const TSharedRef<SNavigationToolTreeRow>& InRowWidget);

protected:
	/** Returns whether the widget is enabled or not */
	virtual bool IsVisibilityWidgetEnabled() const { return true; }

	virtual const FSlateBrush* GetBrush() const;

	//~ Begin SWidget
	virtual FReply OnDragDetected(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) override;
	virtual void OnDragEnter(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent) override;
	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InInGeometry, const FPointerEvent& InPointerEvent) override;
	virtual FReply OnMouseButtonDown(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) override;
	virtual void OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent) override;
	//~ End SWidget

	FReply HandleClick();

	virtual FSlateColor GetForegroundColor() const override;

	Sequencer::EDeactivatableState GetInactiveState() const;
	void SetIsDeactivated(const bool bInIsDeactivated);

	TWeakPtr<FNavigationToolDeactiveStateColumn> WeakColumn;

	FNavigationToolViewModelWeakPtr WeakItem;

	TWeakPtr<INavigationToolView> WeakView;

	TWeakPtr<SNavigationToolTreeRow> WeakRowWidget;

	TUniquePtr<FScopedTransaction> UndoTransaction;
};

} // namespace UE::SequenceNavigator
