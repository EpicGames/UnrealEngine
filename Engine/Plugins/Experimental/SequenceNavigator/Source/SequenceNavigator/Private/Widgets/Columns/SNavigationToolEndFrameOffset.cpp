// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNavigationToolEndFrameOffset.h"
#include "Items/NavigationToolItemUtils.h"
#include "Items/NavigationToolSequence.h"
#include "NavigationToolView.h"
#include "Sections/MovieSceneSubSection.h"
#include "Widgets/SNavigationToolTreeRow.h"

#define LOCTEXT_NAMESPACE "SNavigationToolEndFrameOffset"

namespace UE::SequenceNavigator
{

using namespace Sequencer;

void SNavigationToolEndFrameOffset::Construct(const FArguments& InArgs
	, const FNavigationToolViewModelPtr& InItem
	, const TSharedRef<INavigationToolView>& InView
	, const TSharedRef<SNavigationToolTreeRow>& InRowWidget)
{
	WeakItem = InItem;
	WeakView = InView;
	WeakRowWidget = InRowWidget;

	SNavigationToolTime::Construct(SNavigationToolTime::FArguments(), InItem, InView, InRowWidget);
}

double SNavigationToolEndFrameOffset::GetFrameTimeValue() const
{
	const TViewModelPtr<FNavigationToolSequence> SequenceItem = WeakItem.ImplicitPin();
	if (!SequenceItem.IsValid())
	{
		return 0.;
	}

	UMovieSceneSubSection* const SubSection = SequenceItem->GetSubSection();
	if (!SubSection)
	{
		return 0.;
	}

	return SubSection->Parameters.EndFrameOffset.Value;
}

void SNavigationToolEndFrameOffset::OnFrameTimeValueCommitted(const double InNewValue, const ETextCommit::Type InCommitType)
{
	using namespace ItemUtils;

	UMovieSceneSubSection* const SubSection = GetSequenceItemSubSection(WeakItem.Pin());
	if (!SubSection)
	{
		return;
	}

	if (InNewValue == SubSection->Parameters.EndFrameOffset.Value)
	{
		return;
	}

	const bool bShouldTransact = !UndoTransaction.IsValid() && (InCommitType == ETextCommit::OnEnter);
	const FScopedTransaction Transaction(GetTransactionText(), bShouldTransact);

	SubSection->Modify();
	SubSection->Parameters.EndFrameOffset = static_cast<int32>(InNewValue);
}

FText SNavigationToolEndFrameOffset::GetTransactionText() const
{
	return LOCTEXT("SetEndFrameOffsetTransaction", "Set End Frame Offset");
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
