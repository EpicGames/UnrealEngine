// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NavigationToolDefines.h"
#include "SNavigationToolTime.h"

namespace UE::SequenceNavigator
{

class INavigationToolView;
class SNavigationToolTreeRow;

class SNavigationToolEndFrameOffset : public SNavigationToolTime
{
public:
	SLATE_BEGIN_ARGS(SNavigationToolEndFrameOffset) {}
	SLATE_END_ARGS()

	virtual ~SNavigationToolEndFrameOffset() override = default;

	void Construct(const FArguments& InArgs
		, const FNavigationToolViewModelPtr& InItem
		, const TSharedRef<INavigationToolView>& InView
		, const TSharedRef<SNavigationToolTreeRow>& InRowWidget);

protected:
	//~ Begin SNavigationToolTime
	virtual double GetFrameTimeValue() const override;
	virtual void OnFrameTimeValueCommitted(const double InNewValue, const ETextCommit::Type InCommitType) override;
	virtual FText GetTransactionText() const override;
	//~ End SNavigationToolTime
};

} // namespace UE::SequenceNavigator
