// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNavigationToolHBias.h"
#include "Items/NavigationToolItemUtils.h"
#include "Items/NavigationToolSequence.h"
#include "NavigationToolStyle.h"
#include "NavigationToolView.h"
#include "TrackEditors/SubTrackEditorBase.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/SNavigationToolTreeRow.h"

#define LOCTEXT_NAMESPACE "SNavigationToolHBias"

namespace UE::SequenceNavigator
{

using namespace Sequencer;

void SNavigationToolHBias::Construct(const FArguments& InArgs
	, const FNavigationToolViewModelPtr& InItem
	, const TSharedRef<INavigationToolView>& InView
	, const TSharedRef<SNavigationToolTreeRow>& InRowWidget)
{
	WeakItem = InItem;
	WeakView = InView;
	WeakRowWidget = InRowWidget;

	ChildSlot
		[
			SNew(SSpinBox<int32>)
			.Justification(ETextJustify::Center)
			.Style(&FNavigationToolStyle::Get(), TEXT("SpinBox"))
			.Value(this, &SNavigationToolHBias::GetValue)
			.OnValueChanged(this, &SNavigationToolHBias::OnValueChanged)
			.OnValueCommitted(this, &SNavigationToolHBias::OnValueCommitted)
			.OnBeginSliderMovement(this, &SNavigationToolHBias::OnBeginSliderMovement)
			.OnEndSliderMovement(this, &SNavigationToolHBias::OnEndSliderMovement)
		];
}

int32 SNavigationToolHBias::GetValue() const
{
	if (const UMovieSceneSubSection* const SubSection = ItemUtils::GetSequenceItemSubSection(WeakItem.Pin()))
	{
		return SubSection->Parameters.HierarchicalBias;
	}
	return 0;
}

void SNavigationToolHBias::OnValueChanged(const int32 InNewValue)
{
	OnValueCommitted(InNewValue, ETextCommit::Default);
}

void SNavigationToolHBias::OnValueCommitted(const int32 InNewValue, const ETextCommit::Type InCommitType)
{
	using namespace ItemUtils;

	UMovieSceneSubSection* const SubSection = GetSequenceItemSubSection(WeakItem.Pin());
	if (!SubSection)
	{
		return;
	}

	if (InNewValue == SubSection->Parameters.HierarchicalBias)
	{
		return;
	}

	const bool bShouldTransact = !UndoTransaction.IsValid() && (InCommitType == ETextCommit::OnEnter);
	const FScopedTransaction Transaction(GetTransactionText(), bShouldTransact);

	SubSection->Modify();
	SubSection->Parameters.HierarchicalBias = InNewValue;
}

void SNavigationToolHBias::OnBeginSliderMovement()
{
	if (!UndoTransaction.IsValid())
	{
		UndoTransaction = MakeUnique<FScopedTransaction>(GetTransactionText());
	}
}

void SNavigationToolHBias::OnEndSliderMovement(const int32 InNewValue)
{
	UndoTransaction.Reset();
}

FText SNavigationToolHBias::GetTransactionText() const
{
	return LOCTEXT("SetSequenceHBiasTransaction", "Set Sequence HBias");
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
