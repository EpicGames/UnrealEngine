// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NavigationToolDefines.h"
#include "ScopedTransaction.h"
#include "Templates/UniquePtr.h"
#include "Widgets/SCompoundWidget.h"

namespace UE::SequenceNavigator
{

class FNavigationToolSequence;
class INavigationToolView;
class SNavigationToolTreeRow;

class SNavigationToolHBias : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNavigationToolHBias) {}
	SLATE_END_ARGS()

	virtual ~SNavigationToolHBias() override = default;

	void Construct(const FArguments& InArgs
		, const FNavigationToolViewModelPtr& InItem
		, const TSharedRef<INavigationToolView>& InView
		, const TSharedRef<SNavigationToolTreeRow>& InRowWidget);

protected:
	int32 GetValue() const;
	void OnValueChanged(const int32 InNewValue);
	void OnValueCommitted(const int32 InNewValue, const ETextCommit::Type InCommitType);
	void OnBeginSliderMovement();
	void OnEndSliderMovement(const int32 InNewValue);

	FText GetTransactionText() const;

	FNavigationToolViewModelWeakPtr WeakItem;

	TWeakPtr<INavigationToolView> WeakView;

	TWeakPtr<SNavigationToolTreeRow> WeakRowWidget;

	TUniquePtr<FScopedTransaction> UndoTransaction;
};

} // namespace UE::SequenceNavigator
