// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNavigationToolId.h"
#include "Extensions/IIdExtension.h"
#include "Items/NavigationToolItemUtils.h"
#include "NavigationToolView.h"
#include "Widgets/Input/SEditableText.h"
#include "Widgets/SNavigationToolTreeRow.h"

#define LOCTEXT_NAMESPACE "SNavigationToolId"

namespace UE::SequenceNavigator
{

using namespace Sequencer;

void SNavigationToolId::Construct(const FArguments& InArgs
	, const FNavigationToolViewModelPtr& InItem
	, const TSharedRef<INavigationToolView>& InView
	, const TSharedRef<SNavigationToolTreeRow>& InRowWidget)
{
	check(InItem.IsValid());
	WeakItem = InItem;

	ChildSlot
	[
		SNew(SEditableText)
			.IsReadOnly(true)
			.Text(this, &SNavigationToolId::GetItemText)
	];
}

FText SNavigationToolId::GetItemText() const
{
	if (const FNavigationToolViewModelPtr Item = WeakItem.Pin())
	{
		if (const TViewModelPtr<IIdExtension> IdExtension = Item.ImplicitCast())
		{
			return IdExtension->GetId();
		}
	}
	return FText();
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
