// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NavigationToolDefines.h"
#include "Widgets/SCompoundWidget.h"

namespace UE::SequenceNavigator
{

class INavigationToolView;
class SNavigationToolTreeRow;

class SNavigationToolId : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNavigationToolId) {}
	SLATE_END_ARGS()

	virtual ~SNavigationToolId() override = default;

	void Construct(const FArguments& InArgs
		, const FNavigationToolViewModelPtr& InItem
		, const TSharedRef<INavigationToolView>& InView
		, const TSharedRef<SNavigationToolTreeRow>& InRowWidget);

protected:

	FText GetItemText() const;

	FNavigationToolViewModelWeakPtr WeakItem;
};

} // namespace UE::SequenceNavigator
