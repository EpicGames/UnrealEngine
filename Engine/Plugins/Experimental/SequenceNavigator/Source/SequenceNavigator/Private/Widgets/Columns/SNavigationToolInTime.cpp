// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNavigationToolInTime.h"
#include "ScopedTransaction.h"
#include "Extensions/IInTimeExtension.h"
#include "Items/NavigationToolItemUtils.h"
#include "NavigationToolView.h"
#include "Widgets/SNavigationToolTreeRow.h"

#define LOCTEXT_NAMESPACE "SNavigationToolInTime"

namespace UE::SequenceNavigator
{

using namespace Sequencer;

void SNavigationToolInTime::Construct(const FArguments& InArgs
	, const FNavigationToolViewModelPtr& InItem
	, const TSharedRef<INavigationToolView>& InView
	, const TSharedRef<SNavigationToolTreeRow>& InRowWidget)
{
	WeakItem = InItem;
	WeakView = InView;
	WeakRowWidget = InRowWidget;

	SNavigationToolTime::Construct(SNavigationToolTime::FArguments(), InItem, InView, InRowWidget);
}

FName SNavigationToolInTime::GetStyleName() const
{
	return TEXT("SpinBox.InTime");
}

double SNavigationToolInTime::GetFrameTimeValue() const
{
	if (const TViewModelPtr<IInTimeExtension> Item = WeakItem.ImplicitPin())
	{
		return Item->GetInTime().Value;
	}
	return 0.;
}

void SNavigationToolInTime::OnFrameTimeValueCommitted(const double InNewValue, const ETextCommit::Type InCommitType)
{
	const TViewModelPtr<IInTimeExtension> Item = WeakItem.ImplicitPin();
	if (!Item.IsValid())
	{
		return;
	}

	const bool bShouldTransact = !UndoTransaction.IsValid() && (InCommitType == ETextCommit::OnEnter);
	const FScopedTransaction Transaction(GetTransactionText(), bShouldTransact);

	return Item->SetInTime(static_cast<int32>(InNewValue));
}

FText SNavigationToolInTime::GetTransactionText() const
{
	return LOCTEXT("SetInTimeTransaction", "Set In Time");
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
