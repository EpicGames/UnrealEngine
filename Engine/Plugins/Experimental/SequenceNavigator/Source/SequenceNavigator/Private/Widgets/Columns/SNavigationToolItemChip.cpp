// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNavigationToolItemChip.h"
#include "Brushes/SlateRoundedBoxBrush.h"
#include "Items/INavigationToolItem.h"
#include "NavigationToolView.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/Styling/NavigationToolStyleUtils.h"
#include "Widgets/Views/STableRow.h"

#define LOCTEXT_NAMESPACE "SNavigationToolItemChip"

namespace UE::SequenceNavigator
{

void SNavigationToolItemChip::Construct(const FArguments& InArgs
	, const FNavigationToolViewModelPtr& InItem
	, const TSharedPtr<INavigationToolView>& InView)
{
	check(InItem.IsValid());
	WeakItem = InItem;
	WeakView = InView;

	ChipStyle = InArgs._ChipStyle;
	OnItemChipClicked = InArgs._OnItemChipClicked;
	OnValidDragOver = InArgs._OnValidDragOver;

	constexpr float ChipSize = 14.f;
	const TSharedRef<INavigationToolItem> ItemRef = InItem.ToSharedRef();

	SBorder::Construct(SBorder::FArguments()
		.Padding(2.f, 1.f)
		.Visibility(EVisibility::Visible)
		[
			SNew(SBox)
			.WidthOverride(ChipSize)
			.HeightOverride(ChipSize)
			[
				SNew(SScaleBox)
				.Stretch(EStretch::ScaleToFit)
				[
					SNew(SImage)
					.Image(ItemRef, &INavigationToolItem::GetIconBrush)
					.ColorAndOpacity(this, &SNavigationToolItemChip::GetIconColor)
				]
			]
		]);

	SetToolTipText(TAttribute<FText>::CreateSP(this, &SNavigationToolItemChip::GetTooltipText));
	SetBorderImage(TAttribute<const FSlateBrush*>(this, &SNavigationToolItemChip::GetItemBackgroundBrush));
}

int32 SNavigationToolItemChip::OnPaint(const FPaintArgs& InArgs, const FGeometry& InAllottedGeometry, const FSlateRect& InCullingRect
	, FSlateWindowElementList& OutDrawElements, int32 LayerId
	, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	LayerId = SBorder::OnPaint(InArgs, InAllottedGeometry, InCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
	if (ItemDropZone.IsSet())
	{
		const FSlateBrush* const DropIndicatorBrush = GetDropIndicatorBrush(*ItemDropZone);

		// Reuse the drop indicator asset for horizontal, by rotating the drawn box 90 degrees.
		const FVector2f LocalSize(InAllottedGeometry.GetLocalSize());
		const FVector2f Pivot(LocalSize * 0.5f);
		const FVector2f RotatedLocalSize(LocalSize.Y, LocalSize.X);

		// Make the box centered to the allotted geometry, so that it can be rotated around the center.
		const FSlateLayoutTransform RotatedTransform(Pivot - RotatedLocalSize * 0.5f);

		FSlateDrawElement::MakeRotatedBox(OutDrawElements
			, LayerId++
			, InAllottedGeometry.ToPaintGeometry(RotatedLocalSize, RotatedTransform)
			, DropIndicatorBrush
			, ESlateDrawEffect::None
			, -UE_HALF_PI // 90 deg CCW
			, RotatedLocalSize * 0.5f // Relative center to the flipped
			, FSlateDrawElement::RelativeToElement
			, DropIndicatorBrush->GetTint(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint());
	}
	return LayerId;
}

FReply SNavigationToolItemChip::OnMouseButtonDown(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	if (InPointerEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		Press();

		const TSharedRef<SNavigationToolItemChip> This = SharedThis(this);

		return FReply::Handled()
			.DetectDrag(This, InPointerEvent.GetEffectingButton())
			.CaptureMouse(This)
			.SetUserFocus(This, EFocusCause::Mouse);
	}
	return FReply::Unhandled();
}

FReply SNavigationToolItemChip::OnMouseButtonUp(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	FReply Reply = FReply::Unhandled();

	if (IsPressed() && InPointerEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		Release();

		if (OnItemChipClicked.IsBound())
		{
			bool bEventOverButton = IsHovered();
			if (!bEventOverButton && InPointerEvent.IsTouchEvent())
			{
				bEventOverButton = InGeometry.IsUnderLocation(InPointerEvent.GetScreenSpacePosition());
			}

			if (bEventOverButton && HasMouseCapture())
			{
				Reply = OnItemChipClicked.Execute(WeakItem.Pin(), InPointerEvent);
			}
		}

		// If the user of the button didn't handle this click, then the button's default behavior handles it.
		if (!Reply.IsEventHandled())
		{
			Reply = FReply::Handled();
		}
	}

	// If the user hasn't requested a new mouse captor and the button still has mouse capture,
	// then the default behavior of the button is to release mouse capture.
	if (Reply.GetMouseCaptor().IsValid() == false && HasMouseCapture())
	{
		Reply.ReleaseMouseCapture();
	}

	return Reply;
}

void SNavigationToolItemChip::OnMouseCaptureLost(const FCaptureLostEvent& InCaptureLostEvent)
{
	Release();
}

/*FReply SNavigationToolItemChip::OnDragDetected(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	const FNavigationToolViewModelPtr Item = WeakItem.Pin();
	const TSharedPtr<INavigationToolView> ToolView = WeakView.Pin();

	if (!Item.IsValid() || !ToolView.IsValid())
	{
		ItemDropZone = TOptional<EItemDropZone>();
		return FReply::Unhandled();
	}

	return ToolView->OnDragDetected(InGeometry, InPointerEvent, Item);
}

FReply SNavigationToolItemChip::OnDragOver(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent)
{
	const FNavigationToolViewModelPtr Item = WeakItem.Pin();
	const TSharedPtr<INavigationToolView> ToolView = WeakView.Pin();

	if (!Item.IsValid() || !ToolView.IsValid())
	{
		ItemDropZone = TOptional<EItemDropZone>();
		return FReply::Unhandled();
	}

	const FVector2f LocalPointerPosition = InGeometry.AbsoluteToLocal(InDragDropEvent.GetScreenSpacePosition());

	const EItemDropZone ItemHoverZone = SNavigationToolItemChip::GetHoverZone(LocalPointerPosition
		, InGeometry.GetLocalSize());

	ItemDropZone = ToolView->OnCanDrop(InDragDropEvent, ItemHoverZone, Item);

	if (ItemDropZone.IsSet() && OnValidDragOver.IsBound())
	{
		OnValidDragOver.Execute(InGeometry, InDragDropEvent);
	}

	return FReply::Handled();
}

FReply SNavigationToolItemChip::OnDrop(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent)
{
	ItemDropZone = TOptional<EItemDropZone>();

	const FNavigationToolViewModelPtr Item = WeakItem.Pin();
	const TSharedPtr<INavigationToolView> ToolView = WeakView.Pin();

	if (!Item.IsValid() || !ToolView.IsValid())
	{
		return FReply::Unhandled();
	}

	const FVector2f LocalPointerPosition = InGeometry.AbsoluteToLocal(InDragDropEvent.GetScreenSpacePosition());
	const EItemDropZone DropZone = SNavigationToolItemChip::GetHoverZone(LocalPointerPosition, InGeometry.GetLocalSize());
	if (ToolView->OnCanDrop(InDragDropEvent, DropZone, Item).IsSet())
	{
		return ToolView->OnDrop(InDragDropEvent, DropZone, Item);
	}
	return FReply::Unhandled();
}

void SNavigationToolItemChip::OnDragLeave(const FDragDropEvent& InDragDropEvent)
{
	ItemDropZone = TOptional<EItemDropZone>();
}*/

bool SNavigationToolItemChip::IsSelected() const
{
	const TSharedPtr<INavigationToolView> View = WeakView.Pin();
	if (!View.IsValid())
	{
		return false;
	}

	const FNavigationToolViewModelPtr Item = WeakItem.Pin();
	if (!View.IsValid())
	{
		return false;
	}

	return View->IsItemSelected(Item);
}

const FSlateBrush* SNavigationToolItemChip::GetItemBackgroundBrush() const
{
	const EStyleType StyleType = IsHovered() ? EStyleType::Hovered : EStyleType::Normal;
	return &FStyleUtils::GetBrush(StyleType, IsSelected());
}

const FSlateBrush* SNavigationToolItemChip::GetDropIndicatorBrush(EItemDropZone InItemDropZone) const
{
	switch (InItemDropZone)
	{
	case EItemDropZone::AboveItem:
		return &ChipStyle->DropIndicator_Above;

	case EItemDropZone::OntoItem:
		return &ChipStyle->DropIndicator_Onto;

	case EItemDropZone::BelowItem:
		return &ChipStyle->DropIndicator_Below;

	default:
		return nullptr;
	};
}

void SNavigationToolItemChip::Press()
{
	bIsPressed = true;
}

void SNavigationToolItemChip::Release()
{
	bIsPressed = false;
}

EItemDropZone SNavigationToolItemChip::GetHoverZone(const FVector2f& InLocalPosition, const FVector2f& InLocalSize)
{
	// Clamping Values of the Edge Size so it's not too small nor too big
	// Referenced from the STableRow::ZoneFromPointerPosition
	constexpr float MinEdgeSize = 3.f;
	constexpr float MaxEdgeSize = 10.f;

	const float EdgeZoneSize = FMath::Clamp(InLocalSize.X * 0.25f, MinEdgeSize, MaxEdgeSize);

	if (InLocalPosition.X < EdgeZoneSize)
	{
		return EItemDropZone::AboveItem;
	}

	if (InLocalPosition.X > InLocalSize.X - EdgeZoneSize)
	{
		return EItemDropZone::BelowItem;
	}

	return EItemDropZone::OntoItem;
}

FSlateColor SNavigationToolItemChip::GetIconColor() const
{
	return IsHovered() ? FStyleColors::ForegroundHover : FStyleColors::Foreground;
}

FText SNavigationToolItemChip::GetTooltipText() const
{
	const FNavigationToolViewModelPtr Item = WeakItem.Pin();
	if (!Item.IsValid())
	{
		return FText::GetEmpty();
	}

	const FText ItemDisplayName = Item->GetDisplayName();

	return FText::Format(LOCTEXT("TooltipText", "{0}\n\n"
		"Click to select in Sequencer\n\n"
		"Alt + Click to select item in Sequence Navigator")
		, ItemDisplayName);
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
