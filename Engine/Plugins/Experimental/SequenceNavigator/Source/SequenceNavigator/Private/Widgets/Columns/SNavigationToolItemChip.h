// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/DelegateCombinations.h"
#include "Framework/SlateDelegates.h"
#include "Misc/Optional.h"
#include "NavigationToolDefines.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Templates/SharedPointer.h"
#include "Widgets/Layout/SBorder.h"

enum class EItemDropZone;
struct FTableRowStyle;

namespace UE::SequenceNavigator
{

class FNavigationToolItem;
class INavigationToolView;

/**
 * Chip Widgets that represent an Item in a compact way in the Items Column
 */
class SNavigationToolItemChip : public SBorder
{
public:
	DECLARE_DELEGATE_RetVal_TwoParams(FReply, FOnItemChipClicked, const FNavigationToolViewModelPtr& InItem, const FPointerEvent& InPointerEvent);

	SLATE_BEGIN_ARGS(SNavigationToolItemChip) {}
		SLATE_EVENT(FOnItemChipClicked, OnItemChipClicked)
		SLATE_EVENT(FOnDragOver, OnValidDragOver)
		SLATE_STYLE_ARGUMENT(FTableRowStyle, ChipStyle)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs
		, const FNavigationToolViewModelPtr& InItem
		, const TSharedPtr<INavigationToolView>& InView);

	//~ Begin SWidget
	virtual int32 OnPaint(const FPaintArgs& InArgs, const FGeometry& InAllottedGeometry, const FSlateRect& InCullingRect
		, FSlateWindowElementList& OutDrawElements, int32 LayerId
		, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	virtual FReply OnMouseButtonDown(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) override;
	virtual void OnMouseCaptureLost(const FCaptureLostEvent& InCaptureLostEvent) override;
	/*virtual FReply OnDragDetected(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) override;
	virtual FReply OnDragOver(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent) override;
	virtual FReply OnDrop(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent) override;
	virtual void OnDragLeave(const FDragDropEvent& InDragDropEvent) override;*/
	//~ End SWidget

	bool IsSelected() const;
	const FSlateBrush* GetItemBackgroundBrush() const;
	const FSlateBrush* GetDropIndicatorBrush(EItemDropZone InItemDropZone) const;

protected:
	bool IsPressed() const { return bIsPressed; }
	void Press();
	void Release();

	static EItemDropZone GetHoverZone(const FVector2f& InLocalPosition, const FVector2f& InLocalSize);

	FSlateColor GetIconColor() const;

	FText GetTooltipText() const;

	FNavigationToolViewModelWeakPtr WeakItem;

	TWeakPtr<INavigationToolView> WeakView;

	TOptional<EItemDropZone> ItemDropZone;

	const FTableRowStyle* ChipStyle = nullptr;

	FOnItemChipClicked OnItemChipClicked;

	FOnDragOver OnValidDragOver;

	bool bIsPressed = false;
};

} // namespace UE::SequenceNavigator
