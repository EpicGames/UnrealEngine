// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNavigationToolItemList.h"
#include "Items/NavigationToolItem.h"
#include "NavigationToolSettings.h"
#include "NavigationToolView.h"
#include "Widgets/Columns/SNavigationToolItemChip.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/SNavigationToolTreeRow.h"
#include "Utils/NavigationToolMiscUtils.h"

#define LOCTEXT_NAMESPACE "SNavigationToolItemList"

namespace UE::SequenceNavigator
{

using namespace Sequencer;

void SNavigationToolItemList::Construct(const FArguments& InArgs
	, const FNavigationToolViewModelPtr& InItem
	, const TSharedRef<INavigationToolView>& InView
	, const TSharedRef<SNavigationToolTreeRow>& InRowWidget)
{
	WeakItem = InItem;
	WeakView = InView;
	WeakRowWidget = InRowWidget;

	InView->GetOnToolViewRefreshed().AddSP(this, &SNavigationToolItemList::Refresh);
	InItem->OnExpansionChanged().AddSP(this, &SNavigationToolItemList::OnItemExpansionChanged);

	ChildSlot
		.HAlign(HAlign_Left)
		[
			SAssignNew(ItemListBox, SScrollBox)
			.ConsumeMouseWheel(EConsumeMouseWheel::Always)
			.Orientation(Orient_Horizontal)
			.ScrollBarThickness(FVector2D(2.f))
		];

	Refresh();
}

SNavigationToolItemList::~SNavigationToolItemList()
{
	using namespace Sequencer;

	if (const TSharedPtr<INavigationToolView> ToolView = WeakView.Pin())
	{
		ToolView->GetOnToolViewRefreshed().RemoveAll(this);
	}

	if (const FNavigationToolViewModelPtr Item = WeakItem.Pin())
	{
		Item->OnExpansionChanged().RemoveAll(this);
	}
}

void SNavigationToolItemList::OnItemExpansionChanged(const TSharedPtr<INavigationToolView>& InToolView, bool bInIsExpanded)
{
	Refresh();
}

void SNavigationToolItemList::Refresh()
{
	ItemListBox->ClearChildren();
	WeakChildItems.Reset();

	const FNavigationToolViewModelPtr ParentItem = WeakItem.Pin();
	const TSharedPtr<SNavigationToolTreeRow> TreeRow = WeakRowWidget.Pin();
	const TSharedPtr<FNavigationToolView> ToolView = StaticCastSharedPtr<FNavigationToolView>(WeakView.Pin());

	if (!ParentItem.IsValid() || !ToolView.IsValid() || !TreeRow.IsValid())
	{
		return;
	}

	TSet<FNavigationToolViewModelWeakPtr> WeakDisallowedItems;
	{
		TArray<FNavigationToolViewModelWeakPtr> WeakItemsToDisallow;

		// First get the children in the Navigation Tool View to see if we need to disallow some of the items
		// to avoid redundancy or unnecessarily showing other item's children (e.g. an actor's)
		static const TSet<FNavigationToolViewModelWeakPtr> WeakEmptySet;
		ToolView->GetChildrenOfItem(ParentItem, WeakItemsToDisallow, ENavigationToolItemViewMode::ItemTree, WeakEmptySet);

		// If Parent item is Collapsed, only disallow items that are top levels (as these should deal with their own item list)
		// Items that can't be top level should be visualized by the Parent Item when Collapsed
		if (!EnumHasAllFlags(ToolView->GetViewItemFlags(ParentItem), ENavigationToolItemFlags::Expanded))
		{
			// Keep only top level items
			WeakItemsToDisallow.RemoveAllSwap([](const FNavigationToolViewModelWeakPtr& InWeakChild)
				{
					const TViewModelPtr<INavigationToolItem> Child = InWeakChild.Pin();
					return !Child.IsValid() || !Child->CanBeTopLevel();
				}
				, EAllowShrinking::No);
		}

		WeakDisallowedItems.Append(MoveTemp(WeakItemsToDisallow));
	}

	TArray<FNavigationToolViewModelWeakPtr> WeakChildren;
	ToolView->GetChildrenOfItem(ParentItem, WeakChildren, ENavigationToolItemViewMode::HorizontalItemList, WeakDisallowedItems);

	// TArray<>::Pop will be removing the item from the end which results in the Item List being in reverse
	// so instead of immediately adding it to the slot , we will add it to this array and reverse it
	TArray<FNavigationToolViewModelPtr> ItemsToAdd;

	const INavigationToolView& ToolViewConstRef = *ToolView;

	while (!WeakChildren.IsEmpty())
	{
		const FNavigationToolViewModelPtr Child = WeakChildren.Pop().ImplicitPin();

		if (!Child.IsValid() || WeakDisallowedItems.Contains(Child))
		{
			continue;
		}

		if (Child->IsViewModeSupported(ENavigationToolItemViewMode::HorizontalItemList, ToolViewConstRef))
		{
			ItemsToAdd.Add(Child);
		}
		else
		{
			ToolView->GetChildrenOfItem(Child, WeakChildren, ENavigationToolItemViewMode::HorizontalItemList, WeakDisallowedItems);
		}
	}

	WeakChildItems.Reserve(ItemsToAdd.Num());

	for (int32 ItemIndex = ItemsToAdd.Num() - 1; ItemIndex >= 0; --ItemIndex)
	{
		const FNavigationToolViewModelPtr& Item = ItemsToAdd[ItemIndex];

		WeakChildItems.Add(Item);

		ItemListBox->AddSlot()
			.Padding(0.f, 1.f)
			[
				SNew(SNavigationToolItemChip, Item, ToolView)
				.ChipStyle(TreeRow->GetStyle())
				.OnItemChipClicked(this, &SNavigationToolItemList::OnItemChipSelected)
				.OnValidDragOver(this, &SNavigationToolItemList::OnItemChipValidDragOver)
			];
	}
}

FReply SNavigationToolItemList::OnItemChipSelected(const FNavigationToolViewModelPtr& InItem, const FPointerEvent& InPointerEvent)
{
	const TSharedPtr<INavigationToolView> ToolView = WeakView.Pin();
	const FNavigationToolViewModelPtr ParentItem = WeakItem.Pin();

	if (!InItem.IsValid() || !ToolView.IsValid() || !ParentItem.IsValid())
	{
		return FReply::Unhandled();
	}

	if (InPointerEvent.IsAltDown())
	{
		constexpr ENavigationToolItemSelectionFlags SelectionFlags = ENavigationToolItemSelectionFlags::SignalSelectionChange;
		ToolView->SelectItems({ InItem }, SelectionFlags);

		return FReply::Handled();
	}

	if (const TSharedPtr<INavigationTool> Tool = ToolView->GetOwnerTool())
	{
		FocusItemInSequencer(*Tool, InItem);
	}

	return FReply::Handled();
}

FReply SNavigationToolItemList::OnItemChipValidDragOver(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent)
{
	TSharedPtr<SNavigationToolTreeRow> TreeRow = WeakRowWidget.Pin();
	if (!TreeRow.IsValid())
	{
		return FReply::Unhandled();
	}

	// When Item Chip has a Valid Drag Over (i.e. a Supported Drag/Drop), make sure the Tree Row Holding this Item Chips simulates a Drag Leave
	// Slate App won't do it as it will still find the Widget under the mouse
	TreeRow->OnDragLeave(InDragDropEvent);

	return FReply::Handled();
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
