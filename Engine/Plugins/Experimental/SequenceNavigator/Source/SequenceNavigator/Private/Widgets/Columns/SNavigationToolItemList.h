// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NavigationToolDefines.h"
#include "Widgets/SCompoundWidget.h"

class SScrollBox;

namespace UE::SequenceNavigator
{

class INavigationToolView;
class SNavigationToolTreeRow;

/** Widget that visualizes the list of children an Item has when collapsed */
class SNavigationToolItemList : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNavigationToolItemList) {}
	SLATE_END_ARGS()

	virtual ~SNavigationToolItemList() override;

	void Construct(const FArguments& InArgs
		, const FNavigationToolViewModelPtr& InItem
		, const TSharedRef<INavigationToolView>& InView
		, const TSharedRef<SNavigationToolTreeRow>& InRowWidget);

	void OnItemExpansionChanged(const TSharedPtr<INavigationToolView>& InToolView, bool bInIsExpanded);

	void Refresh();

	FReply OnItemChipSelected(const FNavigationToolViewModelPtr& InItem, const FPointerEvent& InPointerEvent);

	FReply OnItemChipValidDragOver(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent);

protected:
	FNavigationToolViewModelWeakPtr WeakItem;

	TWeakPtr<INavigationToolView> WeakView;

	TWeakPtr<SNavigationToolTreeRow> WeakRowWidget;

	TArray<FNavigationToolViewModelWeakPtr> WeakChildItems;

	TSharedPtr<SScrollBox> ItemListBox;
};

} // namespace UE::SequenceNavigator
