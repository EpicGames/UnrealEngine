// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNavigationToolLabelComponent.h"
#include "Fonts/SlateFontInfo.h"
#include "Items/NavigationToolComponent.h"
#include "Styling/CoreStyle.h"

#define LOCTEXT_NAMESPACE "SNavigationToolLabelComponent"

namespace UE::SequenceNavigator
{

void SNavigationToolLabelComponent::Construct(const FArguments& InArgs
	, const FNavigationToolViewModelPtr& InItem
	, const TSharedRef<SNavigationToolTreeRow>& InRowWidget)
{
	SNavigationToolLabelItem::Construct(SNavigationToolLabelItem::FArguments(), InItem, InRowWidget);
}

const FInlineEditableTextBlockStyle* SNavigationToolLabelComponent::GetTextBlockStyle() const
{
	static FInlineEditableTextBlockStyle Style = *SNavigationToolLabelItem::GetTextBlockStyle();
	static bool bFontSet = false;

	if (!bFontSet)
	{
		Style.TextStyle.Font = FSlateFontInfo(FCoreStyle::GetDefaultFont(), 10, TEXT("Italic"));
		bFontSet = true;
	}

	return &Style;
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
