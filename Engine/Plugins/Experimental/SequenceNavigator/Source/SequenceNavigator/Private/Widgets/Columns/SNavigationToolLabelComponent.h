// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NavigationToolDefines.h"
#include "SNavigationToolLabelItem.h"

struct FInlineEditableTextBlockStyle;

namespace UE::SequenceNavigator
{

class FNavigationToolComponent;
class SNavigationToolTreeRow;

class SNavigationToolLabelComponent : public SNavigationToolLabelItem
{
public:
	SLATE_BEGIN_ARGS(SNavigationToolLabelComponent) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs
		, const FNavigationToolViewModelPtr& InItem
		, const TSharedRef<SNavigationToolTreeRow>& InRowWidget);

	//~ Begin SNavigationToolLabelItem
	virtual const FInlineEditableTextBlockStyle* GetTextBlockStyle() const override;
	//~ End SNavigationToolLabelItem
};

} // namespace UE::SequenceNavigator
