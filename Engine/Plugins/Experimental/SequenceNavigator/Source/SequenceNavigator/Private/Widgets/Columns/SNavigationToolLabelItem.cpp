// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNavigationToolLabelItem.h"
#include "ActorEditorUtils.h"
#include "INavigationTool.h"
#include "Items/INavigationToolItem.h"
#include "Items/NavigationToolSequence.h"
#include "NavigationToolView.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNavigationToolExpanderArrow.h"
#include "Widgets/SNavigationToolTreeRow.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

#define LOCTEXT_NAMESPACE "SNavigationToolLabelItem"

namespace UE::SequenceNavigator
{

using namespace Sequencer;

void SNavigationToolLabelItem::Construct(const FArguments& InArgs
	, const FNavigationToolViewModelPtr& InItem
	, const TSharedRef<SNavigationToolTreeRow>& InRowWidget)
{
	check(InItem.IsValid());
	WeakItem = InItem;
	WeakView = InRowWidget->GetToolView();

	const TSharedRef<INavigationToolItem> ItemRef = InItem.ToSharedRef();

	InItem->OnRenameAction().AddSP(this, &SNavigationToolLabelItem::OnRenameAction);

	ChildSlot
	[
		SNew(SBox)
		.MinDesiredHeight(25)
		.Padding(3.f, 0.f, 0.f, 0.f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(4.f, 0.f, 0.f, 0.f)
			[
				SNew(SNavigationToolExpanderArrow, InRowWidget)
				.ExpanderArrowArgs(SExpanderArrow::FArguments()
				.IndentAmount(12)
				.ShouldDrawWires(true))
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.Padding(2.f, 0.f, 0.f, 0.f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(FMargin(0.f, 1.f, 6.f, 1.f))
				[
					SNew(SBox)
					.WidthOverride(16.f)
					.HeightOverride(16.f)
					[
						SNew(SImage)
						.IsEnabled(this, &SNavigationToolLabelItem::IsItemEnabled)
						.ColorAndOpacity(ItemRef, &INavigationToolItem::GetIconColor)
						.Image(ItemRef, &INavigationToolItem::GetIconBrush)
						.ToolTipText(ItemRef, &INavigationToolItem::GetIconTooltipText)
					]
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				.Padding(0.0f, 0.0f)
				[
					SAssignNew(InlineTextBlock, SInlineEditableTextBlock)
					.IsEnabled(this, &SNavigationToolLabelItem::IsItemEnabled)
					.Text(this, &SNavigationToolLabelItem::GetItemText)
					.Style(GetTextBlockStyle())
					.Font(FAppStyle::GetFontStyle(TEXT("Sequencer.AnimationOutliner.RegularFont")))
					.HighlightText(InRowWidget->GetHighlightText())
					.OnTextCommitted(this, &SNavigationToolLabelItem::OnLabelTextCommitted)
					.OnVerifyTextChanged(this, &SNavigationToolLabelItem::OnVerifyItemLabelChanged)
					.OnEnterEditingMode(this, &SNavigationToolLabelItem::OnEnterEditingMode)
					.OnExitEditingMode(this, &SNavigationToolLabelItem::OnExitEditingMode)
					.IsSelected(FIsSelected::CreateSP(InRowWidget, &SNavigationToolTreeRow::IsSelectedExclusively))
					.IsReadOnly(this, &SNavigationToolLabelItem::IsReadOnly)
				]
			]
		]
	];
}

bool SNavigationToolLabelItem::IsReadOnly() const
{
	const FNavigationToolViewModelPtr Item = WeakItem.Pin();
	if (!Item.IsValid())
	{
		return true;
	}

	if (const TViewModelPtr<IRenameableExtension> RenameableExtension = Item.ImplicitCast())
	{
		if (!RenameableExtension->CanRename())
		{
			return true;
		}
	}

	const TSharedPtr<INavigationToolView> ToolView = WeakView.Pin();
	if (!ToolView.IsValid())
	{
		return true;
	}

	return ToolView->IsItemReadOnly(Item);
}

bool SNavigationToolLabelItem::IsItemEnabled() const
{
	const TSharedPtr<INavigationToolView> View = WeakView.Pin();
	if (!View.IsValid())
	{
		return false;
	}

	const FNavigationToolViewModelPtr Item = WeakItem.Pin();
	if (!Item.IsValid())
	{
		return false;
	}

	return !View->IsItemReadOnly(Item);
}

FText SNavigationToolLabelItem::GetItemText() const
{
	const FNavigationToolViewModelPtr Item = WeakItem.Pin();
	if (!Item.IsValid())
	{
		return FText::GetEmpty();
	}

	if (bInEditingMode)
	{
		return Item->GetDisplayName();
	}

	return Item->GetDisplayName();
}

bool SNavigationToolLabelItem::OnVerifyItemLabelChanged(const FText& InLabel, FText& OutErrorMessage)
{
	return FActorEditorUtils::ValidateActorName(InLabel, OutErrorMessage);
}

void SNavigationToolLabelItem::OnLabelTextCommitted(const FText& InLabel, ETextCommit::Type InCommitInfo)
{
	const FNavigationToolViewModelPtr Item = WeakItem.Pin();
	if (!Item.IsValid())
	{
		return;
	}

	switch (InCommitInfo)
	{
		case ETextCommit::OnEnter:
		case ETextCommit::OnUserMovedFocus:
			RenameItem(InLabel);
			Item->OnRenameAction().Broadcast(ENavigationToolRenameAction::Completed, WeakView.Pin());
			break;

		case ETextCommit::Default:
		case ETextCommit::OnCleared:
		default:
			Item->OnRenameAction().Broadcast(ENavigationToolRenameAction::Cancelled, WeakView.Pin());
			break;
	}
}

void SNavigationToolLabelItem::RenameItem(const FText& InLabel)
{
	const FNavigationToolViewModelPtr Item = WeakItem.Pin();
	if (!Item.IsValid())
	{
		return;
	}

	const TViewModelPtr<IRenameableExtension> RenameableExtension = Item.ImplicitCast();
	if (!RenameableExtension.IsValid())
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("ItemRename", "Rename Item"));

	RenameableExtension->Rename(InLabel);

	if (WeakView.IsValid())
	{
		WeakView.Pin()->SetKeyboardFocus();
	}
}

void SNavigationToolLabelItem::OnRenameAction(ENavigationToolRenameAction InRenameAction, const TSharedPtr<INavigationToolView>& InToolView) const
{
	if (InRenameAction == ENavigationToolRenameAction::Requested && InToolView == WeakView)
	{
		InlineTextBlock->EnterEditingMode();
	}
}

void SNavigationToolLabelItem::OnEnterEditingMode()
{
	bInEditingMode = true;
}

void SNavigationToolLabelItem::OnExitEditingMode()
{
	bInEditingMode = false;
}

const FInlineEditableTextBlockStyle* SNavigationToolLabelItem::GetTextBlockStyle() const
{
	return &FCoreStyle::Get().GetWidgetStyle<FInlineEditableTextBlockStyle>(TEXT("InlineEditableTextBlockStyle"));
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
