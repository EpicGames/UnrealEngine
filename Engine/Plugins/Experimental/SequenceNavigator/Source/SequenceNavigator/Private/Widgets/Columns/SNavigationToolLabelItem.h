// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NavigationToolDefines.h"
#include "Styling/SlateTypes.h"
#include "Widgets/SCompoundWidget.h"

class SInlineEditableTextBlock;
struct FSlateColor;

namespace UE::SequenceNavigator
{

class FNavigationToolItem;
class FNavigationTool;
class INavigationToolView;
class SNavigationToolTreeRow;

class SNavigationToolLabelItem : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNavigationToolLabelItem) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs
		, const FNavigationToolViewModelPtr& InItem
		, const TSharedRef<SNavigationToolTreeRow>& InRowWidget);

	bool IsReadOnly() const;

	bool IsItemEnabled() const;

	FText GetItemText() const;

	bool OnVerifyItemLabelChanged(const FText& InLabel, FText& OutErrorMessage);

	void OnLabelTextCommitted(const FText& InLabel, ETextCommit::Type InCommitInfo);
	void RenameItem(const FText& InLabel);

	void OnRenameAction(ENavigationToolRenameAction InRenameAction, const TSharedPtr<INavigationToolView>& InToolView) const;

	void OnEnterEditingMode();
	void OnExitEditingMode();

	virtual const FInlineEditableTextBlockStyle* GetTextBlockStyle() const;

protected:
	FNavigationToolViewModelWeakPtr WeakItem;

	TWeakPtr<INavigationToolView> WeakView;

	TSharedPtr<SInlineEditableTextBlock> InlineTextBlock;

	bool bInEditingMode = false;
};

} // namespace UE::SequenceNavigator
