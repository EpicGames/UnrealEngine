// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNavigationToolLength.h"
#include "ISequencer.h"
#include "Items/NavigationToolItemUtils.h"
#include "NavigationToolView.h"
#include "Sections/MovieSceneSubSection.h"
#include "Widgets/SNavigationToolTreeRow.h"

#define LOCTEXT_NAMESPACE "SNavigationToolLength"

namespace UE::SequenceNavigator
{

using namespace Sequencer;

void SNavigationToolLength::Construct(const FArguments& InArgs
	, const FNavigationToolViewModelPtr& InItem
	, const TSharedRef<INavigationToolView>& InView
	, const TSharedRef<SNavigationToolTreeRow>& InRowWidget)
{
	WeakItem = InItem;
	WeakView = InView;
	WeakRowWidget = InRowWidget;

	SNavigationToolTime::Construct(SNavigationToolTime::FArguments(), InItem, InView, InRowWidget);
}

double SNavigationToolLength::GetFrameTimeValue() const
{
	if (const UMovieSceneSubSection* const SubSection = ItemUtils::GetSequenceItemSubSection(WeakItem.Pin()))
	{
		return SubSection->GetRange().Size<FFrameNumber>().Value;
	}
	return 0.;
}

FText SNavigationToolLength::GetTransactionText() const
{
	return LOCTEXT("SetLengthTransaction", "Set Length");
}

TSharedPtr<INumericTypeInterface<double>> SNavigationToolLength::GetNumericTypeInterface() const
{
	if (const TSharedPtr<ISequencer> Sequencer = GetSequencer())
	{
		return Sequencer->GetNumericTypeInterface(ENumericIntent::Duration);
	}
	return nullptr;
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
