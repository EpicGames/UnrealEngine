// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNavigationToolLock.h"
#include "Items/INavigationToolItem.h"
#include "Items/NavigationToolItemUtils.h"
#include "NavigationToolSettings.h"
#include "NavigationToolView.h"
#include "Widgets/SNavigationToolTreeRow.h"

#define LOCTEXT_NAMESPACE "SNavigationToolLock"

namespace UE::SequenceNavigator
{

using namespace Sequencer;

class FLockDragDropOp : public FDragDropOperation
{
public:
	DRAG_DROP_OPERATOR_TYPE(FLockDragDropOp, FDragDropOperation)

	/** Flag which defines whether to lock destination items or not */
	bool bShouldLock;

	/** Undo transaction stolen from the gutter which is kept alive for the duration of the drag */
	TUniquePtr<FScopedTransaction> UndoTransaction;

	/** The widget decorator to use */
	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override
	{
		return SNullWidget::NullWidget;
	}

	/** Create a new drag and drop operation out of the specified flag */
	static TSharedRef<FLockDragDropOp> New(const bool bShouldLock, TUniquePtr<FScopedTransaction>& ScopedTransaction)
	{
		const TSharedRef<FLockDragDropOp> Operation = MakeShared<FLockDragDropOp>();
		Operation->bShouldLock     = bShouldLock;
		Operation->UndoTransaction = MoveTemp(ScopedTransaction);
		Operation->Construct();
		return Operation;
	}
};

ELockableLockState GetItemLockState(const FNavigationToolViewModelPtr& InItem)
{
	if (const TViewModelPtr<ILockableExtension> LockableItem = InItem.ImplicitCast())
	{
		return LockableItem->GetLockState();
	}
	return ELockableLockState::None;
}

void SetItemLocked(const FNavigationToolViewModelPtr& InItem, const bool bInIsLocked)
{
	if (const TViewModelPtr<ILockableExtension> LockableItem = InItem.ImplicitCast())
	{
		LockableItem->SetIsLocked(bInIsLocked);
	}
}

void SNavigationToolLock::Construct(const FArguments& InArgs
	, const FNavigationToolViewModelPtr& InItem
	, const TSharedRef<INavigationToolView>& InView
	, const TSharedRef<SNavigationToolTreeRow>& InRowWidget)
{
	WeakItem = InItem;
	WeakView = InView;
	WeakRowWidget = InRowWidget;

	SImage::Construct(SImage::FArguments()
		.ColorAndOpacity(this, &SNavigationToolLock::GetForegroundColor)
		.Image(this, &SNavigationToolLock::GetBrush));
}

FSlateColor SNavigationToolLock::GetForegroundColor() const
{
	const FNavigationToolViewModelPtr Item = WeakItem.Pin();
	if (!Item.IsValid())
	{
		return FSlateColor::UseForeground();
	}

	const bool bIsItemSelected = WeakView.IsValid() && WeakView.Pin()->IsItemSelected(Item);
	const bool bIsItemHovered = WeakRowWidget.IsValid() && WeakRowWidget.Pin()->IsHovered();
	const bool bAlwaysShowLock = GetDefault<UNavigationToolSettings>()->ShouldAlwaysShowLockState();

	// We can hide the brush if settings for "Always Showing Lock State" is OFF
	// and item is not locked while also not being selected or hovered
	if (!bAlwaysShowLock
		&& GetLockState() == ELockableLockState::None
		&& !bIsItemSelected
		&& !bIsItemHovered)
	{
		return FLinearColor::Transparent;
	}

	if (bIsItemHovered || IsHovered())
	{
		switch (GetLockState())
		{
		case ELockableLockState::None:
			return FStyleColors::White25;

		case ELockableLockState::PartiallyLocked:
			return FStyleColors::ForegroundHover;

		case ELockableLockState::Locked:
			return FStyleColors::ForegroundHover;
		}
	}
	else
	{
		switch (GetLockState())
		{
		case ELockableLockState::None:
			return FStyleColors::Transparent;

		case ELockableLockState::PartiallyLocked:
			return FStyleColors::White25;

		case ELockableLockState::Locked:
			return FStyleColors::Foreground;
		}
	}

	return FSlateColor::UseForeground();
}

const FSlateBrush* SNavigationToolLock::GetBrush() const
{
	return (GetLockState() == ELockableLockState::None)
		? FAppStyle::GetBrush(TEXT("Icons.Unlock"))
		: FAppStyle::GetBrush(TEXT("Icons.Lock"));
}

FReply SNavigationToolLock::OnDragDetected(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	if (InPointerEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		const bool bShouldLock = GetLockState() == ELockableLockState::Locked;
		return FReply::Handled().BeginDragDrop(FLockDragDropOp::New(bShouldLock, UndoTransaction));
	}
	return FReply::Unhandled();
}

void SNavigationToolLock::OnDragEnter(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent)
{
	if (const TSharedPtr<FLockDragDropOp> DragDropOp = InDragDropEvent.GetOperationAs<FLockDragDropOp>())
	{
		SetIsLocked(DragDropOp->bShouldLock);
	}
}

FReply SNavigationToolLock::HandleClick()
{
	const TSharedPtr<INavigationToolView> ToolView = WeakView.Pin();
	if (!ToolView.IsValid())
	{
		return FReply::Unhandled();
	}

	const FNavigationToolViewModelPtr Item = WeakItem.Pin();
	if (!Item.IsValid())
	{
		return FReply::Unhandled();
	}

	UndoTransaction = MakeUnique<FScopedTransaction>(LOCTEXT("SetNavigationToolItemLock", "Set Item Lock"));

	const bool bNewIsLocked = (GetLockState() != ELockableLockState::Locked);

	// We operate on all the selected items if the specified item is selected
	if (ToolView->IsItemSelected(Item))
	{
		for (FNavigationToolViewModelWeakPtr& WeakSelectedItem : ToolView->GetSelectedItems())
		{
			SetItemLocked(WeakSelectedItem.Pin(), bNewIsLocked);
		}
	}
	else
	{
		SetIsLocked(bNewIsLocked);
	}

	return FReply::Handled().DetectDrag(SharedThis(this), EKeys::LeftMouseButton);
}

FReply SNavigationToolLock::OnMouseButtonDoubleClick(const FGeometry& InInGeometry, const FPointerEvent& InPointerEvent)
{
	return HandleClick();
}

FReply SNavigationToolLock::OnMouseButtonDown(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	if (InPointerEvent.GetEffectingButton() != EKeys::LeftMouseButton)
	{
		return FReply::Unhandled();
	}
	return HandleClick();
}

FReply SNavigationToolLock::OnMouseButtonUp(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	if (InPointerEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		UndoTransaction.Reset();

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SNavigationToolLock::OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent)
{
	UndoTransaction.Reset();
}

ELockableLockState SNavigationToolLock::GetLockState() const
{
	return GetItemLockState(WeakItem.Pin());
}

void SNavigationToolLock::SetIsLocked(const bool bInIsLocked)
{
	SetItemLocked(WeakItem.Pin(), bInIsLocked);
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
