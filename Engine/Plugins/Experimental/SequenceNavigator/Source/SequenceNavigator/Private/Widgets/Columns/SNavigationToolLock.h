// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/Extensions/ILockableExtension.h"
#include "NavigationToolDefines.h"
#include "ScopedTransaction.h"
#include "Templates/UniquePtr.h"
#include "Widgets/Images/SImage.h"

class FDragDropEvent;
class FReply;
struct FCaptureLostEvent;
struct FGeometry;
struct FPointerEvent;
struct FSlateBrush;
struct FSlateColor;

namespace UE::SequenceNavigator
{

class INavigationToolView;
class SNavigationToolTreeRow;

class SNavigationToolLock : public SImage
{
public:
	SLATE_BEGIN_ARGS(SNavigationToolLock) {}
	SLATE_END_ARGS()

	virtual ~SNavigationToolLock() override = default;

	void Construct(const FArguments& InArgs
		, const FNavigationToolViewModelPtr& InItem
		, const TSharedRef<INavigationToolView>& InView
		, const TSharedRef<SNavigationToolTreeRow>& InRowWidget);

	virtual FSlateColor GetForegroundColor() const override;

	Sequencer::ELockableLockState GetLockState() const;

	void SetIsLocked(const bool bInIsLocked);

	const FSlateBrush* GetBrush() const;

	virtual FReply OnDragDetected(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) override;

	/** If a lock drag drop operation has entered this widget, set its item to the new lock state */
	virtual void OnDragEnter(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent) override;

	FReply HandleClick();

	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InInGeometry, const FPointerEvent& InPointerEvent) override;

	virtual FReply OnMouseButtonDown(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) override;

	virtual FReply OnMouseButtonUp(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) override;

	virtual void OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent) override;

	//void OnSetItemLockedState(const FNavigationToolViewModelPtr& InItem, const bool bInIsLocked);

protected:
	FNavigationToolViewModelWeakPtr WeakItem;

	TWeakPtr<INavigationToolView> WeakView;

	TWeakPtr<SNavigationToolTreeRow> WeakRowWidget;

	TUniquePtr<FScopedTransaction> UndoTransaction;
};

} // namespace UE::SequenceNavigator
