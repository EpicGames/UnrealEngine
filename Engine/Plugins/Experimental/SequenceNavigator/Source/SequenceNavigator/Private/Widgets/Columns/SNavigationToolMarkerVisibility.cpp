// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNavigationToolMarkerVisibility.h"
#include "Columns/NavigationToolMarkerVisibilityColumn.h"
#include "Items/INavigationToolItem.h"
#include "Items/NavigationToolItemUtils.h"
#include "NavigationToolView.h"
#include "Styling/StyleColors.h"
#include "Widgets/SNavigationToolTreeRow.h"

#define LOCTEXT_NAMESPACE "SNavigationToolMarkerVisibility"

namespace UE::SequenceNavigator
{

using namespace Sequencer;

class FMarkerVisibilityDragDropOp : public FDragDropOperation
{
public:
	DRAG_DROP_OPERATOR_TYPE(FMarkerVisibilityDragDropOp, FDragDropOperation)

	/** Flag which defines whether to hide destination items or not */
	EItemMarkerVisibility MarkerVisibility;

	/** Undo transaction stolen from the gutter which is kept alive for the duration of the drag */
	TUniquePtr<FScopedTransaction> UndoTransaction;

	/** The widget decorator to use */
	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override
	{
		return SNullWidget::NullWidget;
	}

	/** Create a new drag and drop operation out of the specified flag */
	static TSharedRef<FMarkerVisibilityDragDropOp> New(const EItemMarkerVisibility InMarkerVisibility, TUniquePtr<FScopedTransaction>& ScopedTransaction)
	{
		TSharedRef<FMarkerVisibilityDragDropOp> Operation = MakeShared<FMarkerVisibilityDragDropOp>();
		Operation->MarkerVisibility = InMarkerVisibility;
		Operation->UndoTransaction = MoveTemp(ScopedTransaction);
		Operation->Construct();
		return Operation;
	}
};

EItemMarkerVisibility GetItemMarkerVisibility(const FNavigationToolViewModelPtr& InItem)
{
	if (!InItem.IsValid())
	{
		return EItemMarkerVisibility::None;
	}

	const TViewModelPtr<IMarkerVisibilityExtension> MarkerVisibilityItem = InItem.ImplicitCast();
	if (!MarkerVisibilityItem)
	{
		return EItemMarkerVisibility::None;
	}

	return MarkerVisibilityItem->GetMarkerVisibility();
}

void SetItemMarkerVisibility(const FNavigationToolViewModelPtr& InItem, const bool bInMarkersVisible)
{
	if (!InItem.IsValid())
	{
		return;
	}

	if (const TViewModelPtr<IMarkerVisibilityExtension> MarkerVisibilityItem = InItem.ImplicitCast())
	{
		MarkerVisibilityItem->SetMarkerVisibility(bInMarkersVisible);
	}
}

void SNavigationToolMarkerVisibility::Construct(const FArguments& InArgs
	, const TSharedRef<FNavigationToolMarkerVisibilityColumn>& InColumn
	, const FNavigationToolViewModelPtr& InItem
	, const TSharedRef<INavigationToolView>& InView
	, const TSharedRef<SNavigationToolTreeRow>& InRowWidget)
{
	WeakColumn = InColumn;
	WeakItem = InItem;
	WeakView = InView;
	WeakRowWidget = InRowWidget;

	SImage::Construct(SImage::FArguments()
		.IsEnabled(this, &SNavigationToolMarkerVisibility::IsVisibilityWidgetEnabled)
		.ColorAndOpacity(this, &SNavigationToolMarkerVisibility::GetForegroundColor)
		.Image(this, &SNavigationToolMarkerVisibility::GetBrush));
}

FReply SNavigationToolMarkerVisibility::OnDragDetected(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	if (InPointerEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		return FReply::Handled().BeginDragDrop(FMarkerVisibilityDragDropOp::New(GetMarkerVisibility(), UndoTransaction));
	}
	return FReply::Unhandled();
}

void SNavigationToolMarkerVisibility::OnDragEnter(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent)
{
	const TSharedPtr<FMarkerVisibilityDragDropOp> DragDropOp = InDragDropEvent.GetOperationAs<FMarkerVisibilityDragDropOp>();
	if (DragDropOp.IsValid())
	{
		const bool bNewVisible = (DragDropOp->MarkerVisibility != EItemMarkerVisibility::Visible);
		SetMarkersVisible(bNewVisible);
	}
}

FReply SNavigationToolMarkerVisibility::HandleClick()
{
	const TSharedPtr<INavigationToolView> ToolView = WeakView.Pin();
	if (!ToolView.IsValid())
	{
		return FReply::Unhandled();
	}

	const FNavigationToolViewModelPtr Item = WeakItem.Pin();
	if (!Item.IsValid())
	{
		return FReply::Unhandled();
	}

	const TSharedPtr<FNavigationToolMarkerVisibilityColumn> Column = WeakColumn.Pin();
	if (!Column.IsValid())
	{
		return FReply::Unhandled();
	}

	if (!IsVisibilityWidgetEnabled())
	{
		return FReply::Unhandled();
	}

	// Open an undo transaction
	UndoTransaction = MakeUnique<FScopedTransaction>(LOCTEXT("SetNavigationToolMarkerVisibility", "Set Marker Visibility"));

	const bool bNewVisible = (GetMarkerVisibility() != EItemMarkerVisibility::Visible);

	// We operate on all the selected items if the specified item is selected
	if (ToolView->IsItemSelected(Item))
	{
		for (const FNavigationToolViewModelWeakPtr& WeakSelectedItem : ToolView->GetSelectedItems())
		{
			if (const TViewModelPtr<IMarkerVisibilityExtension> MarkerVisibilityItem = WeakSelectedItem.ImplicitPin())
			{
				MarkerVisibilityItem->SetMarkerVisibility(bNewVisible);
			}
		}
	}
	else
	{
		SetMarkersVisible(bNewVisible);
	}

	return FReply::Handled().DetectDrag(SharedThis(this), EKeys::LeftMouseButton);
}

FReply SNavigationToolMarkerVisibility::OnMouseButtonDoubleClick(const FGeometry& InInGeometry, const FPointerEvent& InPointerEvent)
{
	return HandleClick();
}

FReply SNavigationToolMarkerVisibility::OnMouseButtonDown(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	if (InPointerEvent.GetEffectingButton() != EKeys::LeftMouseButton)
	{
		return FReply::Unhandled();
	}

	return HandleClick();
}

FReply SNavigationToolMarkerVisibility::OnMouseButtonUp(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	if (InPointerEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		UndoTransaction.Reset();
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SNavigationToolMarkerVisibility::OnMouseCaptureLost(const FCaptureLostEvent& InCaptureLostEvent)
{
	UndoTransaction.Reset();
}

const FSlateBrush* SNavigationToolMarkerVisibility::GetBrush() const
{
	return FAppStyle::GetBrush(TEXT("AnimTimeline.SectionMarker"));
}

FSlateColor SNavigationToolMarkerVisibility::GetForegroundColor() const
{
	const TViewModelPtr<IMarkerVisibilityExtension> Item = WeakItem.ImplicitPin();
	if (!Item.IsValid())
	{
		return FLinearColor::Transparent;
	}

	const bool bIsItemHovered = WeakRowWidget.IsValid() && WeakRowWidget.Pin()->IsHovered();

	if (IsHovered() || bIsItemHovered)
	{
		switch (GetMarkerVisibility())
		{
		case EItemMarkerVisibility::None:
			return FStyleColors::White25;

		case EItemMarkerVisibility::PartiallyVisible:
			return FStyleColors::ForegroundHover;

		case EItemMarkerVisibility::Visible:
			return FStyleColors::ForegroundHover;
		}
	}
	else
	{
		switch (GetMarkerVisibility())
		{
		case EItemMarkerVisibility::None:
			return FStyleColors::Transparent;

		case EItemMarkerVisibility::PartiallyVisible:
			return FStyleColors::White25;

		case EItemMarkerVisibility::Visible:
			return FStyleColors::Foreground;
		}
	}

	return FStyleColors::Transparent;
}

EItemMarkerVisibility SNavigationToolMarkerVisibility::GetMarkerVisibility() const
{
	return GetItemMarkerVisibility(WeakItem.Pin());
}

void SNavigationToolMarkerVisibility::SetMarkersVisible(const bool bInVisible)
{
	SetItemMarkerVisibility(WeakItem.Pin(), bInVisible);
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
