// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Extensions/IMarkerVisibilityExtension.h"
#include "NavigationToolDefines.h"
#include "ScopedTransaction.h"
#include "Templates/UniquePtr.h"
#include "Widgets/Images/SImage.h"

namespace UE::SequenceNavigator
{

class FNavigationToolMarkerVisibilityColumn;
class INavigationToolView;
class SNavigationToolTreeRow;

/** Widget responsible for managing the visibility for a single item */
class SNavigationToolMarkerVisibility : public SImage
{
public:
	SLATE_BEGIN_ARGS(SNavigationToolMarkerVisibility) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs
		, const TSharedRef<FNavigationToolMarkerVisibilityColumn>& InColumn
		, const FNavigationToolViewModelPtr& InItem
		, const TSharedRef<INavigationToolView>& InView
		, const TSharedRef<SNavigationToolTreeRow>& InRowWidget);

protected:
	/** Returns whether the widget is enabled or not */
	virtual bool IsVisibilityWidgetEnabled() const { return true; }

	virtual const FSlateBrush* GetBrush() const;

	virtual FReply OnDragDetected(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) override;
	virtual void OnDragEnter(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent) override;

	FReply HandleClick();

	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InInGeometry, const FPointerEvent& InPointerEvent) override;
	virtual FReply OnMouseButtonDown(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) override;
	virtual void OnMouseCaptureLost(const FCaptureLostEvent& InCaptureLostEvent) override;

	virtual FSlateColor GetForegroundColor() const override;
	
	EItemMarkerVisibility GetMarkerVisibility() const;

	void SetMarkersVisible(const bool bInVisible);

	TWeakPtr<FNavigationToolMarkerVisibilityColumn> WeakColumn;

	FNavigationToolViewModelWeakPtr WeakItem;

	TWeakPtr<INavigationToolView> WeakView;

	TWeakPtr<SNavigationToolTreeRow> WeakRowWidget;

	TUniquePtr<FScopedTransaction> UndoTransaction;
};

} // namespace UE::SequenceNavigator
