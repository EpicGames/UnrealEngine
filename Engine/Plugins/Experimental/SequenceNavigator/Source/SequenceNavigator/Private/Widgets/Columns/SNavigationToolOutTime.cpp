// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNavigationToolOutTime.h"
#include "Items/NavigationToolItemUtils.h"
#include "Items/NavigationToolSequence.h"
#include "NavigationToolView.h"
#include "ScopedTransaction.h"
#include "Widgets/SNavigationToolTreeRow.h"

#define LOCTEXT_NAMESPACE "SNavigationToolOutTime"

namespace UE::SequenceNavigator
{

using namespace Sequencer;

void SNavigationToolOutTime::Construct(const FArguments& InArgs
	, const FNavigationToolViewModelPtr& InItem
	, const TSharedRef<INavigationToolView>& InView
	, const TSharedRef<SNavigationToolTreeRow>& InRowWidget)
{
	WeakItem = InItem;
	WeakView = InView;
	WeakRowWidget = InRowWidget;

	SNavigationToolTime::Construct(SNavigationToolTime::FArguments(), InItem, InView, InRowWidget);
}

FName SNavigationToolOutTime::GetStyleName() const
{
	return TEXT("SpinBox.OutTime");
}

double SNavigationToolOutTime::GetFrameTimeValue() const
{
	if (const TViewModelPtr<IOutTimeExtension> Item = WeakItem.ImplicitPin())
	{
		return Item->GetOutTime().Value;
	}
	return 0.;
}

void SNavigationToolOutTime::OnFrameTimeValueCommitted(const double InNewValue, const ETextCommit::Type InCommitType)
{
	const TViewModelPtr<IOutTimeExtension> Item = WeakItem.ImplicitPin();
	if (!Item.IsValid())
	{
		return;
	}

	const bool bShouldTransact = !UndoTransaction.IsValid() && (InCommitType == ETextCommit::OnEnter);
	const FScopedTransaction Transaction(GetTransactionText(), bShouldTransact);

	return Item->SetOutTime(static_cast<int32>(InNewValue));
}

FText SNavigationToolOutTime::GetTransactionText() const
{
	return LOCTEXT("SetOutTimeTransaction", "Set Out Time");
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
