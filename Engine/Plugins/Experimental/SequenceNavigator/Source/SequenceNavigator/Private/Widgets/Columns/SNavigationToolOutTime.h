// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NavigationToolDefines.h"
#include "SNavigationToolTime.h"

namespace UE::SequenceNavigator
{

class FNavigationToolSequence;
class INavigationToolView;
class IOutTimeExtension;
class SNavigationToolTreeRow;

class SNavigationToolOutTime : public SNavigationToolTime
{
public:
	SLATE_BEGIN_ARGS(SNavigationToolOutTime) {}
	SLATE_END_ARGS()

	virtual ~SNavigationToolOutTime() override = default;

	void Construct(const FArguments& InArgs
		, const FNavigationToolViewModelPtr& InItem
		, const TSharedRef<INavigationToolView>& InView
		, const TSharedRef<SNavigationToolTreeRow>& InRowWidget);

protected:
	//~ Begin SNavigationToolTime
	virtual FName GetStyleName() const override;
	virtual double GetFrameTimeValue() const override;
	virtual void OnFrameTimeValueCommitted(const double InNewValue, const ETextCommit::Type InCommitType) override;
	virtual FText GetTransactionText() const override;
	//~ End SNavigationToolTime
};

} // namespace UE::SequenceNavigator
