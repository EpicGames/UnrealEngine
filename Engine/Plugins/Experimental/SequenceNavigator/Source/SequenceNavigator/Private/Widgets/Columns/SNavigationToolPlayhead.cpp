// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNavigationToolPlayhead.h"
#include "Columns/NavigationToolPlayheadColumn.h"
#include "INavigationTool.h"
#include "Items/INavigationToolItem.h"
#include "Items/NavigationToolItemUtils.h"
#include "NavigationToolView.h"
#include "Styling/StyleColors.h"
#include "Widgets/SNavigationToolTreeRow.h"

#define LOCTEXT_NAMESPACE "SNavigationToolPlayhead"

namespace UE::SequenceNavigator
{

using namespace Sequencer;

void SNavigationToolPlayhead::Construct(const FArguments& InArgs
	, const TSharedRef<FNavigationToolPlayheadColumn>& InColumn
	, const FNavigationToolViewModelPtr& InItem
	, const TSharedRef<INavigationToolView>& InView
	, const TSharedRef<SNavigationToolTreeRow>& InRowWidget)
{
	check(InItem.IsValid());
	WeakColumn = InColumn;
	WeakItem = InItem;
	WeakView = InView;
	WeakRowWidget = InRowWidget;

	SImage::Construct(SImage::FArguments()
		.IsEnabled(this, &SNavigationToolPlayhead::IsVisibilityWidgetEnabled)
		.ColorAndOpacity(this, &SNavigationToolPlayhead::GetForegroundColor)
		.Image(this, &SNavigationToolPlayhead::GetBrush));
}

const FSlateBrush* SNavigationToolPlayhead::GetBrush() const
{
	return FAppStyle::GetBrush(TEXT("GenericPlay"));
}

FSlateColor SNavigationToolPlayhead::GetForegroundColor() const
{
	const bool bIsItemHovered = WeakRowWidget.IsValid() && WeakRowWidget.Pin()->IsHovered();

	if (IsHovered() || bIsItemHovered)
	{
		switch (GetContainsPlayhead())
		{
		case EItemContainsPlayhead::None:
			return FStyleColors::White25;

		case EItemContainsPlayhead::PartiallyContainsPlayhead:
			return FStyleColors::ForegroundHover.GetSpecifiedColor() * FStyleColors::AccentGreen.GetSpecifiedColor();

		case EItemContainsPlayhead::ContainsPlayhead:
			return FStyleColors::ForegroundHover.GetSpecifiedColor() * FStyleColors::AccentGreen.GetSpecifiedColor();
		}
	}
	else
	{
		switch (GetContainsPlayhead())
		{
		case EItemContainsPlayhead::None:
			return FStyleColors::Transparent;

		case EItemContainsPlayhead::PartiallyContainsPlayhead:
			return FStyleColors::White25.GetSpecifiedColor() * FStyleColors::AccentGreen.GetSpecifiedColor();

		case EItemContainsPlayhead::ContainsPlayhead:
			return FStyleColors::Foreground.GetSpecifiedColor() * FStyleColors::AccentGreen.GetSpecifiedColor();
		}
	}

	return FStyleColors::Transparent;
}

EItemContainsPlayhead SNavigationToolPlayhead::GetContainsPlayhead() const
{
	if (const TViewModelPtr<IPlayheadExtension> PlayheadItem = WeakItem.ImplicitPin())
	{
		return PlayheadItem->ContainsPlayhead();
	}
	return EItemContainsPlayhead::None;
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
