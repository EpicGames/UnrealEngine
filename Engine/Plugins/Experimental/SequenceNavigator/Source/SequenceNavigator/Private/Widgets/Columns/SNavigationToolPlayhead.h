// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Extensions/IPlayheadExtension.h"
#include "NavigationToolDefines.h"
#include "Widgets/Images/SImage.h"

namespace UE::SequenceNavigator
{

class FNavigationToolPlayheadColumn;
class INavigationToolView;
class SNavigationToolTreeRow;

/** Widget responsible for showing whether an item's range contains the playhead */
class SNavigationToolPlayhead : public SImage
{
public:
	SLATE_BEGIN_ARGS(SNavigationToolPlayhead) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs
		, const TSharedRef<FNavigationToolPlayheadColumn>& InColumn
		, const FNavigationToolViewModelPtr& InItem
		, const TSharedRef<INavigationToolView>& InView
		, const TSharedRef<SNavigationToolTreeRow>& InRowWidget);

protected:
	/** Returns whether the widget is enabled or not */
	virtual bool IsVisibilityWidgetEnabled() const { return true; }

	virtual const FSlateBrush* GetBrush() const;

	virtual FSlateColor GetForegroundColor() const override;
	
	EItemContainsPlayhead GetContainsPlayhead() const;

	TWeakPtr<FNavigationToolPlayheadColumn> WeakColumn;

	FNavigationToolViewModelWeakPtr WeakItem;

	TWeakPtr<INavigationToolView> WeakView;

	TWeakPtr<SNavigationToolTreeRow> WeakRowWidget;
};

} // namespace UE::SequenceNavigator
