// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNavigationToolRevisionControl.h"
#include "Columns/NavigationToolRevisionControlColumn.h"
#include "Extensions/IRevisionControlExtension.h"
#include "Items/INavigationToolItem.h"
#include "NavigationToolView.h"
#include "Styling/StyleColors.h"
#include "Widgets/SNavigationToolTreeRow.h"

#define LOCTEXT_NAMESPACE "SNavigationToolRevisionControl"

namespace UE::SequenceNavigator
{

using namespace Sequencer;

void SNavigationToolRevisionControl::Construct(const FArguments& InArgs
	, const TSharedRef<FNavigationToolRevisionControlColumn>& InColumn
	, const FNavigationToolViewModelPtr& InItem
	, const TSharedRef<INavigationToolView>& InView
	, const TSharedRef<SNavigationToolTreeRow>& InRowWidget)
{
	WeakColumn = InColumn;
	WeakItem = InItem;
	WeakView = InView;
	WeakRowWidget = InRowWidget;

	SetToolTipText(TAttribute<FText>::CreateSP(this, &SNavigationToolRevisionControl::GetToolTipText));

	SImage::Construct(SImage::FArguments()
		.Image(this, &SNavigationToolRevisionControl::GetBrush)
		.ColorAndOpacity(this, &SNavigationToolRevisionControl::GetForegroundColor));
}

FSlateColor SNavigationToolRevisionControl::GetForegroundColor() const
{
	const TViewModelPtr<IRevisionControlExtension> Item = WeakItem.ImplicitPin();
	if (!Item.IsValid())
	{
		return FStyleColors::Transparent;
	}

	switch (Item->GetRevisionControlState())
	{
	case EItemRevisionControlState::None:
		return FStyleColors::Transparent;
	case EItemRevisionControlState::PartiallySourceControlled:
		return FStyleColors::White25;
	case EItemRevisionControlState::SourceControlled:
		return FStyleColors::Foreground;
	}

	return FStyleColors::Transparent;
}

const FSlateBrush* SNavigationToolRevisionControl::GetBrush() const
{
	if (const TViewModelPtr<IRevisionControlExtension> Item = WeakItem.ImplicitPin())
	{
		return Item->GetRevisionControlStatusIcon();
	}
	return nullptr;
}

FText SNavigationToolRevisionControl::GetToolTipText() const
{
	if (const TViewModelPtr<IRevisionControlExtension> RevisionControlItem = WeakItem.ImplicitPin())
	{
		return RevisionControlItem->GetRevisionControlStatusText();
	}
	return FText::GetEmpty();
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
