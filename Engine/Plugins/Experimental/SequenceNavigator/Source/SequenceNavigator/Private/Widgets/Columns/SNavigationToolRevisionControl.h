// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NavigationToolDefines.h"
#include "Widgets/Images/SImage.h"

namespace UE::SequenceNavigator
{

class FNavigationToolRevisionControlColumn;
class INavigationToolView;
class IRevisionControlExtension;
class SNavigationToolTreeRow;

class SNavigationToolRevisionControl : public SImage
{
public:
	SLATE_BEGIN_ARGS(SNavigationToolRevisionControl) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs
		, const TSharedRef<FNavigationToolRevisionControlColumn>& InColumn
		, const FNavigationToolViewModelPtr& InItem
		, const TSharedRef<INavigationToolView>& InView
		, const TSharedRef<SNavigationToolTreeRow>& InRowWidget);

protected:
	virtual FSlateColor GetForegroundColor() const override;
	
	virtual const FSlateBrush* GetBrush() const;
	virtual FText GetToolTipText() const;

	TWeakPtr<FNavigationToolRevisionControlColumn> WeakColumn;

	FNavigationToolViewModelWeakPtr WeakItem;

	TWeakPtr<INavigationToolView> WeakView;

	TWeakPtr<SNavigationToolTreeRow> WeakRowWidget;
};

} // namespace UE::SequenceNavigator
