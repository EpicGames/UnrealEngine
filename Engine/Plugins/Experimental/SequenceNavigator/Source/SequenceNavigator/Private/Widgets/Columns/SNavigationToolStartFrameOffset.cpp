// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNavigationToolStartFrameOffset.h"
#include "Items/NavigationToolItemUtils.h"
#include "NavigationToolView.h"
#include "Sections/MovieSceneSubSection.h"
#include "Widgets/SNavigationToolTreeRow.h"

#define LOCTEXT_NAMESPACE "SNavigationToolStartFrameOffset"

namespace UE::SequenceNavigator
{

using namespace Sequencer;

void SNavigationToolStartFrameOffset::Construct(const FArguments& InArgs
	, const FNavigationToolViewModelPtr& InItem
	, const TSharedRef<INavigationToolView>& InView
	, const TSharedRef<SNavigationToolTreeRow>& InRowWidget)
{
	WeakItem = InItem;
	WeakView = InView;
	WeakRowWidget = InRowWidget;

	SNavigationToolTime::Construct(SNavigationToolTime::FArguments(), InItem, InView, InRowWidget);
}

double SNavigationToolStartFrameOffset::GetFrameTimeValue() const
{
	if (const UMovieSceneSubSection* const SubSection = ItemUtils::GetSequenceItemSubSection(WeakItem.Pin()))
	{
		return SubSection->Parameters.StartFrameOffset.Value;
	}
	return 0.;
}

void SNavigationToolStartFrameOffset::OnFrameTimeValueCommitted(const double InNewValue, const ETextCommit::Type InCommitType)
{
	using namespace ItemUtils;

	UMovieSceneSubSection* const SubSection = GetSequenceItemSubSection(WeakItem.Pin());
	if (!SubSection)
	{
		return;
	}

	if (InNewValue == SubSection->Parameters.StartFrameOffset.Value)
	{
		return;
	}

	const bool bShouldTransact = !UndoTransaction.IsValid() && (InCommitType == ETextCommit::OnEnter);
	const FScopedTransaction Transaction(GetTransactionText(), bShouldTransact);

	SubSection->Modify();
	SubSection->Parameters.StartFrameOffset = static_cast<int32>(InNewValue);
}

FText SNavigationToolStartFrameOffset::GetTransactionText() const
{
	return LOCTEXT("SetStartFrameOffsetTransaction", "Set Start Frame Offset");
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
