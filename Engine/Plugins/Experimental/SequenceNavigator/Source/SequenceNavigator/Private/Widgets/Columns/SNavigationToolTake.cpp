// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNavigationToolTake.h"
#include "AssetRegistry/AssetData.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ISequencer.h"
#include "Items/NavigationToolItemUtils.h"
#include "MovieSceneSequence.h"
#include "MovieSceneToolHelpers.h"
#include "NavigationTool.h"
#include "NavigationToolView.h"
#include "Styling/StyleColors.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SNavigationToolTreeRow.h"

#define LOCTEXT_NAMESPACE "SNavigationToolTake"

namespace UE::SequenceNavigator
{

void SNavigationToolTake::Construct(const FArguments& InArgs
	, const FNavigationToolViewModelPtr& InItem
	, const TSharedRef<INavigationToolView>& InView
	, const TSharedRef<SNavigationToolTreeRow>& InRowWidget)
{
	WeakItem = InItem;
	WeakView = InView;
	WeakRowWidget = InRowWidget;

	WeakTool = InView->GetOwnerTool();

	CacheTakes();

	if (!ActiveTakeInfo.IsValid())
	{
		return;
	}

	ChildSlot
	[
		SNew(SComboBox<TSharedPtr<FSequenceTakeEntry>>)
		.OptionsSource(&CachedTakes)
		.InitiallySelectedItem(ActiveTakeInfo)
		.OnSelectionChanged(this, &SNavigationToolTake::OnSelectionChanged)
		.OnGenerateWidget(this, &SNavigationToolTake::GenerateTakeWidget, /*bShowTakeIndex=*/false, /*bShowNumberedOf=*/true)
		[
			GenerateTakeWidget(ActiveTakeInfo, /*bShowTakeIndex=*/false, /*bShowNumberedOf=*/false)
		]
	];
}

TSharedRef<SWidget> SNavigationToolTake::GenerateTakeWidget(const TSharedPtr<FSequenceTakeEntry> InTakeInfo
	, const bool bShowTakeIndex, const bool bShowNumberedOf)
{
	if (InTakeInfo.IsValid())
	{
		return SNew(SNavigationToolTakeEntry, InTakeInfo.ToSharedRef())
			.TotalTakeCount_Lambda([this]()
				{
					return CachedTakes.Num();
				})
			.OnEntrySelected(this, &SNavigationToolTake::OnTakeEntrySelected)
			.ShowTakeIndex(bShowTakeIndex)
			.ShowNumberedOf(bShowNumberedOf);
	}
	return SNullWidget::NullWidget;
}

FSlateColor SNavigationToolTake::GetBorderColor() const
{
	if (IsHovered())
	{
		return FSlateColor::UseForeground();
	}
	return FSlateColor::UseSubduedForeground();
}

FReply SNavigationToolTake::OnTakeEntrySelected(const TSharedRef<FSequenceTakeEntry> InTakeInfo)
{
	if (const TSharedPtr<INavigationTool> Tool = WeakTool.Pin())
	{
		SetActiveTake(InTakeInfo);

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SNavigationToolTake::OnSelectionChanged(const TSharedPtr<FSequenceTakeEntry> InTakeInfo, const ESelectInfo::Type InSelectType)
{
	if (InTakeInfo.IsValid())
	{
		SetActiveTake(InTakeInfo);
	}
}

void SNavigationToolTake::CacheTakes()
{
	CachedTakes.Reset();

	const FNavigationToolViewModelPtr Item = WeakItem.Pin();
	if (!Item.IsValid())
	{
		return;
	}

	const Sequencer::TViewModelPtr<FNavigationToolSequence> SequenceItem = Item.ImplicitCast();
	if (!SequenceItem.IsValid())
	{
		return;
	}

	UMovieSceneSubSection* const SubSection = SequenceItem->GetSubSection();
	if (!SubSection)
	{
		return;
	}

	TArray<FAssetData> AssetData;
	uint32 CurrentTakeNumber = INDEX_NONE;
	MovieSceneToolHelpers::GatherTakes(SubSection, AssetData, CurrentTakeNumber);

	AssetData.Sort([SubSection](const FAssetData& InA, const FAssetData& InB)
		{
			uint32 TakeNumberA = INDEX_NONE;
			uint32 TakeNumberB = INDEX_NONE;
			if (MovieSceneToolHelpers::GetTakeNumber(SubSection, InA, TakeNumberA)
				&& MovieSceneToolHelpers::GetTakeNumber(SubSection, InB, TakeNumberB))
			{
				return TakeNumberA < TakeNumberB;
			}
			return true;
		});

	uint32 TakeIndex = 0;
	for (const FAssetData& ThisAssetData : AssetData)
	{
		uint32 TakeNumber = INDEX_NONE;
		if (MovieSceneToolHelpers::GetTakeNumber(SubSection, ThisAssetData, TakeNumber))
		{
			if (UMovieSceneSequence* const Sequence = Cast<UMovieSceneSequence>(ThisAssetData.GetAsset()))
			{
				const TSharedRef<FSequenceTakeEntry> NewTakeInfo = MakeShared<FSequenceTakeEntry>();
				NewTakeInfo->TakeIndex = TakeIndex;
				NewTakeInfo->TakeNumber = TakeNumber;
				NewTakeInfo->DisplayName = Sequence->GetDisplayName();
				NewTakeInfo->WeakSequence = Sequence;
				CachedTakes.Add(NewTakeInfo);

				if (TakeNumber == CurrentTakeNumber)
				{
					ActiveTakeInfo = NewTakeInfo;
				}

				++TakeIndex;
			}
		}
	}
}

void SNavigationToolTake::SetActiveTake(const TSharedPtr<FSequenceTakeEntry>& InTakeInfo)
{
	UMovieSceneSequence* const Sequence = InTakeInfo->WeakSequence.Get();
	if (!Sequence)
	{
		return;
	}

	const TSharedPtr<INavigationTool> Tool = WeakTool.Pin();
	if (!Tool.IsValid())
	{
		return;
	}

	const TSharedPtr<ISequencer> Sequencer = Tool->GetSequencer();
	if (!Sequencer.IsValid())
	{
		return;
	}

	const FNavigationToolViewModelPtr Item = WeakItem.Pin();
	if (!Item.IsValid())
	{
		return;
	}

	const Sequencer::TViewModelPtr<FNavigationToolSequence> SequenceItem = Item.ImplicitCast();
	if (!SequenceItem.IsValid())
	{
		return;
	}

	const UMovieScene* const SequenceMovieScene = SequenceItem->GetSequenceMovieScene();
	if (!SequenceMovieScene)
	{
		return;
	}

	UMovieSceneSubSection* const ParentSubSection = SequenceItem->GetSubSection();
	if (!ParentSubSection)
	{
		return;
	}

	UMovieSceneSubTrack* const ParentSubTrack = Cast<UMovieSceneSubTrack>(ParentSubSection->GetOuter());
	if (!ParentSubTrack)
	{
		return;
	}

	bool bChangedTake = false;

	FScopedTransaction Transaction(LOCTEXT("ChangeTake_Transaction", "Change Take"));

	const TRange<FFrameNumber> NewSectionRange = ParentSubSection->GetRange();
	const FFrameNumber NewSectionStartOffset = ParentSubSection->Parameters.StartFrameOffset;
	const int32 NewSectionPrerollFrames = ParentSubSection->GetPreRollFrames();
	const int32 NewRowIndex = ParentSubSection->GetRowIndex();
	const FFrameNumber NewSectionStartTime = NewSectionRange.GetLowerBound().IsClosed()
		? MovieScene::DiscreteInclusiveLower(NewSectionRange) : 0;
	const int32 NewSectionRowIndex = ParentSubSection->GetRowIndex();
	const FColor NewSectionColorTint = ParentSubSection->GetColorTint();

	const int32 Duration = (NewSectionRange.GetLowerBound().IsClosed() && NewSectionRange.GetUpperBound().IsClosed())
		? MovieScene::DiscreteSize(NewSectionRange) : 1;

	if (UMovieSceneSubSection* const NewSection = ParentSubTrack->AddSequence(Sequence, NewSectionStartTime, Duration))
	{
		ParentSubTrack->RemoveSection(*ParentSubSection);

		NewSection->SetRange(NewSectionRange);
		NewSection->Parameters.StartFrameOffset = NewSectionStartOffset;
		NewSection->Parameters.TimeScale = ParentSubSection->Parameters.TimeScale.DeepCopy(NewSection);
		NewSection->SetPreRollFrames(NewSectionPrerollFrames);
		NewSection->SetRowIndex(NewSectionRowIndex);
		NewSection->SetColorTint(NewSectionColorTint);

		// If the old shot's name is not the same as the sequence's name, assume the user had customized the shot name, so carry it over
		UMovieSceneCinematicShotSection* const ParentShotSection = Cast<UMovieSceneCinematicShotSection>(ParentSubSection);
		UMovieSceneCinematicShotSection* const NewShotSection = Cast<UMovieSceneCinematicShotSection>(NewSection);
		if (NewShotSection && ParentShotSection)
		{
			if (const UMovieSceneSequence* const ParentShotSequence = ParentShotSection->GetSequence())
            {
                const FString ShowDisplayName = ParentShotSection->GetShotDisplayName();
				if (ShowDisplayName != ParentShotSequence->GetName())
				{
					NewShotSection->SetShotDisplayName(ShowDisplayName);
				}
            }
		}

		bChangedTake = true;
	}

	if (bChangedTake)
	{
		Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
	}
	else
	{
		Transaction.Cancel();
	}
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
