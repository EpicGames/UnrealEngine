// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NavigationToolDefines.h"
#include "ScopedTransaction.h"
#include "SNavigationToolTakeEntry.h"
#include "Templates/UniquePtr.h"
#include "Widgets/SCompoundWidget.h"

namespace UE::SequenceNavigator
{

class INavigationTool;
class INavigationToolView;
class SNavigationToolTreeRow;

/** Column widget that displays a combo box dropdown list of takes for a specific sequence */
class SNavigationToolTake : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNavigationToolTake) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs
		, const FNavigationToolViewModelPtr& InItem
		, const TSharedRef<INavigationToolView>& InView
		, const TSharedRef<SNavigationToolTreeRow>& InRowWidget);

	void SetActiveTake(const TSharedPtr<FSequenceTakeEntry>& InTakeInfo);

protected:
	FSlateColor GetBorderColor() const;

	FReply OnTakeEntrySelected(const TSharedRef<FSequenceTakeEntry> InTakeInfo);

	TSharedRef<SWidget> GenerateTakeWidget(const TSharedPtr<FSequenceTakeEntry> InTakeInfo
		, const bool bShowTakeIndex, const bool bShowNumberedOf);

	void OnSelectionChanged(const TSharedPtr<FSequenceTakeEntry> InTakeInfo, const ESelectInfo::Type InSelectType);

	void CacheTakes();

	FNavigationToolViewModelWeakPtr WeakItem;

	TWeakPtr<INavigationToolView> WeakView;

	TWeakPtr<SNavigationToolTreeRow> WeakRowWidget;

	TWeakPtr<INavigationTool> WeakTool;

	TArray<TSharedPtr<FSequenceTakeEntry>> CachedTakes;

	TSharedPtr<FSequenceTakeEntry> ActiveTakeInfo;

	TUniquePtr<FScopedTransaction> UndoTransaction;
};

} // namespace UE::SequenceNavigator
