// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNavigationToolTakeEntry.h"
#include "AssetRegistry/AssetData.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Styling/StyleColors.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "SNavigationToolTakeEntry"

namespace UE::SequenceNavigator
{

class FTakeDragDropOp : public FDragDropOperation
{
public:
	DRAG_DROP_OPERATOR_TYPE(FTakeDragDropOp, FDragDropOperation)

	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override
	{
		return SNullWidget::NullWidget;
	}

	static TSharedRef<FTakeDragDropOp> New(const TSharedPtr<FSequenceTakeEntry>& InTakeInfo)
	{
		const TSharedRef<FTakeDragDropOp> Operation = MakeShared<FTakeDragDropOp>();
		Operation->TakeInfo = InTakeInfo;
		Operation->Construct();
		return Operation;
	}

	TSharedPtr<FSequenceTakeEntry> TakeInfo;
};

void SNavigationToolTakeEntry::Construct(const FArguments& InArgs, const TSharedRef<FSequenceTakeEntry>& InTakeEntry)
{
	TakeEntry = InTakeEntry;

	TotalTakeCount = InArgs._TotalTakeCount;
	OnTakeEntrySelected = InArgs._OnEntrySelected;

	ShowTakeIndex = InArgs._ShowTakeIndex;
	ShowNumberedOf = InArgs._ShowNumberedOf;

	const TSharedRef<SHorizontalBox> HorizontalBox = SNew(SHorizontalBox);

	if (ShowTakeIndex)
	{
		HorizontalBox->AddSlot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), TEXT("SmallText"))
				.ColorAndOpacity(FStyleColors::Hover)
				.Text(this, &SNavigationToolTakeEntry::GetTakeNumberText)
			];
	}

	HorizontalBox->AddSlot()
		.FillWidth(1.f)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(3.f, 0.f, 0.f, 0.f)
		[
			SNew(STextBlock)
			.Text(this, &SNavigationToolTakeEntry::GetTakeNameText)
		];

	if (ShowNumberedOf)
	{
		HorizontalBox->AddSlot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(3.f, 0.f, 0.f, 0.f)
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), TEXT("SmallText"))
				.ColorAndOpacity(FStyleColors::Hover)
				.Text(this, &SNavigationToolTakeEntry::GetTakeIndexText)
			];
	}

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(this, &SNavigationToolTakeEntry::GetBorderImage)
		.Padding(FMargin(2.f, 1.f, 3.f, 1.f))
		[
			HorizontalBox
		]
	];
}

const FSlateBrush* SNavigationToolTakeEntry::GetBorderImage() const
{
	return bHighlightForDragDrop ? FAppStyle::Get().GetBrush(TEXT("DetailsView.Highlight")) : nullptr;
}

FText SNavigationToolTakeEntry::GetTakeNumberText() const
{
	if (TakeEntry.IsValid())
	{
		return FText::Format(LOCTEXT("TakeNumberLabel", "{0}: ")
			, FText::AsNumber(TakeEntry->TakeNumber));
	}
	return FText::GetEmpty();
}

FText SNavigationToolTakeEntry::GetTakeNameText() const
{
	if (TakeEntry.IsValid())
	{
		return TakeEntry->DisplayName;
	}
	return FText::GetEmpty();
}

FText SNavigationToolTakeEntry::GetTakeIndexText() const
{
	if (TakeEntry.IsValid() && TotalTakeCount.IsBound())
	{
		return FText::Format(LOCTEXT("TakeIndexLabel", "({0}/{1})")
			, FText::AsNumber(TakeEntry->TakeIndex + 1)
			, FText::AsNumber(TotalTakeCount.Get()));
	}
	return FText::GetEmpty();
}

FReply SNavigationToolTakeEntry::OnMouseButtonDown(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	FReply Reply = SCompoundWidget::OnMouseButtonDown(InGeometry, InPointerEvent);

	if (InPointerEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		const TSharedRef<SNavigationToolTakeEntry> This = SharedThis(this);
		Reply = Reply.CaptureMouse(This).DetectDrag(This, EKeys::LeftMouseButton);
	}

	return Reply;
}

FReply SNavigationToolTakeEntry::OnMouseButtonUp(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	FReply Reply = FReply::Unhandled();

	bHighlightForDragDrop = false;

	if (IsHovered() && InPointerEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		Reply = FReply::Handled();
	}

	if (!Reply.GetMouseCaptor().IsValid() && HasMouseCapture())
	{
		Reply.ReleaseMouseCapture();
	}

	return Reply;
}

void SNavigationToolTakeEntry::OnFocusLost(const FFocusEvent& InFocusEvent)
{
	bHighlightForDragDrop = false;

	SCompoundWidget::OnFocusLost(InFocusEvent);
}

FReply SNavigationToolTakeEntry::OnDragDetected(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	if (InPointerEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		FSlateApplication::Get().DismissAllMenus();

		return FReply::Handled().BeginDragDrop(FTakeDragDropOp::New(TakeEntry));
	}

	return SCompoundWidget::OnDragDetected(InGeometry, InPointerEvent);
}

void SNavigationToolTakeEntry::OnDragEnter(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent)
{
	if (const TSharedPtr<FTakeDragDropOp> DragOp = InDragDropEvent.GetOperationAs<FTakeDragDropOp>())
	{
		bHighlightForDragDrop = true;
	}

	SCompoundWidget::OnDragEnter(InGeometry, InDragDropEvent);
}

void SNavigationToolTakeEntry::OnDragLeave(const FDragDropEvent& InDragDropEvent)
{
	if (const TSharedPtr<FTakeDragDropOp> DragOp = InDragDropEvent.GetOperationAs<FTakeDragDropOp>())
	{
		bHighlightForDragDrop = false;
	}

	SCompoundWidget::OnDragLeave(InDragDropEvent);
}

FReply SNavigationToolTakeEntry::OnDragOver(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent)
{
	if (const TSharedPtr<FTakeDragDropOp> DragOp = InDragDropEvent.GetOperationAs<FTakeDragDropOp>())
	{
		bHighlightForDragDrop = true;
	}

	return SCompoundWidget::OnDragOver(InGeometry, InDragDropEvent);
}

FReply SNavigationToolTakeEntry::OnDrop(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent)
{
	if (const TSharedPtr<FTakeDragDropOp> DragOp = InDragDropEvent.GetOperationAs<FTakeDragDropOp>())
	{
		if (OnTakeEntrySelected.IsBound())
		{
			return OnTakeEntrySelected.Execute(DragOp->TakeInfo.ToSharedRef());
		}
	}

	bHighlightForDragDrop = false;

	return SCompoundWidget::OnDrop(InGeometry, InDragDropEvent);
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
