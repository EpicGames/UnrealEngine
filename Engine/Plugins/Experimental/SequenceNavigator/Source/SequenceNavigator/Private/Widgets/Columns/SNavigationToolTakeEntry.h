// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class UMovieSceneSequence;

namespace UE::SequenceNavigator
{

/** Stores information about a sequence take */
struct FSequenceTakeEntry
{
	/** Sequence that the take belongs to */
	TWeakObjectPtr<UMovieSceneSequence> WeakSequence;

	/** Index that the take is serialized to that will change depending on added/removed entries */
	uint32 TakeIndex = 0;

	/** Take number that doesn't change after being serialized */
	uint32 TakeNumber = 0;

	FText DisplayName;
};

/** Widget that display information about a single sequence take */
class SNavigationToolTakeEntry : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_RetVal_OneParam(FReply, FOnTakeEntrySelected, const TSharedRef<FSequenceTakeEntry> /*InTakeInfo*/);

	SLATE_BEGIN_ARGS(SNavigationToolTakeEntry) {}
		SLATE_ATTRIBUTE(int32, TotalTakeCount)
		/** Event called when a new take is selected */
		SLATE_EVENT(FOnTakeEntrySelected, OnEntrySelected)
		/** Shows the actual take index in the array in front of the take name */
		SLATE_ARGUMENT(bool, ShowTakeIndex)
		/** Shows numbered "1 / 2" text */
		SLATE_ARGUMENT(bool, ShowNumberedOf)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FSequenceTakeEntry>& InTakeEntry);

	//~ Begin SWidget

	virtual FReply OnMouseButtonDown(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) override;
	virtual void OnFocusLost(const FFocusEvent& InFocusEvent) override;

	virtual FReply OnDragDetected(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) override;
	virtual void OnDragEnter(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent) override;
	virtual void OnDragLeave(const FDragDropEvent& InDragDropEvent) override;
	virtual FReply OnDragOver(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent) override;
	virtual FReply OnDrop(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent) override;

	//~ End SWidget

protected:
	const FSlateBrush* GetBorderImage() const;

	FText GetTakeNumberText() const;
	FText GetTakeNameText() const;
	FText GetTakeIndexText() const;

	TSharedPtr<FSequenceTakeEntry> TakeEntry;

	TAttribute<int32> TotalTakeCount;
	FOnTakeEntrySelected OnTakeEntrySelected;

	bool ShowTakeIndex = false;
	bool ShowNumberedOf = false;

	/** Highlight effect to display which entry is dragged over */
	bool bHighlightForDragDrop = false;
};

} // namespace UE::SequenceNavigator
