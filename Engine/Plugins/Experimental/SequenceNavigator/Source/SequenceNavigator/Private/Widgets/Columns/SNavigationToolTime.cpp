// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNavigationToolTime.h"
#include "ISequencer.h"
#include "Items/NavigationToolItemUtils.h"
#include "NavigationToolStyle.h"
#include "NavigationToolView.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/SNavigationToolTreeRow.h"

#define LOCTEXT_NAMESPACE "SNavigationToolTime"

namespace UE::SequenceNavigator
{

using namespace Sequencer;

void SNavigationToolTime::Construct(const FArguments& InArgs
	, const FNavigationToolViewModelPtr& InItem
	, const TSharedRef<INavigationToolView>& InView
	, const TSharedRef<SNavigationToolTreeRow>& InRowWidget)
{
	check(InItem.IsValid());
	WeakItem = InItem;
	WeakView = InView;
	WeakRowWidget = InRowWidget;

	ChildSlot
	[
		SNew(SSpinBox<double>)
		.Justification(ETextJustify::Center)
		.Style(&FNavigationToolStyle::Get().GetWidgetStyle<FSpinBoxStyle>(GetStyleName()))
		.TypeInterface(GetNumericTypeInterface())
		.Value(this, &SNavigationToolTime::GetFrameTimeValue)
		.OnValueChanged(this, &SNavigationToolTime::OnFrameTimeValueChanged)
		.OnValueCommitted(this, &SNavigationToolTime::OnFrameTimeValueCommitted)
		.OnBeginSliderMovement(this, &SNavigationToolTime::OnBeginSliderMovement)
		.OnEndSliderMovement(this, &SNavigationToolTime::OnEndSliderMovement)
		.Delta(this, &SNavigationToolTime::GetDisplayRateDeltaFrameCount)
	];
}

bool SNavigationToolTime::IsReadOnly() const
{
	return bIsReadOnly;
}

void SNavigationToolTime::SetIsReadOnly(const bool bInIsReadOnly)
{
	bIsReadOnly = bInIsReadOnly;
}

FName SNavigationToolTime::GetStyleName() const
{
	return TEXT("SpinBox");
}

void SNavigationToolTime::OnFrameTimeValueChanged(const double InNewValue)
{
	if (!bIsReadOnly)
	{
		OnFrameTimeValueCommitted(InNewValue, ETextCommit::Default);
	}
}

void SNavigationToolTime::OnFrameTimeValueCommitted(const double InNewValue, const ETextCommit::Type InCommitType)
{
}

void SNavigationToolTime::OnBeginSliderMovement()
{
	if (!bIsReadOnly && !UndoTransaction.IsValid())
	{
		UndoTransaction = MakeUnique<FScopedTransaction>(GetTransactionText());
	}
}

void SNavigationToolTime::OnEndSliderMovement(const double InNewValue)
{
	UndoTransaction.Reset();
}

TSharedPtr<INumericTypeInterface<double>> SNavigationToolTime::GetNumericTypeInterface() const
{
	if (const TSharedPtr<ISequencer> Sequencer = GetSequencer())
	{
		return Sequencer->GetNumericTypeInterface(ENumericIntent::Position);
	}
	return nullptr;
}

double SNavigationToolTime::GetDisplayRateDeltaFrameCount() const
{
	if (const TSharedPtr<ISequencer> Sequencer = GetSequencer())
	{
		return Sequencer->GetFocusedTickResolution().AsDecimal() * Sequencer->GetFocusedDisplayRate().AsInterval();
	}
	return 0.;
}

TSharedPtr<ISequencer> SNavigationToolTime::GetSequencer() const
{
	if (const TSharedPtr<INavigationToolView> View = WeakView.Pin())
	{
		return View->GetSequencer();
	}
	return nullptr;
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
