// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Items/NavigationToolSequence.h"
#include "NavigationToolDefines.h"
#include "ScopedTransaction.h"
#include "Templates/UniquePtr.h"
#include "Widgets/Input/NumericTypeInterface.h"
#include "Widgets/SCompoundWidget.h"

class FName;
class FText;
class ISequencer;

namespace UE::SequenceNavigator
{

//class FNavigationToolSequence;
class INavigationToolView;
class SNavigationToolTreeRow;

/**
 * Base time column widget.
 * Subclass this widget to implement a time-based property column widget.
 */
class SNavigationToolTime : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNavigationToolTime) {}
	SLATE_END_ARGS()

	virtual ~SNavigationToolTime() override = default;

	void Construct(const FArguments& InArgs
		, const FNavigationToolViewModelPtr& InItem
		, const TSharedRef<INavigationToolView>& InView
		, const TSharedRef<SNavigationToolTreeRow>& InRowWidget);

	virtual bool IsReadOnly() const;
	virtual void SetIsReadOnly(const bool bInIsReadOnly);

protected:
	virtual FName GetStyleName() const;

	//~ Begin SNavigationToolTime

	virtual double GetFrameTimeValue() const = 0;

	virtual void OnFrameTimeValueChanged(const double InNewValue);
	virtual void OnFrameTimeValueCommitted(const double InNewValue, const ETextCommit::Type InCommitType);

	virtual void OnBeginSliderMovement();
	virtual void OnEndSliderMovement(const double InNewValue);

	virtual TSharedPtr<INumericTypeInterface<double>> GetNumericTypeInterface() const;

	virtual double GetDisplayRateDeltaFrameCount() const;

	virtual FText GetTransactionText() const = 0;

	//~ End SNavigationToolTime

	TSharedPtr<ISequencer> GetSequencer() const;

	FNavigationToolViewModelWeakPtr WeakItem;
	TWeakPtr<INavigationToolView> WeakView;
	TWeakPtr<SNavigationToolTreeRow> WeakRowWidget;

	TUniquePtr<FScopedTransaction> UndoTransaction;

	bool bIsReadOnly = false;
};

} // namespace UE::SequenceNavigator
