// Copyright Epic Games, Inc. All Rights Reserved.

#include "ModalTextInputDialog.h"
#include "Framework/Application/SlateApplication.h"
#include "Interfaces/IMainFrameModule.h"
#include "SPrimaryButton.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ModalTextInputDialog"

namespace UE::SequenceNavigator
{

class SModalTextInputDialog : public SWindow
{
public:
	DECLARE_DELEGATE_OneParam(FOnDialogAccepted, const FText& /*InText*/);
	
	static void ShowWindow(const TSharedRef<SWindow>& InWindowToShow
		, const bool bInModal
		, const TSharedPtr<SWindow>& InParentWindow = nullptr);

	SLATE_BEGIN_ARGS(SModalTextInputDialog)
	{}
		SLATE_ARGUMENT(FText, DialogTitle);
		SLATE_ARGUMENT(FText, InputLabel);
		SLATE_ARGUMENT(FText, DefaultText)
		SLATE_EVENT(FSimpleDelegate, OnAccept)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	FText GetText() const;

	bool WasLastAccepted() const
	{
		return bWasLastAccepted;
	}

private:
	FReply OnAcceptClick();
	FReply OnCancelClick();

	TAttribute<FText> TextAttribute;

	FSimpleDelegate OnAccept;

	bool bWasLastAccepted = false;
};

void SModalTextInputDialog::Construct(const FArguments& InArgs)
{
	TextAttribute.Set(InArgs._DefaultText);

	OnAccept = InArgs._OnAccept;

	constexpr float Padding = 30.f;
	constexpr float HalfPadding = Padding * 0.5f;

	SWindow::Construct(SWindow::FArguments()
		.Title(InArgs._DialogTitle)
		.HasCloseButton(true)
		.SupportsMaximize(false)
		.SupportsMinimize(false)
		.SizingRule(ESizingRule::FixedSize)
		.ClientSize(FVector2D(680, 100))
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush(TEXT("Brushes.Panel")))
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.VAlign(VAlign_Center)
				.Padding(Padding, 20, Padding, HalfPadding)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(0, 0, 0, 0)
					[
						SNew(STextBlock)
						.Text(InArgs._InputLabel)
					]
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding(20, 0, 0, 0)
					[
						SNew(SEditableTextBox)
						.Text(TextAttribute)
						.OnTextChanged_Lambda([this](const FText& InText)
							{
								TextAttribute.Set(InText);
							})
					]
				]
				+ SVerticalBox::Slot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Bottom)
				.Padding(0, 0, Padding, HalfPadding)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Right)
					[
						SNew(SPrimaryButton)
						.Text(LOCTEXT("AcceptButton", "Accept"))
						.OnClicked(this, &SModalTextInputDialog::OnAcceptClick)
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(HalfPadding, 0, 0, 0)
					.HAlign(HAlign_Right)
					[
						SNew(SButton)
						.Text(LOCTEXT("CancelButton", "Cancel"))
						.OnClicked(this, &SModalTextInputDialog::OnCancelClick)
					]
				]
			]
		]);
}

void SModalTextInputDialog::ShowWindow(const TSharedRef<SWindow>& InWindowToShow
	, const bool bInModal
	, const TSharedPtr<SWindow>& InParentWindow)
{
	if (bInModal)
	{
		TSharedPtr<SWidget> ParentWidget = InParentWindow;

		if (!ParentWidget.IsValid() && FModuleManager::Get().IsModuleLoaded(TEXT("MainFrame")))
		{
			IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
			ParentWidget = MainFrameModule.GetParentWindow();
		}

		FSlateApplication::Get().AddModalWindow(InWindowToShow, ParentWidget);
	}
	else
	{
		FSlateApplication::Get().AddWindow(InWindowToShow);

		if (InParentWindow.IsValid())
		{
			FSlateApplication::Get().AddWindowAsNativeChild(InWindowToShow, InParentWindow.ToSharedRef());
		}
		else
		{
			FSlateApplication::Get().AddWindow(InWindowToShow);
		}
	}
}

FText SModalTextInputDialog::GetText() const
{
	return TextAttribute.Get();
}

FReply SModalTextInputDialog::OnAcceptClick()
{
	RequestDestroyWindow();

	bWasLastAccepted = true;
	OnAccept.ExecuteIfBound();

	return FReply::Handled();
}

FReply SModalTextInputDialog::OnCancelClick()
{
	RequestDestroyWindow();

	return FReply::Handled();
}

FModalTextInputDialog::FModalTextInputDialog()
{
	DialogTitle = LOCTEXT("DefaultDialogTitle", "Text Input");
	InputLabel = FText::GetEmpty();
}

bool FModalTextInputDialog::Open(const FText& InDefaultText, FText& OutText, const TSharedPtr<SWindow>& InParentWindow)
{
	if (WindowWidget.IsValid())
	{
		WindowWidget->BringToFront();
		return false;
	}

	const TSharedRef<SModalTextInputDialog> DialogWidget = SNew(SModalTextInputDialog)
		.DialogTitle(DialogTitle)
		.InputLabel(InputLabel)
		.DefaultText(InDefaultText);
	WindowWidget = DialogWidget;

	WindowWidget->SetOnWindowClosed(FOnWindowClosed::CreateLambda([this](const TSharedRef<SWindow>& InWindow)
		{
			WindowWidget.Reset();
		}));

	SModalTextInputDialog::ShowWindow(WindowWidget.ToSharedRef(), true, InParentWindow);

	if (DialogWidget->WasLastAccepted())
	{
		OutText = DialogWidget->GetText();
		return true;
	}

	return false;
}

bool FModalTextInputDialog::IsOpen()
{
	return WindowWidget.IsValid();
}

void FModalTextInputDialog::Close()
{
	if (WindowWidget.IsValid())
	{
		WindowWidget->RequestDestroyWindow();
		WindowWidget.Reset();
	}
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
