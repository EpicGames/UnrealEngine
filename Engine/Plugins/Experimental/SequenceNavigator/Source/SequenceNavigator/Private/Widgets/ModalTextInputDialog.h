// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"

class SWindow;

namespace UE::SequenceNavigator
{

class FModalTextInputDialog
{
public:
	FModalTextInputDialog();

	/** @return True if the text input was accepted. False if cancel was clicked. */
	bool Open(const FText& InDefaultText, FText& OutText, const TSharedPtr<SWindow>& InParentWindow = nullptr);

	bool IsOpen();

	void Close();

	FText DialogTitle;
	FText InputLabel;

protected:
	TSharedPtr<SWindow> WindowWidget;
};

} // namespace UE::SequenceNavigator
