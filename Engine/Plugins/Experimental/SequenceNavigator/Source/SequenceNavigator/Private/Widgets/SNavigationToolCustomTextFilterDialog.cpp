// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNavigationToolCustomTextFilterDialog.h"
#include "Framework/Application/SlateApplication.h"
#include "Interfaces/IMainFrameModule.h"
#include "SPrimaryButton.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SNavigationToolCustomTextFilterDialog"

namespace UE::SequenceNavigator
{

TSharedPtr<SNavigationToolCustomTextFilterDialog> SNavigationToolCustomTextFilterDialog::DialogInstance;

bool SNavigationToolCustomTextFilterDialog::IsOpen()
{
	return DialogInstance.IsValid();
}

void SNavigationToolCustomTextFilterDialog::CloseWindow()
{
	if (DialogInstance.IsValid())
	{
		DialogInstance->RequestDestroyWindow();
		DialogInstance.Reset();
	}
}

void SNavigationToolCustomTextFilterDialog::ShowWindow(const TSharedRef<SWindow>& InWindowToShow, const bool bInModal, const TSharedPtr<SWindow>& InParentWindow)
{
	if (bInModal)
	{
		TSharedPtr<SWidget> ParentWidget = InParentWindow;

		if (!ParentWidget.IsValid() && FModuleManager::Get().IsModuleLoaded(TEXT("MainFrame")))
		{
			IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
			ParentWidget = MainFrameModule.GetParentWindow();
		}

		FSlateApplication::Get().AddModalWindow(InWindowToShow, ParentWidget);
	}
	else
	{
		FSlateApplication::Get().AddWindow(InWindowToShow);

		if (InParentWindow.IsValid())
		{
			FSlateApplication::Get().AddWindowAsNativeChild(InWindowToShow, InParentWindow.ToSharedRef());
		}
		else
		{
			FSlateApplication::Get().AddWindow(InWindowToShow);
		}
	}
}

void SNavigationToolCustomTextFilterDialog::Construct(const FArguments& InArgs)
{
	CustomTextFilterData = InArgs._CustomTextFilterData;
	OnTryCreateFilter = InArgs._OnTryCreateFilter;
	OnTryModifyFilter = InArgs._OnTryModifyFilter;
	OnTryDeleteFilter = InArgs._OnTryDeleteFilter;

	InitialCustomTextFilterData = CustomTextFilterData;

	const FText WindowTitle = IsEdit()
		? LOCTEXT("ModifyCustomTextFilterWindow", "Modify Custom Filter")
		: LOCTEXT("CreateCustomTextFilterWindow", "Create Custom Filter");

	SWindow::Construct(SWindow::FArguments()
		.Title(WindowTitle)
		.HasCloseButton(true)
		.SupportsMaximize(false)
		.SupportsMinimize(false)
		.SizingRule(ESizingRule::Autosized)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush(TEXT("Brushes.Panel")))
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.Padding(20.f, 40.f, 20.f, 0)
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						ConstructFilterLabelRow()
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						ConstructFilterColorRow()
					]
				]
				+ SVerticalBox::Slot()
				.Padding(20.f, 20.f, 20.f, 0)
				.AutoHeight()
				[
					ConstructFilterStringRow()
				]
				+ SVerticalBox::Slot()
				.Padding(20.f, 40.f, 20.f, 20.f)
				.AutoHeight()
				[
					ConstructButtonRow()
				]
			]
		]);

	GetOnWindowClosedEvent().AddSP(this, &SNavigationToolCustomTextFilterDialog::HandleWindowClosed);
	//OnWindowClosed.BindSP(this, &SNavigationToolCustomTextFilterDialog::HandleWindowClosed);
}

TSharedRef<SWidget> SNavigationToolCustomTextFilterDialog::ConstructFilterLabelRow()
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SBox)
			.WidthOverride(120)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("FilterLabelText", "Filter Label"))
			]
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SBox)
			.WidthOverride(254)
			[
				SAssignNew(FilterLabelTextBox, SEditableTextBox)
				.Text_Lambda([this]()
					{
						return CustomTextFilterData.FilterLabel;
					})
				.OnTextChanged_Lambda([this](const FText& InText)
					{
						CustomTextFilterData.FilterLabel = InText;
					})
			]
		];
}

TSharedRef<SWidget> SNavigationToolCustomTextFilterDialog::ConstructFilterColorRow()
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(40, 0, 0, 0)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("FilterColorText", "Color"))
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(20, 0, 0, 0)
		[
			SNew(SColorBlock)
			.Color_Lambda([this]()
				{
					return CustomTextFilterData.FilterColor;
				})
			.CornerRadius(FVector4(4.f, 4.f, 4.f, 4.f))
			.Size(FVector2D(70.f, 22.f))
			.AlphaDisplayMode(EColorBlockAlphaDisplayMode::Ignore)
			.OnMouseButtonDown(this, &SNavigationToolCustomTextFilterDialog::OnColorBlockMouseButtonDown)
		];
}

TSharedRef<SWidget> SNavigationToolCustomTextFilterDialog::ConstructFilterStringRow()
{
	const TSharedRef<SEditableTextBox> TextStringTextBox = SNew(SEditableTextBox)
		.Text_Lambda([this]()
			{
				return CustomTextFilterData.FilterString;
			})
		.OnTextChanged_Lambda([this](const FText& InText)
			{
				CustomTextFilterData.FilterString = InText;
			});

	return ConstructContentRow(LOCTEXT("TextFilterString", "Text Filter String"), TextStringTextBox);
}

TSharedRef<SWidget> SNavigationToolCustomTextFilterDialog::ConstructContentRow(const FText& InLabel, const TSharedRef<SWidget>& InContentWidget)
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(SBox)
			.WidthOverride(120.f)
			[
				SNew(STextBlock)
				.Text(InLabel)
			]
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SBox)
			.WidthOverride(560.f)
			[
				InContentWidget
			]
		];
}

TSharedRef<SWidget> SNavigationToolCustomTextFilterDialog::ConstructButtonRow()
{
	const TSharedRef<SHorizontalBox> ButtonBox = SNew(SHorizontalBox);

	if (IsEdit())
	{
		ButtonBox->AddSlot()
		.FillWidth(1.f)
		.HAlign(HAlign_Right)
		.Padding(0.f, 0.f, 16.f, 0.f)
		[
			SNew(SPrimaryButton)
			.Text(LOCTEXT("ModifyFilterButton", "Save"))
			.OnClicked(this, &SNavigationToolCustomTextFilterDialog::OnSaveButtonClick)
		];

		ButtonBox->AddSlot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		.Padding(0.f, 0.f, 16.f, 0.f)
		[
			SNew(SButton)
			.Text(LOCTEXT("DeleteButton", "Delete"))
			.OnClicked(this, &SNavigationToolCustomTextFilterDialog::OnDeleteButtonClick)
		];
	}
	else
	{
		ButtonBox->AddSlot()
		.FillWidth(1.f)
		.HAlign(HAlign_Right)
		.Padding(0.f, 0.f, 16.f, 0.f)
		[
			SNew(SButton)
			.Text(LOCTEXT("CreateAndApplyButton", "Create and Apply"))
			.OnClicked(this, &SNavigationToolCustomTextFilterDialog::OnCreateButtonClick, true)
		];

		ButtonBox->AddSlot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		.Padding(0.f, 0.f, 16.f, 0.f)
		[
			SNew(SPrimaryButton)
			.Text(LOCTEXT("CreateButton", "Create"))
			.OnClicked(this, &SNavigationToolCustomTextFilterDialog::OnCreateButtonClick, false)
		];
	}

	// Button to close the dialog box, common to both modes
	ButtonBox->AddSlot()
	.AutoWidth()
	.HAlign(HAlign_Right)
	[
		SNew(SButton)
		.Text(LOCTEXT("CancelButton", "Cancel"))
		.OnClicked(this, &SNavigationToolCustomTextFilterDialog::OnCancelButtonClick)
	];

	return ButtonBox;
}

FReply SNavigationToolCustomTextFilterDialog::OnColorBlockMouseButtonDown(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	if (InPointerEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		const FOnLinearColorValueChanged ColorValueChangedDelegate = FOnLinearColorValueChanged::CreateLambda([this](const FLinearColor& InNewColor)
			{
				CustomTextFilterData.FilterColor = InNewColor;
			});

		FColorPickerArgs PickerArgs = FColorPickerArgs(CustomTextFilterData.FilterColor, ColorValueChangedDelegate);
		PickerArgs.bIsModal = true;
		PickerArgs.ParentWidget = SharedThis(this);

		OpenColorPicker(PickerArgs);

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

bool SNavigationToolCustomTextFilterDialog::IsEdit() const
{
	return OnTryModifyFilter.IsBound() || OnTryDeleteFilter.IsBound();
}

FReply SNavigationToolCustomTextFilterDialog::OnCreateButtonClick(const bool bInApply)
{
	if (OnTryCreateFilter.IsBound())
	{
		FText ErrorText;
		if (OnTryCreateFilter.Execute(CustomTextFilterData, InitialCustomTextFilterData.FilterLabel.ToString(), bInApply, ErrorText))
		{
			RequestDestroyWindow();
		}
		else
		{
			FilterLabelTextBox->SetError(ErrorText);
		}
	}

	return FReply::Handled();
}

FReply SNavigationToolCustomTextFilterDialog::OnSaveButtonClick()
{
	if (OnTryModifyFilter.IsBound())
	{
		FText ErrorText;
		if (OnTryModifyFilter.Execute(CustomTextFilterData, InitialCustomTextFilterData.FilterLabel.ToString(), ErrorText))
		{
			RequestDestroyWindow();
		}
		else
		{
			FilterLabelTextBox->SetError(ErrorText);
		}
	}

	return FReply::Handled();
}

FReply SNavigationToolCustomTextFilterDialog::OnDeleteButtonClick()
{
	if (OnTryDeleteFilter.IsBound())
	{
		const FString FilterName = CustomTextFilterData.FilterLabel.ToString();

		FText ErrorText;
		if (OnTryDeleteFilter.Execute(FilterName, ErrorText))
		{
			RequestDestroyWindow();
		}
		else
		{
			FilterLabelTextBox->SetError(ErrorText);
		}
	}

	return FReply::Handled();
}

FReply SNavigationToolCustomTextFilterDialog::OnCancelButtonClick()
{
	RequestDestroyWindow();

	return FReply::Handled();
}

void SNavigationToolCustomTextFilterDialog::HandleWindowClosed(const TSharedRef<SWindow>& InWindow)
{
	DialogInstance.Reset();
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
