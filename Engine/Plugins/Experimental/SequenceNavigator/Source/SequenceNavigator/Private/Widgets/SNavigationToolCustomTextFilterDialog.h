// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/CustomTextFilters.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/SWindow.h"

namespace UE::SequenceNavigator
{

DECLARE_DELEGATE_RetVal_FourParams(bool, FOnNavigationToolTryCreateFilter
	, const FCustomTextFilterData& /*InNewFilterData*/, const FString& /*InOldFilterName*/, const bool /*bInApply*/, FText& /*OutErrorText*/);

DECLARE_DELEGATE_RetVal_ThreeParams(bool, FOnNavigationToolTryModifyFilter
	, const FCustomTextFilterData& /*InNewFilterData*/, const FString& /*InOldFilterName*/, FText& /*OutErrorText*/);

DECLARE_DELEGATE_RetVal_TwoParams(bool, FOnNavigationToolTryDeleteFilter
	, const FString& /*InFilterName*/, FText& /*OutErrorText*/);

class SNavigationToolCustomTextFilterDialog : public SWindow
{
public:
	friend class FNavigationToolFilterBar;

	static bool IsOpen();
	static void CloseWindow();

	SLATE_BEGIN_ARGS(SNavigationToolCustomTextFilterDialog)
	{}
		SLATE_ARGUMENT(FCustomTextFilterData, CustomTextFilterData)
		SLATE_EVENT(FOnNavigationToolTryCreateFilter, OnTryCreateFilter)
		SLATE_EVENT(FOnNavigationToolTryModifyFilter, OnTryModifyFilter)
		SLATE_EVENT(FOnNavigationToolTryDeleteFilter, OnTryDeleteFilter)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

protected:
	static void ShowWindow(const TSharedRef<SWindow>& InWindowToShow, const bool bInModal, const TSharedPtr<SWindow>& InParentWindow = nullptr);

	/** Only allow one instance of this dialog at a time. This class is private so we will store the instance here. */
	static TSharedPtr<SNavigationToolCustomTextFilterDialog> DialogInstance;

	TSharedRef<SWidget> ConstructFilterLabelRow();
	TSharedRef<SWidget> ConstructFilterColorRow();
	TSharedRef<SWidget> ConstructFilterStringRow();
	TSharedRef<SWidget> ConstructContentRow(const FText& InLabel, const TSharedRef<SWidget>& InContentWidget);
	TSharedRef<SWidget> ConstructButtonRow();

	FReply OnColorBlockMouseButtonDown(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent);

	bool IsEdit() const;

	FReply OnCreateButtonClick(const bool bInApply);
	FReply OnSaveButtonClick();
	FReply OnDeleteButtonClick();
	FReply OnCancelButtonClick();

	void HandleWindowClosed(const TSharedRef<SWindow>& InWindow);

	FOnNavigationToolTryCreateFilter OnTryCreateFilter;
	FOnNavigationToolTryModifyFilter OnTryModifyFilter;
	FOnNavigationToolTryDeleteFilter OnTryDeleteFilter;

	TSharedPtr<SEditableTextBox> FilterLabelTextBox;

	FCustomTextFilterData InitialCustomTextFilterData;
	FCustomTextFilterData CustomTextFilterData;
};

} // namespace UE::SequenceNavigator
