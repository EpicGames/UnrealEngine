// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNavigationToolExpanderArrow.h"
#include "Widgets/Views/ITableRow.h"

namespace UE::SequenceNavigator
{

void SNavigationToolExpanderArrow::Construct(const FArguments& InArgs, const TSharedPtr<ITableRow>& InTableRow)
{
	WireTint = InArgs._WireTint;
	SExpanderArrow::Construct(InArgs._ExpanderArrowArgs, InTableRow);
}

int32 SNavigationToolExpanderArrow::OnPaint(const FPaintArgs& InArgs
	, const FGeometry& InAllottedGeometry
	, const FSlateRect& InCullingRect
	, FSlateWindowElementList& OutDrawElements
	, int32 InLayerId
	, const FWidgetStyle& InWidgetStyle
	, const bool bInParentEnabled) const
{
	static constexpr float WireThickness = 2.f;
	static constexpr float HalfWireThickness = WireThickness * 0.5f;
	static constexpr float LineIndent = 5.f;

	// We want to support drawing wires for the tree
	//                 Needs Wire Array
	//   v-[A]         {}
	//   |-v[B]        {1}
	//   | '-v[B]      {1,1}
	//   |   |--[C]    {1,0,1}
	//   |   |--[D]    {1,0,1}
	//   |   '--[E]    {1,0,1}
	//   |>-[F]        {}
	//   '--[G]        {}
	//
	//

	const FSlateBrush* const VerticalBarBrush = StyleSet ? StyleSet->GetBrush(TEXT("WhiteBrush")) : nullptr;

	if (VerticalBarBrush && ShouldDrawWires.Get())
	{
		const FLinearColor WireColor = WireTint.Get(FLinearColor(0.1f, 0.1f, 0.1f, 0.25f));
		const float Indent = GetIndentAmount();

		const TSharedPtr<ITableRow> OwnerRow = OwnerRowPtr.Pin();
		check(OwnerRow.IsValid());

		const bool bIsRootNode = OwnerRow->GetIndentLevel() == 0
			&& OwnerRow->GetIndexInList() == 0
			&& OwnerRow->IsLastChild();

		// Draw vertical wires to indicate paths to parent nodes
		const TBitArray<>& NeedsWireByLevel = OwnerRow->GetWiresNeededByDepth();
		const int32 NumLevels = NeedsWireByLevel.Num();

		if (!bIsRootNode)
		{
			for (int32 Level = 0; Level < NumLevels; ++Level)
			{
				if (NeedsWireByLevel[Level])
				{
					const float CurrentIndent = (Indent * Level);

					const FVector2D WireSize(WireThickness, InAllottedGeometry.Size.Y);
					const FVector2D WireTranslation(CurrentIndent - LineIndent, 0);

					FSlateDrawElement::MakeBox(OutDrawElements
						, InLayerId
						, InAllottedGeometry.ToPaintGeometry(WireSize, FSlateLayoutTransform(WireTranslation))
						, VerticalBarBrush
						, ESlateDrawEffect::None
						, WireColor);
				}
			}
		}

		const float HalfCellHeight = 0.5f * InAllottedGeometry.Size.Y;

		// For items that are the last expanded child in a list, we need to draw a special half-height vertical connector wire
		if (OwnerRow->IsLastChild() && !bIsRootNode)
		{
			const float CurrentIndent = Indent * (NumLevels - 1);

			const FVector2D WireSize(WireThickness, HalfCellHeight + HalfWireThickness);
			const FVector2D WireTranslation(CurrentIndent - LineIndent, 0);

			FSlateDrawElement::MakeBox(OutDrawElements
				, InLayerId
				, InAllottedGeometry.ToPaintGeometry(WireSize, FSlateLayoutTransform(WireTranslation))
				, VerticalBarBrush
				, ESlateDrawEffect::None
				, WireColor);
		}

		// If this item is expanded, we need to draw a line 1/2-height to its first child cell
		if (OwnerRow->IsItemExpanded() && OwnerRow->DoesItemHaveChildren())
		{
			const float CurrentIndent = Indent * NumLevels;

			const FVector2D WireSize(WireThickness, HalfCellHeight + HalfWireThickness);
			const FVector2D WireTranslation(CurrentIndent - LineIndent, HalfCellHeight - HalfWireThickness);
			
			FSlateDrawElement::MakeBox(OutDrawElements
				, InLayerId
				, InAllottedGeometry.ToPaintGeometry(WireSize, FSlateLayoutTransform(WireTranslation))
				, VerticalBarBrush
				, ESlateDrawEffect::None
				, WireColor);
		}

		// Draw horizontal connector from parent wire to child
		if (!bIsRootNode)
		{
			const float LeafDepth = OwnerRow->DoesItemHaveChildren() ? 10.f : 0.0f;
			const float HorizontalWireStart = (NumLevels - 1) * Indent;

			const FVector2D WireSize(InAllottedGeometry.Size.X - HorizontalWireStart - WireThickness - LeafDepth
				, WireThickness);
			const FVector2D WireTranslation(HorizontalWireStart + WireThickness - LineIndent
				, 0.5f * (InAllottedGeometry.Size.Y - WireThickness));

			FSlateDrawElement::MakeBox(OutDrawElements
				, InLayerId
				, InAllottedGeometry.ToPaintGeometry(WireSize, FSlateLayoutTransform(WireTranslation))
				, VerticalBarBrush
				, ESlateDrawEffect::None
				, WireColor);
		}
	}

	InLayerId = SCompoundWidget::OnPaint(InArgs, InAllottedGeometry, InCullingRect, OutDrawElements, InLayerId, InWidgetStyle, bInParentEnabled);
	return InLayerId;
}

} // namespace UE::SequenceNavigator
