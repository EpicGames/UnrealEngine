// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/SExpanderArrow.h"

class ITableRow;

namespace UE::SequenceNavigator
{

class SNavigationToolExpanderArrow : public SExpanderArrow
{
public:
	SLATE_BEGIN_ARGS(SNavigationToolExpanderArrow) {}
		SLATE_ARGUMENT(SExpanderArrow::FArguments, ExpanderArrowArgs)
		SLATE_ATTRIBUTE(FLinearColor, WireTint)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedPtr<ITableRow>& InTableRow);

protected:
	//~ Begin SWidget
	virtual int32 OnPaint(const FPaintArgs& InArgs
		, const FGeometry& InAllottedGeometry
		, const FSlateRect& InCullingRect
		, FSlateWindowElementList& OutDrawElements
		, int32 InLayerId
		, const FWidgetStyle& InWidgetStyle
		, const bool bInParentEnabled) const override;
	//~ End SWidget

	TAttribute<FLinearColor> WireTint;
};

} // namespace UE::SequenceNavigator
