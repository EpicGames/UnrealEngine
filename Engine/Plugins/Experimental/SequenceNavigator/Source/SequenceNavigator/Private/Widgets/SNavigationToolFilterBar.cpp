// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNavigationToolFilterBar.h"
#include "Filters/Filters/NavigationToolFilter_CustomText.h"
#include "Filters/Filters/NavigationToolFilterBase.h"
#include "Filters/NavigationToolFilterBar.h"
#include "Filters/Widgets/SFilterBarClippingHorizontalBox.h"
#include "Filters/Widgets/SFilterExpressionHelpDialog.h"
#include "Filters/Widgets/SSequencerFilter.h"
#include "Filters/Widgets/SSequencerSearchBox.h"
#include "Menus/NavigationToolFilterBarContextMenu.h"
#include "Menus/NavigationToolFilterContextMenu.h"
#include "MovieScene.h"
#include "NavigationToolSettings.h"
#include "SequenceNavigatorLog.h"
#include "Widgets/SNavigationToolCustomTextFilterDialog.h"
#include "Widgets/SWindow.h"

#define LOCTEXT_NAMESPACE "SNavigationToolFilterBar"

namespace UE::SequenceNavigator
{

using namespace Sequencer;

SNavigationToolFilterBar::~SNavigationToolFilterBar()
{
	if (const TSharedPtr<FNavigationToolFilterBar> FilterBar = WeakFilterBar.Pin())
	{
		FilterBar->OnFiltersChanged().RemoveAll(this);
	}

	if (SNavigationToolCustomTextFilterDialog::IsOpen())
	{
		SNavigationToolCustomTextFilterDialog::CloseWindow();
	}

	if (TextExpressionHelpDialog.IsValid())
	{
		TextExpressionHelpDialog->RequestDestroyWindow();
		TextExpressionHelpDialog.Reset();
	}
}

void SNavigationToolFilterBar::Construct(const FArguments& InArgs, const TWeakPtr<FNavigationToolFilterBar>& InWeakFilterBar)
{
	WeakFilterBar = InWeakFilterBar;

	WeakSearchBox = InArgs._FilterSearchBox;
	FilterBarLayout = InArgs._FilterBarLayout;
	bCanChangeOrientation = InArgs._CanChangeOrientation;
	FilterPillStyle = InArgs._FilterPillStyle;

	ContextMenu = MakeShared<FNavigationToolFilterBarContextMenu>();

	HorizontalContainerWidget = SNew(SFilterBarClippingHorizontalBox)
		.OnWrapButtonClicked(FOnGetContent::CreateSP(this, &SNavigationToolFilterBar::OnWrapButtonClicked))
		.IsFocusable(false);

	ChildSlot
	[
		SAssignNew(FilterBoxWidget, SWidgetSwitcher)
		.WidgetIndex_Lambda([this]()
			{
				return (FilterBarLayout == EFilterBarLayout::Horizontal) ? 0 : 1;
			})
		+ SWidgetSwitcher::Slot()
		.Padding(FMargin(0.f, 2.f, 0.f, 0.f))
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			[
				HorizontalContainerWidget.ToSharedRef()
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				HorizontalContainerWidget->CreateWrapButton()
			]
		]
		+ SWidgetSwitcher::Slot()
		[
			SAssignNew(VerticalContainerWidget, SScrollBox)
			.Visibility_Lambda([this]()
				{
				   return HasAnyFilterWidgets() ? EVisibility::Visible : EVisibility::Collapsed;
				})
		]
	];

	AttachFilterSearchBox(InArgs._FilterSearchBox);

	CreateFilterWidgetsFromConfig();

	if (const TSharedPtr<FNavigationToolFilterBar> FilterBar = WeakFilterBar.Pin())
	{
		FilterBar->OnFiltersChanged().AddSP(this, &SNavigationToolFilterBar::OnFiltersChanged);
	}

	SetMuted(InArgs._FiltersMuted);
}

FReply SNavigationToolFilterBar::OnMouseButtonUp(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	const TSharedPtr<FNavigationToolFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return FReply::Unhandled();
	}

	if (InPointerEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		const FWidgetPath* EventPath = InPointerEvent.GetEventPath();

		FSlateApplication::Get().PushMenu(AsShared()
			, EventPath ? *EventPath : FWidgetPath()
			, ContextMenu->CreateMenu(FilterBar.ToSharedRef())
			, InPointerEvent.GetScreenSpacePosition()
			, FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));

		return FReply::Handled().ReleaseMouseCapture();
	}

	return FReply::Unhandled();
}

TSharedPtr<FNavigationToolFilterBar> SNavigationToolFilterBar::GetFilterBar() const
{
	return WeakFilterBar.Pin();
}

void SNavigationToolFilterBar::SetTextFilterString(const FString& InText)
{
	const TSharedPtr<FNavigationToolFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return;
	}

	if (const TSharedPtr<SSequencerSearchBox> SearchBox = WeakSearchBox.Pin())
	{
		if (!InText.Equals(SearchBox->GetText().ToString()))
		{
			SearchBox->SetText(FText::FromString(InText));
		}

		SearchBox->SetError(FilterBar->GetFilterErrorText());
	}
}

FText SNavigationToolFilterBar::GetFilterErrorText() const
{
	return WeakFilterBar.IsValid() ? WeakFilterBar.Pin()->GetFilterErrorText() : FText::GetEmpty();
}

EFilterBarLayout SNavigationToolFilterBar::GetLayout() const
{
	return FilterBarLayout;
}

void SNavigationToolFilterBar::SetLayout(const EFilterBarLayout InFilterBarLayout)
{
	if (!bCanChangeOrientation)
	{
		return;
	}

	FilterBarLayout = InFilterBarLayout;

	HorizontalContainerWidget->ClearChildren();
	VerticalContainerWidget->ClearChildren();

	for (const TPair<TSharedRef<FNavigationToolFilter>, TSharedRef<SSequencerFilter>>& FilterPair : FilterWidgets)
	{
		AddWidgetToLayout(FilterPair.Value);
	}
}

void SNavigationToolFilterBar::AttachFilterSearchBox(const TSharedPtr<SSequencerSearchBox>& InFilterSearchBox)
{
	if (InFilterSearchBox)
	{
		WeakSearchBox = InFilterSearchBox;

		InFilterSearchBox->SetOnSaveSearchHandler(
			 SFilterSearchBox::FOnSaveSearchClicked::CreateSP(this, &SNavigationToolFilterBar::CreateAddCustomTextFilterWindowFromSearch));
	}
}

bool SNavigationToolFilterBar::HasAnyFilterWidgets() const
{
	return FilterWidgets.Num() > 0;
}

void SNavigationToolFilterBar::AddWidgetToLayout(const TSharedRef<SWidget>& InWidget)
{
	FMargin SlotPadding = FMargin(1); // default editor-wide is FMargin(4, 2) for vertical only

	if (FilterBarLayout == EFilterBarLayout::Horizontal)
	{
		switch (FilterPillStyle)
		{
		case EFilterPillStyle::Basic:
			SlotPadding = FMargin(1); // default editor-wide is 2
			break;
		case EFilterPillStyle::Default:
		default:
			SlotPadding = FMargin(1); // default editor-wide is 3
		}

		HorizontalContainerWidget->AddSlot()
			.AutoWidth()
			.Padding(SlotPadding)
			[
				InWidget
			];
	}
	else
	{
		VerticalContainerWidget->AddSlot()
			.AutoSize()
			.Padding(SlotPadding)
			[
				InWidget
			];
	}
}

void SNavigationToolFilterBar::RemoveWidgetFromLayout(const TSharedRef<SWidget>& InWidget)
{
	if (FilterBarLayout == EFilterBarLayout::Horizontal)
	{
		HorizontalContainerWidget->RemoveSlot(InWidget);
	}
	else
	{
		VerticalContainerWidget->RemoveSlot(InWidget);
	}
}

void SNavigationToolFilterBar::CreateAndAddFilterWidget(const TSharedRef<FNavigationToolFilter>& InFilter)
{
	const TSharedPtr<FNavigationToolFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return;
	}

	const TSharedRef<FNavigationToolFilterBar> FilterBarRef = FilterBar.ToSharedRef();

	const TSharedRef<SSequencerFilter> NewFilterWidget = SNew(SSequencerFilter)
		.FilterPillStyle(FilterPillStyle)
		.DisplayName(this, &SNavigationToolFilterBar::GetFilterDisplayName, InFilter)
        .ToolTipText(InFilter, &FNavigationToolFilter::GetToolTipText)
		.BlockColor(this, &SNavigationToolFilterBar::GetFilterBlockColor, InFilter)
		.OnIsFilterActive(FilterBarRef, &FNavigationToolFilterBar::IsFilterActive, InFilter)
		.OnFilterToggle(this, &SNavigationToolFilterBar::OnFilterToggle, InFilter)
		.OnAltClick(this, &SNavigationToolFilterBar::OnFilterAltClick, InFilter)
		.OnMiddleClick(this, &SNavigationToolFilterBar::OnFilterMiddleClick, InFilter)
		.OnDoubleClick(this, &SNavigationToolFilterBar::OnFilterDoubleClick, InFilter)
		.OnGetMenuContent(this, &SNavigationToolFilterBar::OnGetMenuContent, InFilter);

	AddFilterWidget(InFilter, NewFilterWidget);
}

FText SNavigationToolFilterBar::GetFilterDisplayName(const TSharedRef<FNavigationToolFilter> InFilter) const
{
	return InFilter->GetDisplayName();
}

void SNavigationToolFilterBar::OnFilterToggle(const ECheckBoxState InNewState, const TSharedRef<FNavigationToolFilter> InFilter)
{
	const TSharedPtr<FNavigationToolFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return;
	}

	const bool bNewActive = InNewState == ECheckBoxState::Checked;
	FilterBar->SetFilterActive(InFilter, bNewActive, true);
}

void SNavigationToolFilterBar::OnFilterCtrlClick(const TSharedRef<FNavigationToolFilter> InFilter)
{
	ActivateAllButThis(false, InFilter);
}

void SNavigationToolFilterBar::OnFilterAltClick(const TSharedRef<FNavigationToolFilter> InFilter)
{
	ActivateAllButThis(true, InFilter);
}

void SNavigationToolFilterBar::OnFilterMiddleClick(const TSharedRef<FNavigationToolFilter> InFilter)
{
	const TSharedPtr<FNavigationToolFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return;
	}

	FilterBar->SetFilterEnabled(InFilter, false, true);
}

void SNavigationToolFilterBar::OnFilterDoubleClick(const TSharedRef<FNavigationToolFilter> InFilter)
{
	ActivateAllButThis(false, InFilter);
}

FSlateColor SNavigationToolFilterBar::GetFilterBlockColor(const TSharedRef<FNavigationToolFilter> InFilter) const
{
	const TSharedPtr<FNavigationToolFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return FSlateColor();
	}

	if (!FilterBar->IsFilterActive(InFilter))
	{
		return FAppStyle::Get().GetSlateColor(TEXT("Colors.Recessed"));;
	}

	return InFilter->GetColor();
}

TSharedRef<SWidget> SNavigationToolFilterBar::OnGetMenuContent(const TSharedRef<FNavigationToolFilter> InFilter)
{
	FilterContextMenu = MakeShared<FNavigationToolFilterContextMenu>();
	return FilterContextMenu->CreateMenuWidget(InFilter);
}

void SNavigationToolFilterBar::ActivateAllButThis(const bool bInActive, const TSharedRef<FNavigationToolFilter> InFilter)
{
	const TSharedPtr<FNavigationToolFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return;
	}

	FilterBar->ActivateAllEnabledFilters(bInActive, {});
	FilterBar->SetFilterActive(InFilter, !bInActive, true);
}

void SNavigationToolFilterBar::AddFilterWidget(const TSharedRef<FNavigationToolFilter>& InFilter, const TSharedRef<SSequencerFilter>& InFilterWidget)
{
	FilterWidgets.Add(InFilter, InFilterWidget);
	FilterWidgets.KeySort([](const TSharedRef<FNavigationToolFilter>& InFilterA
		, const TSharedRef<FNavigationToolFilter>& InFilterB)
		{
			return InFilterA->GetDisplayName().CompareTo(InFilterB->GetDisplayName()) < 0;
		});

	AddWidgetToLayout(InFilterWidget);
}

void SNavigationToolFilterBar::RemoveFilterWidget(const TSharedRef<FNavigationToolFilter>& InFilter)
{
	if (!FilterWidgets.Contains(InFilter))
	{
		return;
	}

	RemoveWidgetFromLayout(FilterWidgets[InFilter]);

	FilterWidgets.Remove(InFilter);
}

void SNavigationToolFilterBar::RemoveAllFilterWidgets()
{
	for (const TPair<TSharedRef<FNavigationToolFilter>, TSharedRef<SSequencerFilter>>& FilterPair : FilterWidgets)
	{
		RemoveWidgetFromLayout(FilterPair.Value);
	}

	FilterWidgets.Empty();
}

void SNavigationToolFilterBar::RemoveAllFilterWidgetsButThis(const TSharedRef<FNavigationToolFilter>& InFilter)
{
	if (!FilterWidgets.Contains(InFilter))
	{
		return;
	}

	for (const TPair<TSharedRef<FNavigationToolFilter>, TSharedRef<SSequencerFilter>>& FilterPair : FilterWidgets)
	{
		if (FilterPair.Key == InFilter)
		{
			continue;
		}

		RemoveWidgetFromLayout(FilterPair.Value);
	}

	const TSharedRef<SSequencerFilter> PreviousWidget = FilterWidgets[InFilter];

	FilterWidgets.Empty();

	AddFilterWidget(InFilter, PreviousWidget);
}

void SNavigationToolFilterBar::OnEnableAllGroupFilters(bool bEnableAll)
{
	const TSharedPtr<FNavigationToolFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return;
	}

	UMovieSceneSequence* const FocusedMovieSequence = FilterBar->GetSequencer().GetFocusedMovieSceneSequence();
	if (!FocusedMovieSequence)
	{
		return;
	}
	
	UMovieScene* const FocusedMovieScene = FocusedMovieSequence->GetMovieScene();
	if (!FocusedMovieScene)
	{
		return;
	}
	
	for (UMovieSceneNodeGroup* const NodeGroup : FocusedMovieScene->GetNodeGroups())
	{
		NodeGroup->SetEnableFilter(bEnableAll);
	}
}

void SNavigationToolFilterBar::OnNodeGroupFilterClicked(UMovieSceneNodeGroup* NodeGroup)
{
	if (NodeGroup)
	{
		NodeGroup->SetEnableFilter(!NodeGroup->GetEnableFilter());
	}
}

UWorld* SNavigationToolFilterBar::GetWorld() const
{
	const TSharedPtr<FNavigationToolFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return nullptr;
	}

	UObject* const PlaybackContext = FilterBar->GetSequencer().GetPlaybackContext();
	if (!PlaybackContext)
	{
		return nullptr;
	}

	return PlaybackContext->GetWorld();
}

TWeakPtr<SSequencerSearchBox> SNavigationToolFilterBar::GetSearchBox() const
{
	return WeakSearchBox;
}

void SNavigationToolFilterBar::SetMuted(bool bInMuted)
{
	if (HorizontalContainerWidget.IsValid())
	{
		HorizontalContainerWidget->SetEnabled(!bInMuted);
	}

	if (VerticalContainerWidget.IsValid())
	{
		VerticalContainerWidget->SetEnabled(!bInMuted);
	}

	if (WeakSearchBox.IsValid())
	{
		WeakSearchBox.Pin()->SetEnabled(!bInMuted);
	}
}

void SNavigationToolFilterBar::OnFiltersChanged(const ENavigationToolFilterChange InChangeType, const TSharedRef<FNavigationToolFilter>& InFilter)
{
	switch (InChangeType)
	{
	case ENavigationToolFilterChange::Enable:
	case ENavigationToolFilterChange::Activate:
		{
			if (!FilterWidgets.Contains(InFilter))
			{
				CreateAndAddFilterWidget(InFilter);
			}
			break;
		}
	case ENavigationToolFilterChange::Disable:
		{
			RemoveFilterWidget(InFilter);
			break;
		}
	case ENavigationToolFilterChange::Deactivate:
		{
			break;
		}
	};
}

void SNavigationToolFilterBar::CreateAddCustomTextFilterWindowFromSearch(const FText& InSearchText)
{
	const TSharedPtr<FNavigationToolFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return;
	}

	FilterBar->CreateWindow_AddCustomTextFilter(FNavigationToolFilterBar::DefaultNewCustomTextFilterData(InSearchText));
}

void SNavigationToolFilterBar::CreateFilterWidgetsFromConfig()
{
	const TSharedPtr<FNavigationToolFilterBar> FilterBar = WeakFilterBar.Pin();
	if (!FilterBar.IsValid())
	{
		return;
	}

	UNavigationToolSettings* const ToolSettings = GetMutableDefault<UNavigationToolSettings>();
	if (!ToolSettings)
	{
		return;
	}

	const FName InstanceIdentifier = FilterBar->GetIdentifier();
	FSequencerFilterBarConfig* const Config = ToolSettings->FindFilterBar(InstanceIdentifier);
	if (!Config)
	{
		UE_LOGF(LogSequenceNavigator, Error, "SNavigationToolFilterBar requires that you specify a FilterBarIdentifier to load settings");
		return;
	}

	RemoveAllFilterWidgets();

	const TSet<TSharedRef<FFilterCategory>> DisplayableCategories = FilterBar->GetConfigCategories();

	auto LoadFilterFromConfig = [this, &Config, &DisplayableCategories](const TSharedRef<FNavigationToolFilter>& InFilter)
	{
		const TSharedPtr<FFilterCategory> FilterCategory = InFilter->GetCategory();
		if (FilterCategory.IsValid() && !DisplayableCategories.Contains(FilterCategory.ToSharedRef()))
		{
			return;
		}

		const FString FilterName = InFilter->GetDisplayName().ToString();
		if (!Config->IsFilterEnabled(FilterName))
		{
			return;
		}

		if (!FilterWidgets.Contains(InFilter))
		{
			CreateAndAddFilterWidget(InFilter);
		}
	};

	for (const TSharedRef<FNavigationToolFilter>& Filter : FilterBar->GetCommonFilters())
	{
		LoadFilterFromConfig(Filter);
	}

	for (const TSharedRef<FNavigationToolFilter_CustomText>& Filter : FilterBar->GetAllCustomTextFilters())
	{
		LoadFilterFromConfig(StaticCastSharedRef<FNavigationToolFilter>(Filter));
	}
}

TSharedRef<SWidget> SNavigationToolFilterBar::OnWrapButtonClicked()
{
	const TSharedRef<SVerticalBox> VerticalContainer = SNew(SVerticalBox);

	const int32 NumSlots = HorizontalContainerWidget->NumSlots();
	int32 SlotIndex = HorizontalContainerWidget->GetClippedIndex();
	for (; SlotIndex < NumSlots; ++SlotIndex)
	{
		SHorizontalBox::FSlot& Slot = HorizontalContainerWidget->GetSlot(SlotIndex);

		VerticalContainer->AddSlot()
			.AutoHeight()
			.Padding(1.f)
			[
				Slot.GetWidget()
			];
	}

	const TSharedRef<SBorder> ContainerBorder = SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush(TEXT("Brushes.Panel")))
		.Padding(0.f, 2.f, 2.f, 2.f)
		[
			VerticalContainer
		];

	return SNew(SBox)
		.Padding(8.f)
		[
			HorizontalContainerWidget->WrapVerticalListWithHeading(ContainerBorder
				, FPointerEventHandler::CreateSP(this, &SNavigationToolFilterBar::OnMouseButtonUp))
		];
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
