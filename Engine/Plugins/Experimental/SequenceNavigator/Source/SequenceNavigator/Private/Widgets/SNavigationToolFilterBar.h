// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/Filters/NavigationToolFilterBase.h"
#include "Filters/NavigationToolFilterBar.h"
#include "Filters/SBasicFilterBar.h"

class SFilterBarClippingHorizontalBox;
class SFilterExpressionHelpDialog;
class SSequencerFilter;
class SSequencerSearchBox;
class UMovieSceneNodeGroup;

namespace UE::SequenceNavigator
{

class FNavigationToolFilter;
class FNavigationToolFilterBar;
class FNavigationToolFilterBarContextMenu;
class FNavigationToolFilterContextMenu;
enum class ENavigationToolFilterChange : uint8;

class SNavigationToolFilterBar : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNavigationToolFilterBar)
		: _FilterBarLayout(EFilterBarLayout::Vertical)
		, _FiltersMuted(false)
		, _CanChangeOrientation(true)
		, _FilterPillStyle(EFilterPillStyle::Default)
		, _UseSectionsForCategories(true)
	{}
		/** An SFilterSearchBox that can be attached to this filter bar. When provided along with a CreateTextFilter
		 *  delegate, allows the user to save searches from the Search Box as text filters for the filter bar.
		 *	NOTE: Will bind a delegate to SFilterSearchBox::OnClickedAddSearchHistoryButton */
		SLATE_ARGUMENT(TSharedPtr<SSequencerSearchBox>, FilterSearchBox)

		/** The layout that determines how the filters are laid out */
		SLATE_ARGUMENT(EFilterBarLayout, FilterBarLayout)

		/** Sets the filters muted state */
		SLATE_ARGUMENT(bool, FiltersMuted)

		/** If true, allow dynamically changing the orientation and saving in the config */
		SLATE_ARGUMENT(bool, CanChangeOrientation)

		/** Determines how each individual filter pill looks like */
		SLATE_ARGUMENT(EFilterPillStyle, FilterPillStyle)

		/** Whether to use submenus or sections for categories in the filter menu */
		SLATE_ARGUMENT(bool, UseSectionsForCategories)

	SLATE_END_ARGS()

	virtual ~SNavigationToolFilterBar() override;

	void Construct(const FArguments& InArgs, const TWeakPtr<FNavigationToolFilterBar>& InWeakFilterBar);

	//~ Begin SWidget
	virtual FReply OnMouseButtonUp(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) override;
	//~ End SWidget

	TSharedPtr<FNavigationToolFilterBar> GetFilterBar() const;

	void SetTextFilterString(const FString& InFilterString);

	FText GetFilterErrorText() const;

	EFilterBarLayout GetLayout() const;
	void SetLayout(const EFilterBarLayout InFilterBarLayout);

	void AttachFilterSearchBox(const TSharedPtr<SSequencerSearchBox>& InFilterSearchBox);

	bool HasAnyFilterWidgets() const;

	void CreateAddCustomTextFilterWindowFromSearch(const FText& InSearchText);

	TWeakPtr<SSequencerSearchBox> GetSearchBox() const;

	// Set the state of the filter bar. Muted means that the filters are muted, but the context menu is still enabled and accessible.
	void SetMuted(bool bInMuted);

protected:
	void AddWidgetToLayout(const TSharedRef<SWidget>& InWidget);
	void RemoveWidgetFromLayout(const TSharedRef<SWidget>& InWidget);

	void CreateAndAddFilterWidget(const TSharedRef<FNavigationToolFilter>& InFilter);
	void AddFilterWidget(const TSharedRef<FNavigationToolFilter>& InFilter, const TSharedRef<SSequencerFilter>& InFilterWidget);
	void RemoveFilterWidget(const TSharedRef<FNavigationToolFilter>& InFilter);
	void RemoveAllFilterWidgets();
	void RemoveAllFilterWidgetsButThis(const TSharedRef<FNavigationToolFilter>& InFilter);

	UWorld* GetWorld() const;

	void OnEnableAllGroupFilters(bool bEnableAll);
	void OnNodeGroupFilterClicked(UMovieSceneNodeGroup* NodeGroup);

	void CreateFilterWidgetsFromConfig();

	TSharedRef<SWidget> OnWrapButtonClicked();

	FText GetFilterDisplayName(const TSharedRef<FNavigationToolFilter> InFilter) const;
	FSlateColor GetFilterBlockColor(const TSharedRef<FNavigationToolFilter> InFilter) const;

	void OnFilterToggle(const ECheckBoxState InNewState, const TSharedRef<FNavigationToolFilter> InFilter);
	void OnFilterCtrlClick(const TSharedRef<FNavigationToolFilter> InFilter);
	void OnFilterAltClick(const TSharedRef<FNavigationToolFilter> InFilter);
	void OnFilterMiddleClick(const TSharedRef<FNavigationToolFilter> InFilter);
	void OnFilterDoubleClick(const TSharedRef<FNavigationToolFilter> InFilter);

	TSharedRef<SWidget> OnGetMenuContent(const TSharedRef<FNavigationToolFilter> InFilter);

	void ActivateAllButThis(const bool bInActive, const TSharedRef<FNavigationToolFilter> InFilter);

	void OnFiltersChanged(const ENavigationToolFilterChange InChangeType, const TSharedRef<FNavigationToolFilter>& InFilter);

	TWeakPtr<FNavigationToolFilterBar> WeakFilterBar;

	TWeakPtr<SSequencerSearchBox> WeakSearchBox;

	TSharedPtr<SWidgetSwitcher> FilterBoxWidget;
	TSharedPtr<SFilterBarClippingHorizontalBox> HorizontalContainerWidget;
	TSharedPtr<SScrollBox> VerticalContainerWidget;

	EFilterBarLayout FilterBarLayout = EFilterBarLayout::Vertical;
	bool bCanChangeOrientation = true;
	EFilterPillStyle FilterPillStyle = EFilterPillStyle::Default;

	TMap<TSharedRef<FNavigationToolFilter>, TSharedRef<SSequencerFilter>> FilterWidgets;

	TSharedPtr<SFilterExpressionHelpDialog> TextExpressionHelpDialog;

	TSharedPtr<FNavigationToolFilterBarContextMenu> ContextMenu;
	TSharedPtr<FNavigationToolFilterContextMenu> FilterContextMenu;
};

} // namespace UE::SequenceNavigator
