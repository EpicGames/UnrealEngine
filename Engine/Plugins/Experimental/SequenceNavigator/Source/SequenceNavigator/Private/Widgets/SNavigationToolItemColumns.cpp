// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNavigationToolItemColumns.h"
#include "Columns/INavigationToolColumn.h"
#include "Framework/Application/SlateApplication.h"
#include "Internationalization/Text.h"
#include "NavigationTool.h"
#include "NavigationToolView.h"
#include "Styling/NavigationToolStyleUtils.h"
#include "Styling/SlateBrush.h"
#include "Styling/StyleColors.h"
#include "Styling/ToolBarStyle.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSeparator.h"

#define LOCTEXT_NAMESPACE "SNavigationToolItemColumns"

namespace UE::SequenceNavigator
{

void SNavigationToolItemColumns::Construct(const FArguments& InArgs, const TSharedRef<FNavigationToolView>& InToolView)
{
	WeakToolView = InToolView;

	constexpr float SequenceDuration = 0.125f;
	ExpandCurveSequence.AddCurve(0.f, SequenceDuration, ECurveEaseFunction::CubicInOut);

	if (const TSharedPtr<FNavigationTool> Tool = StaticCastSharedPtr<FNavigationTool>(InToolView->GetOwnerTool()))
	{
		Tool->OnToolLoaded.AddSP(this, &SNavigationToolItemColumns::OnToolLoaded);
	}

	TSharedRef<SVerticalBox> VerticalPanel = SNew(SVerticalBox);

	ItemScrollBox = SNew(SScrollBox)
		.Orientation(Orient_Vertical)
		.ScrollBarVisibility(EVisibility::Collapsed);

	VerticalPanel->AddSlot()
		.AutoHeight()
		[
			SNew(SButton)
			.ButtonStyle(&GetFilterItemMenuButtonStyle())
			.ToolTipText(this, &SNavigationToolItemColumns::GetItemMenuButtonToolTip)
			.OnClicked(this, &SNavigationToolItemColumns::ToggleShowItemColumns)
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush(TEXT("BTEditor.Graph.BTNode.Decorator.Optional.Icon")))
				.DesiredSizeOverride(FVector2D(18.f))
			]
		];

	VerticalPanel->AddSlot()
	   .AutoHeight()
	   .MaxHeight(3.f)
		[
			SNew(SColorBlock)
			.Color(FStyleUtils::GetColor(EStyleType::Normal, true).GetSpecifiedColor() * 0.75f)
		];

	for (const TPair<FName, TSharedPtr<INavigationToolColumn>>& ColumnPair : InToolView->GetColumns())
	{
		if (ColumnPair.Value->CanHideColumn(ColumnPair.Key))
		{
			AddItemSlot(ColumnPair.Value);
		}
	}

	VerticalPanel->AddSlot()
		.FillHeight(1.f)
		[
			SAssignNew(ItemBox, SBox)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.MaxWidth(4.f)
				[
					SNew(SColorBlock)
					.Color(FStyleUtils::GetColor(EStyleType::Normal, true).GetSpecifiedColor())
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1.f)
				[
					ItemScrollBox.ToSharedRef()
				]
			]
		];

	VerticalPanel->AddSlot()
		.AutoHeight()
		.Padding(1.f, 5.f, 1.f, 5.f)
		[
			SNew(SSeparator)
			.Orientation(Orient_Horizontal)
			.Thickness(FStyleUtils::GetSlimToolBarStyle().SeparatorThickness)
			.SeparatorImage(&FStyleUtils::GetSlimToolBarStyle().SeparatorBrush)
		];

	auto AddShortcut = [&VerticalPanel, this](const FSlateBrush* Brush
		, FOnClicked&& OnClicked
		, FText&& Tooltip)
	{
		VerticalPanel->AddSlot()
	       .AutoHeight()
	       .Padding(4.f, 2.f, 4.f, 2.f)
	       .HAlign(HAlign_Left)
	       .VAlign(VAlign_Top)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), TEXT("SimpleButton"))
				.OnClicked(MoveTemp(OnClicked))
				.ToolTipText(MoveTemp(Tooltip))
				[
					SNew(SImage)
					.Image(Brush)
					.DesiredSizeOverride(FVector2D(16.f))
				]
			];
	};

	AddShortcut(FAppStyle::GetBrush(TEXT("FoliageEditMode.SelectAll"))
		, FOnClicked::CreateSP(this, &SNavigationToolItemColumns::ShowAll)
		, LOCTEXT("ShowAllColumns", "Show All Columns"));

	AddShortcut(FAppStyle::GetBrush(TEXT("FoliageEditMode.DeselectAll"))
		, FOnClicked::CreateSP(this, &SNavigationToolItemColumns::HideAll)
		, LOCTEXT("HideAllColumns", "Hide All Columns"));

	ChildSlot
		.VAlign(VAlign_Top)
		.HAlign(HAlign_Fill)
		.Padding(0.f, 2.f, 0.f, 0.f)
		[
			VerticalPanel
		];
}

SNavigationToolItemColumns::~SNavigationToolItemColumns()
{
}

void SNavigationToolItemColumns::AddItemSlot(const TSharedPtr<INavigationToolColumn>& InColumn)
{
	if (!InColumn.IsValid())
	{
		return;
	}

	const FName ColumnId = InColumn->GetColumnId();
	TMap<FName, TSharedPtr<SWidget>>& Slots = ItemSlots;

	if (TSharedPtr<SWidget>* const FoundExistingSlot = Slots.Find(ColumnId))
	{
		if (FoundExistingSlot->IsValid())
		{
			ItemScrollBox->RemoveSlot(FoundExistingSlot->ToSharedRef());	
		}
	}

	const TSharedRef<SWidget> Slot = SNew(SCheckBox)
		.Style(&GetFilterItemCheckboxStyle())
		.ToolTipText(InColumn->GetColumnDisplayNameText())
		.OnCheckStateChanged(this, &SNavigationToolItemColumns::OnCheckBoxStateChanged, InColumn)
		.IsChecked(this, &SNavigationToolItemColumns::IsChecked, InColumn)
		[
			SNew(SScaleBox)
			[
				SNew(SImage)
				.Image(InColumn->GetIconBrush())
				.DesiredSizeOverride(FVector2D(16.f))
			]
		];

	ItemScrollBox->AddSlot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Top)
		[
			Slot
		];

	Slots.Add(ColumnId, Slot);
}

void SNavigationToolItemColumns::OnToolLoaded()
{
	if (const TSharedPtr<FNavigationToolView> ToolView = WeakToolView.Pin())
	{
		OnExpandItemsChanged(*ToolView);
	}
}

void SNavigationToolItemColumns::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	const bool bPlayingSequence = ExpandCurveSequence.IsPlaying();

	if (bPlayingSequence)
	{
		const float Alpha = GetExpandItemsLerp();
		ItemBox->SetRenderOpacity(Alpha);
		ItemBox->SetHeightOverride(Alpha * ItemBoxTargetHeight);
	}
	else if (bPlayedSequenceLastTick)
	{
		ItemBox->SetRenderOpacity(static_cast<float>(bItemsExpanded));
		ItemBox->SetHeightOverride(bItemsExpanded ? TAttribute<FOptionalSize>() : 0.f);
	}

	bPlayedSequenceLastTick = bPlayingSequence;
}

FSlateColor SNavigationToolItemColumns::GetItemStateColor(TSharedPtr<INavigationToolColumn> InColumn) const
{
	if (WeakToolView.IsValid() && WeakToolView.Pin()->IsColumnVisible(InColumn))
	{
		return FSlateColor(FLinearColor(0.701f, 0.225f, 0.003f));
	}
	return FSlateColor::UseForeground();
}

ECheckBoxState SNavigationToolItemColumns::IsChecked(TSharedPtr<INavigationToolColumn> InColumn) const
{
	if (WeakToolView.IsValid() && WeakToolView.Pin()->IsColumnVisible(InColumn))
	{
		return ECheckBoxState::Checked;
	}
	return ECheckBoxState::Unchecked;
}

void SNavigationToolItemColumns::OnCheckBoxStateChanged(const ECheckBoxState InNewCheckState, const TSharedPtr<INavigationToolColumn> InColumn) const
{
	if (const TSharedPtr<FNavigationToolView> ToolView = WeakToolView.Pin())
	{
		if (InNewCheckState == ECheckBoxState::Checked)
		{
			ToolView->ShowColumn(InColumn);
		}
		else
		{
			ToolView->HideColumn(InColumn);
		}
	}
}

float SNavigationToolItemColumns::GetExpandItemsLerp() const
{
	if (ExpandCurveSequence.IsPlaying())
	{
		return ExpandCurveSequence.GetLerp();
	}
	return static_cast<float>(bItemsExpanded);
}

void SNavigationToolItemColumns::OnExpandItemsChanged(const FNavigationToolView& InToolView)
{
	bItemsExpanded = InToolView.ShouldShowItemColumns();
}

FReply SNavigationToolItemColumns::ToggleShowItemColumns()
{
	const TSharedPtr<FNavigationToolView> ToolView = WeakToolView.Pin();
	if (!ToolView.IsValid())
	{
		return FReply::Unhandled();
	}

	if (FSlateApplication::Get().GetModifierKeys().IsShiftDown())
	{
		for (const TPair<FName, TSharedPtr<INavigationToolColumn>>& ColumnPair : ToolView->GetColumns())
		{
			ToolView->HideColumn(ColumnPair.Value);
		}

		ToolView->Refresh();

		return FReply::Handled();
	}

	ToolView->ToggleShowItemColumns();
	OnExpandItemsChanged(*ToolView);

	const float ViewFraction = ItemScrollBox->GetViewFraction();

	if (ViewFraction > 0.f)
	{
		ItemBoxTargetHeight = ItemScrollBox->GetDesiredSize().Y / ViewFraction;
	}
	else
	{
		ItemBoxTargetHeight = 0.f;
	}

	if (ToolView->ShouldShowItemColumns())
	{
		ExpandCurveSequence.Play(SharedThis(this));
	}
	else
	{
		ExpandCurveSequence.PlayReverse(SharedThis(this));
	}

	return FReply::Handled();
}

FReply SNavigationToolItemColumns::ShowAll()
{
	if (const TSharedPtr<FNavigationToolView> ToolView = WeakToolView.Pin())
	{
		for (const TPair<FName, TSharedPtr<INavigationToolColumn>>& ColumnPair : ToolView->GetColumns())
		{
			ToolView->ShowColumn(ColumnPair.Value);
		}
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

FReply SNavigationToolItemColumns::HideAll()
{
	if (const TSharedPtr<FNavigationToolView> ToolView = WeakToolView.Pin())
	{
		for (const TPair<FName, TSharedPtr<INavigationToolColumn>>& ColumnPair : ToolView->GetColumns())
		{
			ToolView->HideColumn(ColumnPair.Value);
		}
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

FText SNavigationToolItemColumns::GetItemMenuButtonToolTip() const
{
	if (const TSharedPtr<FNavigationToolView> ToolView = WeakToolView.Pin())
	{
		FText OutText;

		for (const TPair<FName, TSharedPtr<INavigationToolColumn>>& ColumnPair : ToolView->GetColumns())
		{
			if (ToolView->IsColumnVisible(ColumnPair.Value))
			{
				OutText = FText::Format(LOCTEXT("ItemColumnText", "{0}\n  {1}"), OutText, ColumnPair.Value->GetColumnDisplayNameText());
			}
		}

		return OutText.IsEmpty()
			? LOCTEXT("NoColumnsDisplayedTooltip", "No columns displayed")
			: FText::Format(LOCTEXT("ColumnsDisplayedText", "Columns Displayed: {0}\n\n"
				"Shift + Click to hide all columns"), OutText);
	}

	return FText();
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
