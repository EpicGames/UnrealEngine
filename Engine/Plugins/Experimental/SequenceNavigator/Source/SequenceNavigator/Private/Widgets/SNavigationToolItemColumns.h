// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/CurveSequence.h"
#include "Widgets/SCompoundWidget.h"

class FText;
class SBox;
class SScrollBox;

namespace UE::SequenceNavigator
{

class FNavigationToolView;
class INavigationToolColumn;

class SNavigationToolItemColumns : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNavigationToolItemColumns) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FNavigationToolView>& InToolView);

	virtual ~SNavigationToolItemColumns() override;

	void AddItemSlot(const TSharedPtr<INavigationToolColumn>& InColumn);

	void OnToolLoaded();

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	FSlateColor GetItemStateColor(TSharedPtr<INavigationToolColumn> InColumn) const;

	ECheckBoxState IsChecked(TSharedPtr<INavigationToolColumn> InColumn) const;
	void OnCheckBoxStateChanged(const ECheckBoxState InNewCheckState, const TSharedPtr<INavigationToolColumn> InColumn) const;

	float GetExpandItemsLerp() const;
	void OnExpandItemsChanged(const FNavigationToolView& InToolView);

	FReply ToggleShowItemColumns();

	FReply ShowAll();
	FReply HideAll();

protected:
	FText GetItemMenuButtonToolTip() const;

	TWeakPtr<FNavigationToolView> WeakToolView;

	TSharedPtr<SBox> ItemBox;

	TSharedPtr<SScrollBox> ItemScrollBox;

	TMap<FName, TSharedPtr<SWidget>> ItemSlots;

	FCurveSequence ExpandCurveSequence;

	/** Target Height of the Item Filter Box when playing the sequence */
	float ItemBoxTargetHeight = 0.f;

	/** Whether to expand and show the Item Filter List */
	bool bItemsExpanded = false;

	/** The cached state of Expand Filters Sequence, to know when states have changed. Default to true so that we run it at the start  */
	bool bPlayedSequenceLastTick = true;
};

} // namespace UE::SequenceNavigator
