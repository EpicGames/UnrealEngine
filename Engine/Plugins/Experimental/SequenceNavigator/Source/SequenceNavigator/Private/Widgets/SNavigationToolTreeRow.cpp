// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SNavigationToolTreeRow.h"
#include "Columns/INavigationToolColumn.h"
#include "Items/NavigationToolItem.h"
#include "NavigationToolDefines.h"
#include "NavigationToolStyle.h"
#include "NavigationToolView.h"
#include "Widgets/SNavigationToolTreeView.h"

#define LOCTEXT_NAMESPACE "SNavigationToolTreeRow"

namespace UE::SequenceNavigator
{

void SNavigationToolTreeRow::Construct(const FArguments& InArgs
	, const TSharedRef<FNavigationToolView>& InToolView
	, const TSharedPtr<SNavigationToolTreeView>& InTreeView
	, const FNavigationToolViewModelPtr& InItem)
{
	WeakToolView = InToolView;
	WeakTreeView = InTreeView;
	WeakItem = InItem;
	HighlightText = InArgs._HighlightText;

	SetColorAndOpacity(TAttribute<FLinearColor>::CreateSP(&*InToolView, &FNavigationToolView::GetItemBrushColor, InItem));

	SMultiColumnTableRow::Construct(FSuperRowType::FArguments()
			.Style(&FNavigationToolStyle::Get().GetWidgetStyle<FTableRowStyle>(TEXT("TableViewRow")))
			/*.OnCanAcceptDrop_Lambda([this, InToolView](const FDragDropEvent& InDragDropEvent
				, const EItemDropZone InDropZone
				, const FNavigationToolViewModelWeakPtr InWeakTargetItem)
				{
					const FNavigationToolViewModelWeakPtr WeakTargetItem = InWeakTargetItem.ImplicitPin();
					return InToolView->OnCanDrop(InDragDropEvent, InDropZone, WeakTargetItem);
				})*/
			.OnCanAcceptDrop(InToolView, &FNavigationToolView::OnCanDrop)

			/*.OnDragDetected_Lambda([this, InToolView](const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
				{
					const FNavigationToolViewModelWeakPtr WeakTargetItem = WeakItem.ImplicitPin();
					return InToolView->OnDragDetected(InGeometry, InPointerEvent, WeakTargetItem);
				})*/
			.OnDragDetected(InToolView, &FNavigationToolView::OnDragDetected, WeakItem)

			/*.OnDragEnter_Lambda([this, InToolView](const FDragDropEvent& InDragDropEvent)
				{
					//const FNavigationToolViewModelWeakPtr WeakItem = WeakItem.ImplicitPin();
					return InToolView->OnDragEnter(InDragDropEvent, WeakItem.ImplicitPin());
				})*/
			.OnDragEnter(InToolView, &FNavigationToolView::OnDragEnter, WeakItem)

			/*.OnDragLeave_Lambda([this, InToolView](const FDragDropEvent& InDragDropEvent)
				{
					//const FNavigationToolViewModelWeakPtr WeakItem = WeakItem.ImplicitPin();
					return InToolView->OnDragLeave(InDragDropEvent, WeakItem.ImplicitPin());
				})*/
			.OnDragLeave(InToolView, &FNavigationToolView::OnDragLeave, WeakItem)
	
			/*.OnAcceptDrop_Lambda([InToolView](const FDragDropEvent& InDragDropEvent
				, const EItemDropZone InDropZone
				, const FNavigationToolViewModelWeakPtr InWeakTargetItem)
				{
					const FNavigationToolViewModelWeakPtr WeakTargetItem = InWeakTargetItem.ImplicitPin();
					return InToolView->OnDrop(InDragDropEvent, InDropZone, WeakTargetItem);
				})*/
			.OnAcceptDrop(InToolView, &FNavigationToolView::OnDrop)

			.OnDrop(this, &SNavigationToolTreeRow::OnDefaultDrop)
		, InTreeView.ToSharedRef());
}

TSharedRef<SWidget> SNavigationToolTreeRow::GenerateWidgetForColumn(const FName& InColumnName)
{
	using namespace Sequencer;

	const FNavigationToolViewModelPtr Item = WeakItem.Pin();
	if (!Item.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	const TSharedPtr<FNavigationToolView> ToolView = WeakToolView.Pin();
	if (!ToolView.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	if (const TSharedPtr<INavigationToolColumn> Column = ToolView->GetColumns().FindRef(InColumnName))
	{
		return Column->ConstructRowWidget(Item, ToolView.ToSharedRef(), SharedThis(this));
	}

	return SNullWidget::NullWidget;
}

FReply SNavigationToolTreeRow::OnMouseButtonUp(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	const FNavigationToolViewModelPtr Item = WeakItem.Pin();
	if (!Item.IsValid())
	{
		return FReply::Unhandled();
	}

	const TSharedPtr<FNavigationToolView> ToolView = WeakToolView.Pin();
	if (!ToolView.IsValid())
	{
		return FReply::Unhandled();
	}

	//Select Item and the Tree of Children it contains
	if (InPointerEvent.GetEffectingButton() == EKeys::MiddleMouseButton)
	{
		ENavigationToolItemSelectionFlags Flags = ENavigationToolItemSelectionFlags::IncludeChildren
			| ENavigationToolItemSelectionFlags::SignalSelectionChange
			| ENavigationToolItemSelectionFlags::ScrollIntoView;

		if (InPointerEvent.IsControlDown())
		{
			Flags |= ENavigationToolItemSelectionFlags::AppendToCurrentSelection;
		}

		ToolView->SelectItems({ Item }, Flags);

		Item->OnSelect();

		return FReply::Handled();
	}

	return SMultiColumnTableRow::OnMouseButtonUp(InGeometry, InPointerEvent);
}

FReply SNavigationToolTreeRow::OnMouseButtonDoubleClick(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	// Select Item and the Tree of Children it contains
	if (InPointerEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		if (const TSharedPtr<FNavigationToolView> ToolView = WeakToolView.Pin())
		{
			if (const FNavigationToolViewModelPtr Item = WeakItem.Pin())
			{
				Item->OnDoubleClick();
			}

			return FReply::Handled();
		}
	}

	return SMultiColumnTableRow::OnMouseButtonDoubleClick(InGeometry, InPointerEvent);
}

TSharedPtr<FNavigationToolView> SNavigationToolTreeRow::GetToolView() const
{
	return WeakToolView.Pin();
}

FReply SNavigationToolTreeRow::OnDefaultDrop(const FDragDropEvent& InDragDropEvent) const
{
	if (const TSharedPtr<FNavigationToolView> ToolView = WeakToolView.Pin())
	{
		ToolView->SetDragIntoTreeRoot(false);
	}

	// Always return handled as no action should take place if the Drop wasn't accepted
	return FReply::Handled();
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
