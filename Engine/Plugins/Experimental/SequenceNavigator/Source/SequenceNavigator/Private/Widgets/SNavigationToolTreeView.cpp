// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNavigationToolTreeView.h"
#include "Framework/Commands/UICommandList.h"
#include "Items/INavigationToolItem.h"
#include "Items/NavigationToolItem.h"
#include "NavigationToolDefines.h"
#include "NavigationToolView.h"
#include "MVVM/ViewModels/ViewModelIterators.h"

namespace UE::SequenceNavigator
{

using namespace Sequencer;

void SNavigationToolTreeView::Construct(const FArguments& InArgs, const TSharedPtr<FNavigationToolView>& InToolView)
{
	WeakToolView = InToolView;

	if (InToolView.IsValid())
	{
		InToolView->SaveColumnState();
	}

	STreeView::Construct(InArgs._TreeViewArgs);
}

int32 SNavigationToolTreeView::GetItemIndex(const FNavigationToolViewModelWeakPtr& InWeakItem) const
{
	if (SListView::HasValidItemsSource())
	{
		//const TSharedRef<INavigationToolItem> ItemToFind =
		//	TListTypeTraits<TSharedRef<INavigationToolItem>>::NullableItemTypeConvertToItemType(InItem);
		return SListView::GetItems().Find(InWeakItem);
	}
	return INDEX_NONE;
}

void SNavigationToolTreeView::FocusOnItem(const FNavigationToolViewModelWeakPtr& InWeakItem)
{
	SelectorItem = InWeakItem;
	RangeSelectionStart = InWeakItem;
}

void SNavigationToolTreeView::UpdateItemExpansions(const FNavigationToolViewModelWeakPtr& InWeakItem)
{
	const FNavigationToolViewModelPtr Item = InWeakItem.Pin();
	if (!Item.IsValid())
	{
		return;
	}

	const FSparseItemInfo* const SparseItemInfo = SparseItemInfos.Find(InWeakItem);

	bool bIsExpanded = false;
	bool bHasExpandedChildren = false;

	if (SparseItemInfo)
	{
		bIsExpanded = SparseItemInfo->bIsExpanded;
		bHasExpandedChildren = SparseItemInfo->bHasExpandedChildren;
	}

	// Skip to avoid redundancy
	if (bIsExpanded || bHasExpandedChildren)
	{
		return;
	}

	for (const FNavigationToolViewModelPtr& ChildItem : Item.AsModel()->GetDescendantsOfType<INavigationToolItem>())
	{
		if (IsItemExpanded(ChildItem))
		{
			bHasExpandedChildren = true;
			break;
		}
	}

	if (bIsExpanded || bHasExpandedChildren)
	{
		SparseItemInfos.Add(InWeakItem, FSparseItemInfo(bIsExpanded, bHasExpandedChildren));
	}
}

void SNavigationToolTreeView::Private_SetItemSelection(const FNavigationToolViewModelWeakPtr InWeakItem,
	const bool bInShouldBeSelected, const bool bInWasUserDirected)
{
	const FNavigationToolViewModelPtr Item = InWeakItem.Pin();
	if (!Item.IsValid())
	{
		return;
	}

	const TSharedPtr<FNavigationToolView> ToolView = WeakToolView.Pin();
	if (!ToolView.IsValid()
		|| ToolView->IsToolLocked()
		|| !ToolView->CanSelectItem(Item))
	{
		return;
	}

	STreeView::Private_SetItemSelection(InWeakItem, bInShouldBeSelected, bInWasUserDirected);
}

void SNavigationToolTreeView::Private_ClearSelection()
{
	const TSharedPtr<FNavigationToolView> ToolView = WeakToolView.Pin();
	if (ToolView.IsValid() && !ToolView->IsToolLocked())
	{
		STreeView::Private_ClearSelection();
	}
}

void SNavigationToolTreeView::Private_SignalSelectionChanged(const ESelectInfo::Type InSelectInfo)
{
	STreeView::Private_SignalSelectionChanged(InSelectInfo);

	const TItemSet AddedItems = SelectedItems.Difference(PreviousSelectedItems);
	for (const FNavigationToolViewModelWeakPtr& WeakAddedItem : AddedItems)
	{
		if (const FNavigationToolViewModelPtr AddedItem = WeakAddedItem.Pin())
		{
			AddedItem->OnItemSelectionChanged(true);
		}
	}

	const TItemSet RemovedItems = PreviousSelectedItems.Difference(SelectedItems);
	for (const FNavigationToolViewModelWeakPtr& WeakRemovedItem : RemovedItems)
	{
		if (const FNavigationToolViewModelPtr RemovedItem = WeakRemovedItem.Pin())
		{
			RemovedItem->OnItemSelectionChanged(false);
		}
	}
	PreviousSelectedItems = SelectedItems;
}

void SNavigationToolTreeView::Private_UpdateParentHighlights()
{
	this->Private_ClearHighlightedItems();

	for (const FNavigationToolViewModelWeakPtr& WeakSelectedItem : SelectedItems)
	{
		const FNavigationToolViewModelPtr SelectedItem = WeakSelectedItem.Pin();
		if (!SelectedItem.IsValid())
		{
			continue;
		}

		// Sometimes selection events can come through before the linearized list is built, so the item may not exist yet.
		const int32 ItemIndex = LinearizedItems.Find(SelectedItem);
		if (ItemIndex == INDEX_NONE)
		{
			FNavigationToolViewModelPtr ParentItem = SelectedItem->GetParent();
			while (ParentItem.IsValid())
			{
				if (LinearizedItems.Contains(ParentItem))
				{
					this->SetItemHighlighted(ParentItem, true);
				}
				ParentItem = ParentItem->GetParent();
			}
			continue;
		}

		if (DenseItemInfos.IsValidIndex(ItemIndex))
		{
			const FItemInfo& ItemInfo = DenseItemInfos[ItemIndex];

			int32 ParentIndex = ItemInfo.ParentIndex;

			while (ParentIndex != INDEX_NONE)
			{
				const FNavigationToolViewModelWeakPtr ParentItem = this->LinearizedItems[ParentIndex];
				this->Private_SetItemHighlighted(ParentItem, true);

				const FItemInfo& ParentItemInfo = DenseItemInfos[ParentIndex];

				ParentIndex = ParentItemInfo.ParentIndex;
			}
		}
	}
}

FCursorReply SNavigationToolTreeView::OnCursorQuery(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) const
{
	if (IsRightClickScrolling() && InPointerEvent.IsMouseButtonDown(EKeys::MiddleMouseButton))
	{
		// We hide the native cursor as we'll be drawing the software EMouseCursor::GrabHandClosed cursor
		return FCursorReply::Cursor(EMouseCursor::None);
	}
	return STreeView::OnCursorQuery(InGeometry, InPointerEvent);
}

FReply SNavigationToolTreeView::OnMouseMove(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	if (InPointerEvent.IsMouseButtonDown(EKeys::MiddleMouseButton) && !InPointerEvent.IsTouchEvent())
	{
		// We only care about deltas along the scroll axis
		FTableViewDimensions CursorDeltaDimensions(Orientation, InPointerEvent.GetCursorDelta());
		CursorDeltaDimensions.LineAxis = 0.f;

		const float ScrollByAmount = CursorDeltaDimensions.ScrollAxis / InGeometry.Scale;

		// If scrolling with the right mouse button, we need to remember how much we scrolled.
		// If we did not scroll at all, we will bring up the context menu when the mouse is released.
		AmountScrolledWhileRightMouseDown += FMath::Abs(ScrollByAmount);

		// Has the mouse moved far enough with the right mouse button held down to start capturing
		// the mouse and dragging the view?
		if (IsRightClickScrolling())
		{
			// Make sure the active timer is registered to update the inertial scroll
			if (!bIsScrollingActiveTimerRegistered)
			{
				bIsScrollingActiveTimerRegistered = true;
				RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateSP(this, &SNavigationToolTreeView::UpdateInertialScroll));
			}

			TickScrollDelta -= ScrollByAmount;

			const float AmountScrolled = this->ScrollBy(InGeometry, -ScrollByAmount, AllowOverscroll);

			FReply Reply = FReply::Handled();

			// The mouse moved enough that we're now dragging the view. Capture the mouse
			// so the user does not have to stay within the bounds of the list while dragging.
			if (this->HasMouseCapture() == false)
			{
				Reply.CaptureMouse(AsShared()).UseHighPrecisionMouseMovement(AsShared());
				SoftwareCursorPosition = InGeometry.AbsoluteToLocal(InPointerEvent.GetScreenSpacePosition());
				bShowSoftwareCursor    = true;
			}

			// Check if the mouse has moved.
			if (AmountScrolled != 0)
			{
				SoftwareCursorPosition += CursorDeltaDimensions.ToVector2D();
			}

			return Reply;
		}
	}

	return STreeView::OnMouseMove(InGeometry, InPointerEvent);
}

FReply SNavigationToolTreeView::OnMouseButtonUp(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	if (InPointerEvent.GetEffectingButton() == EKeys::MiddleMouseButton)
	{
		FReply Reply = FReply::Handled().ReleaseMouseCapture();
		AmountScrolledWhileRightMouseDown = 0;
		bShowSoftwareCursor = false;

		// If we have mouse capture, snap the mouse back to the closest location that is within the list's bounds
		if (HasMouseCapture())
		{
			const FSlateRect ListScreenSpaceRect = InGeometry.GetLayoutBoundingRect();
			const FVector2D CursorPosition = InGeometry.LocalToAbsolute(SoftwareCursorPosition);

			const FIntPoint BestPositionInList(
					FMath::RoundToInt(FMath::Clamp(CursorPosition.X, ListScreenSpaceRect.Left, ListScreenSpaceRect.Right)),
					FMath::RoundToInt(FMath::Clamp(CursorPosition.Y, ListScreenSpaceRect.Top, ListScreenSpaceRect.Bottom))
				);

			Reply.SetMousePos(BestPositionInList);
		}

		return Reply;
	}
	return STreeView::OnMouseButtonUp(InGeometry, InPointerEvent);
}

FReply SNavigationToolTreeView::OnKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent)
{
	if (const TSharedPtr<FNavigationToolView> ToolView = WeakToolView.Pin())
	{
		ToolView->UpdateRecentViews();

		TSharedPtr<FUICommandList> CommandList = ToolView->GetViewCommandList();
		if (CommandList.IsValid() && CommandList->ProcessCommandBindings(InKeyEvent))
		{
			return FReply::Handled();	
		}
	}
	return STreeView::OnKeyDown(InGeometry, InKeyEvent);
}

} // namespace UE::SequenceNavigator
