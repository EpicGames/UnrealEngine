// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Items/NavigationToolItem.h"
#include "MVVM/ViewModelPtr.h"
#include "Widgets/Views/STreeView.h"

namespace UE::SequenceNavigator
{

class FNavigationToolView;

class SNavigationToolTreeView
	: public STreeView<FNavigationToolViewModelWeakPtr>
{
public:
	SLATE_BEGIN_ARGS(SNavigationToolTreeView) {}
		SLATE_ARGUMENT(STreeView<FNavigationToolViewModelWeakPtr>::FArguments, TreeViewArgs)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedPtr<FNavigationToolView>& InToolView);

	int32 GetItemIndex(const FNavigationToolViewModelWeakPtr& InWeakItem) const;

	void FocusOnItem(const FNavigationToolViewModelWeakPtr& InWeakItem);

	void UpdateItemExpansions(const FNavigationToolViewModelWeakPtr& InWeakItem);

	//~ Begin STreeView
	virtual void Private_UpdateParentHighlights() override;
	//~ End STreeView

	//~ Begin ITypedTableView
	virtual void Private_SetItemSelection(const FNavigationToolViewModelWeakPtr InWeakItem
		, const bool bInShouldBeSelected, const bool bInWasUserDirected = false) override;
	virtual void Private_SignalSelectionChanged(const ESelectInfo::Type InSelectInfo) override;
	virtual void Private_ClearSelection() override;
	//~ End SListView

	//~ Begin SWidget
	virtual FCursorReply OnCursorQuery(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) const override;
	virtual FReply OnMouseMove(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) override;
	virtual FReply OnKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent) override;
	//~ End SWidget

protected:
	TWeakPtr<FNavigationToolView> WeakToolView;

	TItemSet PreviousSelectedItems;
};

} // namespace UE::SequenceNavigator
