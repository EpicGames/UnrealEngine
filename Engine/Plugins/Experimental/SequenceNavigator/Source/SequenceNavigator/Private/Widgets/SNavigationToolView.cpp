// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNavigationToolView.h"
#include "Filters/Widgets/SFilterBarClippingHorizontalBox.h"
#include "Columns/INavigationToolColumn.h"
#include "DragDropOps/NavigationToolItemDragDropOp.h"
#include "Filters/Filters/NavigationToolFilter_Text.h"
#include "Filters/NavigationToolFilterBar.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UICommandList.h"
#include "Misc/MessageDialog.h"
#include "NavigationTool.h"
#include "NavigationToolSettings.h"
#include "NavigationToolView.h"
#include "Menus/NavigationToolToolbarMenu.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNavigationToolFilterBar.h"
#include "Widgets/SNavigationToolTreeRow.h"
#include "Widgets/SNavigationToolTreeView.h"

#define LOCTEXT_NAMESPACE "SNavigationTool"

namespace UE::SequenceNavigator
{

using namespace Sequencer;

void ExtractAssetSearchFilterTerms(const FText& SearchText, FString* OutFilterKey, FString* OutFilterValue, int32* OutSuggestionInsertionIndex)
{
	const FString SearchString = SearchText.ToString();

	if (OutFilterKey)
	{
		OutFilterKey->Reset();
	}
	if (OutFilterValue)
	{
		OutFilterValue->Reset();
	}
	if (OutSuggestionInsertionIndex)
	{
		*OutSuggestionInsertionIndex = SearchString.Len();
	}

	// Build the search filter terms so that we can inspect the tokens
	FTextFilterExpressionEvaluator LocalFilter(ETextFilterExpressionEvaluatorMode::Complex);
	LocalFilter.SetFilterText(SearchText);

	// Inspect the tokens to see what the last part of the search term was
	// If it was a key->value pair then we'll use that to control what kinds of results we show
	// For anything else we just use the text from the last token as our filter term to allow incremental auto-complete
	const TArray<FExpressionToken>& FilterTokens = LocalFilter.GetFilterExpressionTokens();
	if (FilterTokens.Num() > 0)
	{
		const FExpressionToken& LastToken = FilterTokens.Last();

		// If the last token is a text token, then consider it as a value and walk back to see if we also have a key
		if (LastToken.Node.Cast<TextFilterExpressionParser::FTextToken>())
		{
			if (OutFilterValue)
			{
				*OutFilterValue = LastToken.Context.GetString();
			}
			if (OutSuggestionInsertionIndex)
			{
				*OutSuggestionInsertionIndex = FMath::Min(*OutSuggestionInsertionIndex, LastToken.Context.GetCharacterIndex());
			}

			if (FilterTokens.IsValidIndex(FilterTokens.Num() - 2))
			{
				const FExpressionToken& ComparisonToken = FilterTokens[FilterTokens.Num() - 2];
				if (ComparisonToken.Node.Cast<TextFilterExpressionParser::FEqual>())
				{
					if (FilterTokens.IsValidIndex(FilterTokens.Num() - 3))
					{
						const FExpressionToken& KeyToken = FilterTokens[FilterTokens.Num() - 3];
						if (KeyToken.Node.Cast<TextFilterExpressionParser::FTextToken>())
						{
							if (OutFilterKey)
							{
								*OutFilterKey = KeyToken.Context.GetString();
							}
							if (OutSuggestionInsertionIndex)
							{
								*OutSuggestionInsertionIndex = FMath::Min(*OutSuggestionInsertionIndex, KeyToken.Context.GetCharacterIndex());
							}
						}
					}
				}
			}
		}
		// If the last token is a comparison operator, then walk back and see if we have a key
		else if (LastToken.Node.Cast<TextFilterExpressionParser::FEqual>())
		{
			if (FilterTokens.IsValidIndex(FilterTokens.Num() - 2))
			{
				const FExpressionToken& KeyToken = FilterTokens[FilterTokens.Num() - 2];
				if (KeyToken.Node.Cast<TextFilterExpressionParser::FTextToken>())
				{
					if (OutFilterKey)
					{
						*OutFilterKey = KeyToken.Context.GetString();
					}
					if (OutSuggestionInsertionIndex)
					{
						*OutSuggestionInsertionIndex = FMath::Min(*OutSuggestionInsertionIndex, KeyToken.Context.GetCharacterIndex());
					}
				}
			}
		}
	}
}

void SNavigationToolView::Construct(const FArguments& InArgs, const TSharedRef<FNavigationToolView>& InToolView)
{
	WeakToolView = InToolView;

	ToolbarMenu = MakeShared<FNavigationToolToolbarMenu>();

	if (const TSharedPtr<FNavigationToolFilterBar> FilterBar = InToolView->GetFilterBar())
	{
		FilterBar->OnActivationStateChanged().AddSP(this, &SNavigationToolView::OnFilterActivationChanged);
		FilterBar->OnVisibilityChanged().AddSP(this, &SNavigationToolView::OnFilterVisibilityChanged);
		FilterBar->OnLayoutChanged().AddSP(this, &SNavigationToolView::OnFilterLayoutChanged);
		FilterBar->OnFiltersChanged().AddSP(this, &SNavigationToolView::OnFiltersChanged);
	}

	RebuildWidget(true);
}

void SNavigationToolView::RebuildWidget(const bool bInReconstructColumns)
{
	bIsEnterLastKeyPressed = false;

	if (!HeaderRowWidget.IsValid())
	{
		HeaderRowWidget = SNew(SHeaderRow)
			.CanSelectGeneratedColumn(true);
	}

	if (const TSharedPtr<FNavigationToolView> ToolView = WeakToolView.Pin())
	{
		if (const TSharedPtr<FNavigationToolFilterBar> FilterBar = ToolView->GetFilterBar())
		{
			FilterBarWidget = FilterBar->GenerateWidget();
		}
	}

	if (bInReconstructColumns)
	{
		ReconstructColumns();
	}

	ChildSlot
	[
		ConstructSplitterContent()
	];

	RequestTreeRefresh();
}

TSharedRef<SWidget> SNavigationToolView::ConstructSplitterContent()
{
	const TSharedPtr<FNavigationToolView> ToolView = WeakToolView.Pin();
	if (!ToolView.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	const TSharedPtr<FNavigationToolFilterBar> FilterBar = ToolView->GetFilterBar();
	if (!FilterBar.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	RebuildSearchAndFilterRow();

	if (!FilterBar->ShouldShowFilterBarWidget()
		|| GetFilterBarLayout() == EFilterBarLayout::Horizontal)
	{
		return ConstructMainContent();
	}

	const TSharedRef<SNavigationToolFilterBar> FilterBarWidgetRef = FilterBarWidget.ToSharedRef();

	return SNew(SSplitter)
		.Orientation(Orient_Horizontal)
		.OnSplitterFinishedResizing_Lambda([this]()
			{
				if (UNavigationToolSettings* const ToolSettings = GetMutableDefault<UNavigationToolSettings>())
				{
					ToolSettings->SaveConfig();
				}
			})
		+ SSplitter::Slot()
		.Value(TAttribute<float>::CreateLambda([this]()
			{
				if (const UNavigationToolSettings* const ToolSettings = GetDefault<UNavigationToolSettings>())
				{
					return ToolSettings->GetLastFilterBarSizeCoefficient();
				}
				return 0.f;
			}))
		.OnSlotResized_Lambda([this](const float InNewCoefficient)
			{
				if (UNavigationToolSettings* const ToolSettings = GetMutableDefault<UNavigationToolSettings>())
				{
					ToolSettings->SetLastFilterBarSizeCoefficient(InNewCoefficient, false);
				}
			})
		[
			SFilterBarClippingHorizontalBox::WrapVerticalListWithHeading(FilterBarWidgetRef
				, FPointerEventHandler::CreateSP(FilterBarWidgetRef, &SNavigationToolFilterBar::OnMouseButtonUp))
		]
		+ SSplitter::Slot()
		.Value(0.94f)
		[
			ConstructMainContent()
		];
}

TSharedRef<SWidget> SNavigationToolView::ConstructMainContent()
{
	const TSharedPtr<FNavigationToolView> ToolView = WeakToolView.Pin();
	if (!ToolView.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	const TSharedPtr<FNavigationToolFilterBar> FilterBar = ToolView->GetFilterBar();
	if (!FilterBar.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	const TSharedRef<FNavigationToolView> ToolViewRef = ToolView.ToSharedRef();

	const TSharedRef<SVerticalBox> VerticalBox = SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		[
			ToolbarMenu->CreateToolbar(ToolViewRef)
		];

	if (FilterBar->ShouldShowFilterBarWidget()
		&& GetFilterBarLayout() == EFilterBarLayout::Horizontal)
	{
		VerticalBox->AddSlot()
			.AutoHeight()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			.Padding(FMargin(0, -1, 0, 1))
			[
				FilterBarWidget.ToSharedRef()
			];
	}

	VerticalBox->AddSlot()
		.FillHeight(1.f)
		.HAlign(HAlign_Fill)
		[
			SNew(SOverlay)
			+ SOverlay::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				SAssignNew(TreeBorder, SBorder)
				.BorderImage(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>(TEXT("SceneOutliner.TableViewRow")).DropIndicator_Onto)
				.Visibility(EVisibility::Hidden)
			]
			+ SOverlay::Slot()
			.HAlign(HAlign_Fill)
			[
				SAssignNew(TreeView, SNavigationToolTreeView, ToolViewRef)
				.TreeViewArgs(STreeView<FNavigationToolViewModelWeakPtr>::FArguments()
					.HeaderRow(HeaderRowWidget)
					.TreeItemsSource(&ToolViewRef->GetRootVisibleItems())
					.OnGetChildren(ToolViewRef, &FNavigationToolView::GetChildrenOfItem)
					.OnGenerateRow(this, &SNavigationToolView::OnItemGenerateRow)
					.OnSelectionChanged(this, &SNavigationToolView::OnItemSelectionChanged)
					.OnExpansionChanged(ToolViewRef, &FNavigationToolView::OnItemExpansionChanged)
					.OnContextMenuOpening(ToolViewRef, &FNavigationToolView::CreateItemContextMenu)
					.OnSetExpansionRecursive(ToolViewRef, &FNavigationToolView::SetItemExpansionRecursive)
					.HighlightParentNodesForSelection(true)
					.AllowInvisibleItemSelection(true) // To select items that are still collapsed
					.SelectionMode(ESelectionMode::Multi)
					.OnItemToString_Debug_Lambda([](const FNavigationToolViewModelWeakPtr& InWeakItem)
						{
							const FNavigationToolViewModelPtr Item = InWeakItem.Pin();
							return Item.IsValid() ? Item->GetItemId().GetStringId() : FString();
						})
					)
			]
		];

	return VerticalBox;
}

void SNavigationToolView::RebuildSearchAndFilterRow()
{
	const TSharedPtr<FNavigationToolView> ToolView = WeakToolView.Pin();
	if (!ToolView.IsValid())
	{
		return;
	}

	const TSharedPtr<FNavigationToolFilterBar> FilterBar = ToolView->GetFilterBar();
	if (!FilterBar.IsValid())
	{
		return;
	}

	const TSharedRef<FNavigationToolView> ToolViewRef = ToolView.ToSharedRef();

	if (!SearchAndFilterRow.IsValid())
	{
		SearchAndFilterRow = SNew(SVerticalBox);
	}

	SearchAndFilterRow->ClearChildren();

	SearchAndFilterRow->AddSlot()
		.AutoHeight()
		[
			ToolbarMenu->CreateToolbar(ToolViewRef)
		];

	if (FilterBar->ShouldShowFilterBarWidget()
		&& GetFilterBarLayout() == EFilterBarLayout::Horizontal)
	{
		SearchAndFilterRow->AddSlot()
			.AutoHeight()
			.Padding(0)
			[
				FilterBarWidget.ToSharedRef()
			];
	}
}

void SNavigationToolView::ReconstructColumns()
{
	const TSharedPtr<FNavigationToolView> ToolView = WeakToolView.Pin();
	if (!ToolView.IsValid())
	{
		return;
	}

	const TSharedPtr<INavigationTool> OwnerTool = ToolView->GetOwnerTool();
	if (!OwnerTool.IsValid())
	{
		return;
	}

	HeaderRowWidget->ClearColumns();

	// Add the columns allocated by the owning instance
	for (const TPair<FName, TSharedPtr<INavigationToolColumn>>& Pair : ToolView->GetColumns())
	{
		OwnerTool->ForEachProvider([this, &ToolView, &OwnerTool, &Pair]
			(const TSharedRef<FNavigationToolProvider>& InProvider)
			{
				const TSharedPtr<INavigationToolColumn>& Column = Pair.Value;
				if (!Column.IsValid())
				{
					return true;
				}

				const FName ColumnId = Pair.Key;

				bool bColumnVisible = Column->ShouldShowColumnByDefault();
				float ColumnWidth = Column->GetFillWidth();

				if (FNavigationToolViewSaveState* const ToolViewSaveState = InProvider->GetViewSaveState(*OwnerTool, ToolView->GetToolViewId()))
				{
					if (const FNavigationToolViewColumnSaveState* const ColumnState = ToolViewSaveState->ColumnsState.Find(ColumnId))
					{
						bColumnVisible = ColumnState->bVisible;
						ColumnWidth = ColumnState->Size;
					}
				}

				HeaderRowWidget->AddColumn(Column->ConstructHeaderRowColumn(ToolView.ToSharedRef(), ColumnWidth));
				HeaderRowWidget->SetShowGeneratedColumn(ColumnId, bColumnVisible);

				return true;
			});
	}
}

TSharedPtr<FNavigationToolFilterBar> SNavigationToolView::GetFilterBar() const
{
	if (const TSharedPtr<FNavigationToolView> ToolView = WeakToolView.Pin())
	{
		return ToolView->GetFilterBar();
	}
	return nullptr;
}

bool SNavigationToolView::IsColumnVisible(const FName InColumnId) const
{
	return HeaderRowWidget.IsValid() ? HeaderRowWidget->IsColumnVisible(InColumnId) : false;
}

void SNavigationToolView::ShowHideColumn(const FName InColumnId, const bool bInVisible)
{
	if (HeaderRowWidget.IsValid())
	{
		HeaderRowWidget->SetShowGeneratedColumn(InColumnId, bInVisible);
	}
}

void SNavigationToolView::SetColumnWidth(const FName InColumnId, const float InWidth)
{
	if (HeaderRowWidget.IsValid())
	{
		HeaderRowWidget->SetColumnWidth(InColumnId, InWidth);
	}
}

void SNavigationToolView::RequestTreeRefresh()
{
	check(TreeView.IsValid());
	TreeView->RequestTreeRefresh();
}

void SNavigationToolView::SetItemSelection(const TArray<FNavigationToolViewModelWeakPtr>& InWeakItems
	, const bool bInSignalSelectionChange)
{
	if (bSelectingItems)
	{
		return;
	}

	TGuardValue<bool> Guard(bSelectingItems, true);

	TreeView->Private_ClearSelection();

	if (!InWeakItems.IsEmpty())
	{
		TreeView->SetItemSelection(InWeakItems, true, ESelectInfo::Type::Direct);
	}

	if (bInSignalSelectionChange)
	{
		TreeView->Private_SignalSelectionChanged(ESelectInfo::OnMouseClick);
	}
}

void SNavigationToolView::OnItemSelectionChanged(const FNavigationToolViewModelWeakPtr InWeakItem
	, const ESelectInfo::Type InSelectionType)
{
	const TSharedPtr<FNavigationToolView> ToolView = WeakToolView.Pin();
	if (!ToolView.IsValid())
	{
		return;
	}

	const TArray<FNavigationToolViewModelWeakPtr> WeakSelectedItems = TreeView->GetSelectedItems();
	const bool bUpdateModeTools = InSelectionType != ESelectInfo::Type::Direct;
	ToolView->NotifyItemSelectionChanged(WeakSelectedItems, InWeakItem.Pin(), bUpdateModeTools);
}

void SNavigationToolView::ScrollItemIntoView(const FNavigationToolViewModelWeakPtr& InWeakItem) const
{
	TreeView->RequestScrollIntoView(InWeakItem);
}

bool SNavigationToolView::IsItemExpanded(const FNavigationToolViewModelWeakPtr& InWeakItem) const
{
	return TreeView->IsItemExpanded(InWeakItem);
}

void SNavigationToolView::SetItemExpansion(const FNavigationToolViewModelWeakPtr& InWeakItem, const bool bInExpand) const
{
	TreeView->SetItemExpansion(InWeakItem, bInExpand);
}

void SNavigationToolView::UpdateItemExpansions(const FNavigationToolViewModelWeakPtr& InWeakItem) const
{
	TreeView->UpdateItemExpansions(InWeakItem);
}

TSharedRef<ITableRow> SNavigationToolView::OnItemGenerateRow(const FNavigationToolViewModelWeakPtr InWeakItem
	, const TSharedRef<STableViewBase>& InOwnerTable)
{
	const TSharedPtr<FNavigationToolView> ToolView = WeakToolView.Pin();
	check(ToolView.IsValid());

	const FNavigationToolViewModelPtr Item = InWeakItem.Pin();
	check(Item.IsValid());

	const TSharedPtr<FNavigationToolFilterBar> FilterBar = GetFilterBar();
	check(FilterBar.IsValid());

	return SNew(SNavigationToolTreeRow, ToolView.ToSharedRef(), TreeView, Item)
		.HighlightText(FilterBar->GetTextFilter(), &FNavigationToolFilter_Text::GetRawFilterText);
}

void SNavigationToolView::SetKeyboardFocus() const
{
	if (SupportsKeyboardFocus())
	{
		FWidgetPath TreeViewWidgetPath;
		// NOTE: Careful, GeneratePathToWidget can be reentrant in that it can call visibility delegates and such
		FSlateApplication::Get().GeneratePathToWidgetUnchecked(TreeView.ToSharedRef(), TreeViewWidgetPath);
		FSlateApplication::Get().SetKeyboardFocus(TreeViewWidgetPath, EFocusCause::SetDirectly);
	}
}

void SNavigationToolView::OnMouseEnter(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	if (const TSharedPtr<FNavigationToolView> ToolView = WeakToolView.Pin())
	{
		ToolView->UpdateRecentViews();
	}
	SCompoundWidget::OnMouseEnter(InGeometry, InPointerEvent);
}

FReply SNavigationToolView::OnKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent)
{
	bIsEnterLastKeyPressed = InKeyEvent.GetKey() == EKeys::Enter;
	if (const TSharedPtr<FNavigationToolView> ToolView = WeakToolView.Pin())
	{
		ToolView->UpdateRecentViews();

		TSharedPtr<FUICommandList> CommandList = ToolView->GetBaseCommandList();
		if (CommandList.IsValid() && CommandList->ProcessCommandBindings(InKeyEvent))
		{
			return FReply::Handled();
		}
	}
	return SCompoundWidget::OnKeyDown(InGeometry, InKeyEvent);
}

FReply SNavigationToolView::OnDrop(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent)
{
	const TSharedPtr<FNavigationToolView> ToolView = WeakToolView.Pin();
	if (!ToolView.IsValid())
	{
		return FReply::Unhandled();
	}

	return ToolView->OnDrop(InDragDropEvent, EItemDropZone::OntoItem, nullptr);
}

void SNavigationToolView::OnDragEnter(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent)
{
	if (TSharedPtr<FNavigationToolItemDragDropOp> ItemDragDropOp = InDragDropEvent.GetOperationAs<FNavigationToolItemDragDropOp>())
	{
		bool bIsDragFromThisToolView = ItemDragDropOp->GetToolView() == WeakToolView;
		bool bContainedItemDragDropOp = ItemDragDropOps.Remove(ItemDragDropOp) > 0;

		// Don't process Drag Enter unless it has already left before.
		// Only applicable if the DragDrop starts from is from within (i.e. same outliner view and it's an FNavigationToolItemDragDropOp)
		// This is because DragEnter doesn't have an FReply to stop SNavigationTool from receiving the DragEnter event as soon as a drag starts
		if (bIsDragFromThisToolView && !bContainedItemDragDropOp)
		{
			return;
		}
	}

	if (const TSharedPtr<FNavigationToolView> ToolView = WeakToolView.Pin())
	{
		ToolView->OnDragEnter(InDragDropEvent, nullptr);
	}
}

void SNavigationToolView::OnDragLeave(const FDragDropEvent& InDragDropEvent)
{
	if (const TSharedPtr<FNavigationToolItemDragDropOp> ItemDragDropOp = InDragDropEvent.GetOperationAs<FNavigationToolItemDragDropOp>())
	{
		ItemDragDropOps.Add(ItemDragDropOp);
	}

	if (const TSharedPtr<FNavigationToolView> ToolView = WeakToolView.Pin())
	{
		ToolView->OnDragLeave(InDragDropEvent, nullptr);
	}
}

void SNavigationToolView::SetTreeBorderVisibility(const bool bInVisible)
{
	TreeBorder->SetVisibility(bInVisible ? EVisibility::SelfHitTestInvisible : EVisibility::Hidden);
}

void SNavigationToolView::GenerateColumnState(const FName InColumnId, FNavigationToolViewColumnSaveState& OutColumnState)
{
	for (const SHeaderRow::FColumn& Column : HeaderRowWidget->GetColumns())
	{
		if (Column.ColumnId == InColumnId)
		{
			OutColumnState.bVisible = Column.bIsVisible;
			OutColumnState.Size = Column.GetWidth();
			return;
		}
	}
}

void SNavigationToolView::GenerateColumnStateMap(TMap<FName, FNavigationToolViewColumnSaveState>& OutStateMap)
{
	const TIndirectArray<SHeaderRow::FColumn>& Columns = HeaderRowWidget->GetColumns();
	OutStateMap.Empty(Columns.Num());

	for (const SHeaderRow::FColumn& Column : Columns)
	{
		FNavigationToolViewColumnSaveState NewColumnState;
		NewColumnState.bVisible = Column.bIsVisible;
		NewColumnState.Size = Column.GetWidth();
		OutStateMap.Add(Column.ColumnId, NewColumnState);
	}
}

EFilterBarLayout SNavigationToolView::GetFilterBarLayout() const
{
	const UNavigationToolSettings* const ToolSettings = GetDefault<UNavigationToolSettings>();
	if (ensure(ToolSettings))
	{
		return ToolSettings->GetFilterBarLayout();
	}
	return EFilterBarLayout::Vertical;
}

void SNavigationToolView::SetFilterBarLayout(const EFilterBarLayout InLayout)
{
	UNavigationToolSettings* const ToolSettings = GetMutableDefault<UNavigationToolSettings>();
	if (ensure(ToolSettings))
	{
		ToolSettings->SetFilterBarLayout(InLayout);
	}

	RebuildWidget();
}

void SNavigationToolView::OnFilterActivationChanged()
{
	RebuildWidget();
}

void SNavigationToolView::OnFilterVisibilityChanged(bool bInIsVisible)
{
	RebuildWidget();
}

void SNavigationToolView::OnFilterLayoutChanged(EFilterBarLayout InLayout)
{
	RebuildWidget();
}

void SNavigationToolView::OnFiltersChanged(const ENavigationToolFilterChange InChangeType
                                           , const TSharedRef<FNavigationToolFilter>& InFilter)
{
	RebuildWidget();
}

} // namespace UE::SequenceNavigator

#undef LOCTEXT_NAMESPACE
