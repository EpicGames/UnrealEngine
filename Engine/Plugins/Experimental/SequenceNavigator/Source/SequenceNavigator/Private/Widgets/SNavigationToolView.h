// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/SBasicFilterBar.h"
#include "Items/INavigationToolItem.h"
#include "NavigationToolDefines.h"
#include "Widgets/SCompoundWidget.h"

class ITableRow;
class SBorder;
class SHeaderRow;
class STableViewBase;

namespace UE::SequenceNavigator
{

class FNavigationToolFilter;
class FNavigationToolFilterBar;
class FNavigationToolItemDragDropOp;
class FNavigationToolToolbarMenu;
class FNavigationToolView;
class SNavigationToolFilterBar;
class SNavigationToolTreeView;
enum class ENavigationToolFilterChange : uint8;

class SNavigationToolView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNavigationToolView) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FNavigationToolView>& InToolView);

	TSharedPtr<SNavigationToolTreeView> GetTreeView() const { return TreeView; }

	/** Rebuilds the entire widget. Needs to be called when the filter bar layout changes. */
	void RebuildWidget(const bool bInReconstructColumns = false);

	void ReconstructColumns();

	bool IsColumnVisible(const FName InColumnId) const;
	void ShowHideColumn(const FName InColumnId, const bool bInVisible);

	void SetColumnWidth(const FName InColumnId, const float InWidth);

	void RequestTreeRefresh();

	void SetItemSelection(const TArray<FNavigationToolViewModelWeakPtr>& InWeakItems
		, const bool bInSignalSelectionChange);

	void OnItemSelectionChanged(const FNavigationToolViewModelWeakPtr InWeakItem
		, const ESelectInfo::Type InSelectionType);

	void ScrollItemIntoView(const FNavigationToolViewModelWeakPtr& InWeakItem) const;

	bool IsItemExpanded(const FNavigationToolViewModelWeakPtr& InWeakItem) const;
	void SetItemExpansion(const FNavigationToolViewModelWeakPtr& InWeakItem, const bool bInExpand) const;

	void UpdateItemExpansions(const FNavigationToolViewModelWeakPtr& InWeakItem) const;

	TSharedRef<ITableRow> OnItemGenerateRow(const FNavigationToolViewModelWeakPtr InWeakItem
		, const TSharedRef<STableViewBase>& InOwnerTable);

	void SetKeyboardFocus() const;

	//~ Begin SWidget
	virtual void OnMouseEnter(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) override;
	virtual FReply OnKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent) override;
	virtual FReply OnDrop(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent) override;
	virtual void OnDragEnter(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent) override;
	virtual void OnDragLeave(const FDragDropEvent& InDragDropEvent) override;
	//~ End SWidget

	void SetTreeBorderVisibility(const bool bInVisible);

	void GenerateColumnState(const FName InColumnId, FNavigationToolViewColumnSaveState& OutColumnState);
	void GenerateColumnStateMap(TMap<FName, FNavigationToolViewColumnSaveState>& OutStateMap);

	EFilterBarLayout GetFilterBarLayout() const;
	void SetFilterBarLayout(const EFilterBarLayout InLayout);

protected:
	TSharedRef<SWidget> ConstructSplitterContent();
	TSharedRef<SWidget> ConstructMainContent();

	void RebuildSearchAndFilterRow();

	void OnFilterActivationChanged();
	void OnFilterVisibilityChanged(bool bInIsVisible);
	void OnFilterLayoutChanged(EFilterBarLayout InLayout);
	void OnFiltersChanged(const ENavigationToolFilterChange InChangeType, const TSharedRef<FNavigationToolFilter>& InFilter);

	TSharedPtr<FNavigationToolFilterBar> GetFilterBar() const;

	TWeakPtr<FNavigationToolView> WeakToolView;

	TSharedPtr<FNavigationToolToolbarMenu> ToolbarMenu;

	TSharedPtr<SVerticalBox> SearchAndFilterRow;
	TSharedPtr<SHeaderRow> HeaderRowWidget;
	TSharedPtr<SBorder> TreeBorder;
	TSharedPtr<SNavigationToolTreeView> TreeView;
	TSharedPtr<SNavigationToolFilterBar> FilterBarWidget;

	/** If true the SuggestionList shouldn't appear since we selected what we wanted */
	bool bIsEnterLastKeyPressed = false;

	bool bSelectingItems = false;

	TSet<TWeakPtr<FNavigationToolItemDragDropOp>> ItemDragDropOps;
};

} // namespace UE::SequenceNavigator
