// Copyright Epic Games, Inc. All Rights Reserved.

#include "NavigationToolStyleUtils.h"
#include "Brushes/SlateColorBrush.h"
#include "Brushes/SlateNoResource.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateBrush.h"
#include "Styling/ToolBarStyle.h"

namespace UE::SequenceNavigator
{

const FToolBarStyle& FStyleUtils::GetSlimToolBarStyle()
{
	static const FToolBarStyle& SlimToolBarStyle = FAppStyle::Get().GetWidgetStyle<FToolBarStyle>(TEXT("SlimToolBar"));
	return SlimToolBarStyle;
}

const FSlateBrush& FStyleUtils::GetBrush(EStyleType InStyleType, bool bIsSelected)
{
	const FCheckBoxStyle& Style = GetSlimToolBarStyle().ToggleButton;
	
	switch (InStyleType)
	{
	case EStyleType::Normal:
		return bIsSelected ? Style.CheckedImage : Style.UncheckedImage;
	
	case EStyleType::Hovered:
		return bIsSelected ? Style.CheckedHoveredImage : Style.UncheckedHoveredImage;
	
	case EStyleType::Pressed:
		return bIsSelected ? Style.CheckedPressedImage : Style.UncheckedPressedImage;
	
	default:
		break;
	}
	
	static const FSlateNoResource NullBrush;
	return NullBrush;
}

FSlateColor FStyleUtils::GetColor(EStyleType InStyleType, bool bIsSelected)
{
	return GetBrush(InStyleType, bIsSelected).TintColor;
}

FSlateColorBrush FStyleUtils::GetColorBrush(EStyleType InStyleType, bool bIsSelected)
{
	return FSlateColorBrush(GetColor(InStyleType, bIsSelected));
}

const FButtonStyle& GetFilterItemMenuButtonStyle()
{
	static const FButtonStyle FilterItemMenuButtonStyle = FButtonStyle()
		   .SetNormal(FStyleUtils::GetColorBrush(EStyleType::Normal, true))
		   .SetHovered(FStyleUtils::GetColorBrush(EStyleType::Hovered, true))
		   .SetPressed(FStyleUtils::GetColorBrush(EStyleType::Pressed, true));
	return FilterItemMenuButtonStyle;
}

const FCheckBoxStyle& GetFilterItemCheckboxStyle()
{
	static const FCheckBoxStyle ItemFilterCheckboxStyle = FCheckBoxStyle(FStyleUtils::GetSlimToolBarStyle().ToggleButton)
		   .SetPadding(FMargin(8.f, 4.f, 8.f, 4.f))
		   .SetCheckedImage(FStyleUtils::GetColorBrush(EStyleType::Normal, true))
		   .SetCheckedHoveredImage(FStyleUtils::GetColorBrush(EStyleType::Hovered, true))
		   .SetUncheckedHoveredImage(FStyleUtils::GetColorBrush(EStyleType::Hovered, false))
		   .SetCheckedPressedImage(FStyleUtils::GetColorBrush(EStyleType::Pressed, true))
		   .SetUncheckedPressedImage(FStyleUtils::GetColorBrush(EStyleType::Pressed, false));
	return ItemFilterCheckboxStyle;
}

} // namespace UE::SequenceNavigator
