// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NavigationToolDefines.h"
#include "Widgets/Views/SHeaderRow.h"

#define UE_API SEQUENCENAVIGATOR_API

namespace UE::SequenceNavigator
{

class INavigationToolView;
class SNavigationToolTreeRow;

class INavigationToolColumn
{
public:
	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, INavigationToolColumn)

	virtual ~INavigationToolColumn() = default;

	virtual FName GetColumnId() const = 0;

	virtual FText GetColumnDisplayNameText() const = 0;

	virtual const FSlateBrush* GetIconBrush() const { return nullptr; }

	virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn(const TSharedRef<INavigationToolView>& InToolView, const float InFillSize) = 0;

	virtual float GetFillWidth() const { return 0.f; }

	/*
	 * Determines whether the Column should be Showing by Default while still be able to toggle it on/off.
	 * Used when calling SHeaderRow::SetShowGeneratedColumn (requires ShouldGenerateWidget to not be set).
	 */
	virtual bool ShouldShowColumnByDefault() const { return false; }

	virtual bool CanHideColumn(const FName InColumnId) const { return true; }

	virtual TSharedRef<SWidget> ConstructRowWidget(const FNavigationToolViewModelPtr& InItem
		, const TSharedRef<INavigationToolView>& InView
		, const TSharedRef<SNavigationToolTreeRow>& InRow) = 0;

	virtual void Tick(const float InDeltaTime) {}
};

} // namespace UE::SequenceNavigator

#undef UE_API
