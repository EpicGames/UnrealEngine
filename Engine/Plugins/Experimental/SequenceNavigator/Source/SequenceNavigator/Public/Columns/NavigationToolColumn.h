// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Columns/INavigationToolColumn.h"
#include "Items/NavigationToolItem.h"

#define UE_API SEQUENCENAVIGATOR_API

namespace UE::SequenceNavigator
{

/** Base Navigation Tool Column to extend from */
class FNavigationToolColumn
	: public Sequencer::FViewModel
	, public INavigationToolColumn
{
public:
	UE_SEQUENCER_DECLARE_CASTABLE_API(UE_API, FNavigationToolColumn
		, Sequencer::FViewModel, INavigationToolColumn)
};

} // namespace UE::SequenceNavigator

#undef UE_API
