// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NavigationToolDefines.h"
#include "Templates/UnrealTypeTraits.h"

#define UE_API SEQUENCENAVIGATOR_API

namespace UE::SequenceNavigator
{

class FNavigationToolColumn;

/**
 * Extension Class to add Navigation Tool Columns
 *
 * Example #1:
 * This would create a Tag Column at the end of the current column list (order matters!)
 *    ColumnExtender.AddColumn<FNavigationToolTagColumn>();
 *
 * Example #2:
 * This would create a Tag Column before the Label Column (if it doesn't exist, it's the same behavior as the above example)
 *    ColumnExtender.AddColumn<FNavigationToolTagColumn, ENavigationToolExtensionPosition::Before, FNavigationToolLabelColumn>();
 */
class FNavigationToolColumnExtender
{
public:
	template<typename InColumnType, ENavigationToolExtensionPosition InExtensionPosition, typename InRefColumnType
		UE_REQUIRES(TIsDerivedFrom<InColumnType, FNavigationToolColumn>::Value && TIsDerivedFrom<InRefColumnType, FNavigationToolColumn>::Value)>
	void AddColumn()
	{
		AddColumn(MakeShared<InColumnType>(), InExtensionPosition, InRefColumnType::GetStaticTypeName());
	}

	template<typename InColumnType
		UE_REQUIRES(TIsDerivedFrom<InColumnType, FNavigationToolColumn>::Value)>
	void AddColumn()
	{
		AddColumn(MakeShared<InColumnType>(), ENavigationToolExtensionPosition::Before, NAME_None);
	}

	void AddColumn(const TSharedPtr<FNavigationToolColumn>& InNewColumn
		, const ENavigationToolExtensionPosition InPosition = ENavigationToolExtensionPosition::After)
	{
		AddColumn(InNewColumn, InPosition, NAME_None);
	}

	const TArray<TSharedPtr<FNavigationToolColumn>>& GetColumns() const { return Columns; }

private:
	UE_API void AddColumn(const TSharedPtr<FNavigationToolColumn>& InColumn
		, const ENavigationToolExtensionPosition InExtensionPosition
		, const FName InReferenceColumnId);

	int32 FindColumnIndex(const FName InColumnId) const;

	TArray<TSharedPtr<FNavigationToolColumn>> Columns;
};

} // namespace UE::SequenceNavigator

#undef UE_API
