// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Columns/NavigationToolColumn.h"
#include "Items/NavigationToolItem.h"
#include "MVVM/ICastable.h"

#define UE_API SEQUENCENAVIGATOR_API

namespace UE::SequenceNavigator
{

class INavigationToolView;

class FNavigationToolRevisionControlColumn
	: public FNavigationToolColumn
{
public:
	UE_SEQUENCER_DECLARE_CASTABLE_API(UE_API, FNavigationToolRevisionControlColumn
		, FNavigationToolColumn)

	static FName StaticColumnId() { return TEXT("RevisionControl"); }

protected:
	//~ Begin INavigationToolColumn
	virtual FName GetColumnId() const override { return StaticColumnId(); }
	UE_API virtual FText GetColumnDisplayNameText() const override;
	UE_API virtual const FSlateBrush* GetIconBrush() const override;
	UE_API virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn(const TSharedRef<INavigationToolView>& InView
		, const float InFillSize) override;
	virtual bool ShouldShowColumnByDefault() const override { return true; }
	UE_API virtual TSharedRef<SWidget> ConstructRowWidget(const FNavigationToolViewModelPtr& InItem
		, const TSharedRef<INavigationToolView>& InView
		, const TSharedRef<SNavigationToolTreeRow>& InRow) override;
	//~ End INavigationToolColumn
};

} // namespace UE::SequenceNavigator

#undef UE_API
