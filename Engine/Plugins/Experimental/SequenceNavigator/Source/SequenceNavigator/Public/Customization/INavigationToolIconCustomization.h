// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NavigationToolDefines.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

struct FSlateIcon;

namespace UE::SequenceNavigator
{

class FNavigationToolItem;

class INavigationToolIconCustomization
{
public:
	virtual ~INavigationToolIconCustomization() = default;

	virtual FName GetToolItemIdentifier() const = 0;

	virtual bool HasOverrideIcon(const FNavigationToolViewModelPtr& InItem) const = 0;

	virtual FSlateIcon GetOverrideIcon(const FNavigationToolViewModelPtr& InItem) const = 0;
};

} // namespace UE::SequenceNavigator
