// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Items/NavigationToolItem.h"
#include "MVVM/ViewModels/ViewModel.h"
#include "NavigationToolDefines.h"
#include "Templates/EnableIf.h"

#define UE_API SEQUENCENAVIGATOR_API

enum class EItemDropZone;
template<typename OptionalType> struct TOptional;

namespace UE::SequenceNavigator
{

class FNavigationToolItemDragDropOp;

/**
 * Base Class to Handle Dropping Navigation Tool Items into a Target Navigation Tool Item
 * @see built-in example FNavigationToolActorDropHandler
 */
class FNavigationToolItemDropHandler
	: public Sequencer::FViewModel
	//, public TSharedFromThis<FNavigationToolItemDropHandler>
{
public:
	UE_SEQUENCER_DECLARE_CASTABLE_API(UE_API, FNavigationToolItemDropHandler
		, Sequencer::FViewModel)

	TConstArrayView<FNavigationToolViewModelWeakPtr> GetItems() const
	{
		return WeakItems;
	}	

protected:
	friend FNavigationToolItemDragDropOp;

	UE_API void Initialize(const FNavigationToolItemDragDropOp& InDragDropOp);

	virtual bool IsDraggedItemSupported(const FNavigationToolViewModelPtr& InDraggedItem) const = 0;

	virtual TOptional<EItemDropZone> CanDrop(EItemDropZone InDropZone, const FNavigationToolViewModelPtr& InTargetItem) const = 0;

	virtual bool Drop(EItemDropZone InDropZone, const FNavigationToolViewModelPtr& InTargetItem) = 0;

	enum class EIterationResult
	{
		Continue,
		Break,
	};
	template<typename InItemType, typename = typename TEnableIf<TIsDerivedFrom<InItemType, INavigationToolItem>::Value>::Type>
	void ForEachItem(const TFunctionRef<EIterationResult(InItemType&)>& InFunc) const
	{
		using namespace Sequencer;

		for (const FNavigationToolViewModelWeakPtr& WeakItem : WeakItems)
		{
			const FNavigationToolViewModelPtr Item = WeakItem.Pin();
			if (!Item.IsValid())
			{
				continue;
			}

			const TViewModelPtr<InItemType> CastedItem = Item.ImplicitCast();
			if (!CastedItem)
			{
				continue;
			}

			const EIterationResult IterationResult = InFunc(*CastedItem);
			if (IterationResult == EIterationResult::Break)
			{
				break;
			}
		}
	}

	TArray<FNavigationToolViewModelWeakPtr> WeakItems;

	ENavigationToolDragDropActionType ActionType = ENavigationToolDragDropActionType::Move;
};

} // namespace UE::SequenceNavigator

#undef UE_API
