// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Handlers/NavigationToolItemDropHandler.h"

enum class EItemDropZone;

namespace UE::SequenceNavigator
{

class INavigationToolView;
class FNavigationToolView;
class FNavigationToolItemDropHandler;

/** Drag Drop Operation for Navigation Tool Items. Customized behavior can be added in via the AddDropHandler function */
class FNavigationToolItemDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FNavigationToolItemDragDropOp, FDecoratedDragDropOp)

	static TSharedRef<FNavigationToolItemDragDropOp> New(const TArray<FNavigationToolViewModelWeakPtr>& InWeakItems
		, const TSharedPtr<FNavigationToolView>& InToolView
		, const ENavigationToolDragDropActionType InActionType);

	TSharedPtr<INavigationToolView> GetToolView() const
	{
		return WeakToolView.Pin();
	}

	TConstArrayView<FNavigationToolViewModelWeakPtr> GetItems() const
	{
		return WeakItems;
	}

	ENavigationToolDragDropActionType GetActionType() const
	{
		return ActionType;
	}

	template<typename InDropHandlerType
		, typename = typename TEnableIf<TIsDerivedFrom<InDropHandlerType, FNavigationToolItemDropHandler>::Value>::Type
		, typename... InArgTypes>
	void AddDropHandler(InArgTypes&&... InArgs)
	{
		const TSharedRef<FNavigationToolItemDropHandler> DropHandler = MakeShared<InDropHandlerType>(Forward<InArgTypes>(InArgs)...);
		DropHandler->Initialize(*this);
		DropHandlers.Add(DropHandler);
	}

	FReply Drop(EItemDropZone InDropZone, const FNavigationToolViewModelPtr& InTargetItem);

	TOptional<EItemDropZone> CanDrop(const EItemDropZone InDropZone, const FNavigationToolViewModelPtr& InTargetItem) const;

protected:
	void Init(const TArray<FNavigationToolViewModelWeakPtr>& InWeakItems
		, const TSharedPtr<FNavigationToolView>& InToolView
		, const ENavigationToolDragDropActionType InActionType);

	TArray<FNavigationToolViewModelWeakPtr> WeakItems;

	TArray<TSharedRef<FNavigationToolItemDropHandler>> DropHandlers;

	TWeakPtr<INavigationToolView> WeakToolView;

	ENavigationToolDragDropActionType ActionType = ENavigationToolDragDropActionType::Move;
};

} // namespace UE::SequenceNavigator
