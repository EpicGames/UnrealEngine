// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Optional.h"
#include "MVVM/ViewModelTypeID.h"

#define UE_API SEQUENCENAVIGATOR_API

namespace UE::SequenceNavigator
{

class IIdExtension
{
public:
	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, IIdExtension)

	virtual ~IIdExtension() = default;

	virtual FText GetId() const = 0;
};

} // namespace UE::SequenceNavigator

#undef UE_API
