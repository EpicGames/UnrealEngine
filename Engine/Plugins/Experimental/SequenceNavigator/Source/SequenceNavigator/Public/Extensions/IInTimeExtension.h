// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/FrameNumber.h"
#include "MVVM/ViewModelTypeID.h"

#define UE_API SEQUENCENAVIGATOR_API

namespace UE::SequenceNavigator
{

class IInTimeExtension
{
public:
	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, IInTimeExtension)

	virtual ~IInTimeExtension() = default;

	virtual FFrameNumber GetInTime() const = 0;

	virtual void SetInTime(const FFrameNumber& InTime) = 0;
};

} // namespace UE::SequenceNavigator

#undef UE_API
