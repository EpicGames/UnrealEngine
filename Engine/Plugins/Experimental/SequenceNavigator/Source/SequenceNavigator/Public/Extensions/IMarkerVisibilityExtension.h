// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/ViewModelTypeID.h"

#define UE_API SEQUENCENAVIGATOR_API

namespace UE::SequenceNavigator
{

enum class EItemMarkerVisibility
{
	None             = 0,
	Visible          = 1 << 0,
	PartiallyVisible = 1 << 1,
};
ENUM_CLASS_FLAGS(EItemMarkerVisibility)

class IMarkerVisibilityExtension
{
public:
	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, IMarkerVisibilityExtension)

	virtual ~IMarkerVisibilityExtension() = default;

	virtual EItemMarkerVisibility GetMarkerVisibility() const = 0;

	virtual void SetMarkerVisibility(const bool bInVisible) = 0;
};

} // namespace UE::SequenceNavigator

#undef UE_API
