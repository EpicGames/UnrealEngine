// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/ViewModelTypeID.h"

#define UE_API SEQUENCENAVIGATOR_API

namespace UE::SequenceNavigator
{

enum class EItemContainsPlayhead
{
	None                      = 0,
	ContainsPlayhead          = 1 << 0,
	PartiallyContainsPlayhead = 1 << 1,
};
ENUM_CLASS_FLAGS(EItemContainsPlayhead)

class IPlayheadExtension
{
public:
	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, IPlayheadExtension)

	virtual ~IPlayheadExtension() = default;

	virtual EItemContainsPlayhead ContainsPlayhead() const = 0;
};

} // namespace UE::SequenceNavigator

#undef UE_API
