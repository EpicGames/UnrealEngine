// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISourceControlModule.h"
#include "Misc/PackageName.h"
#include "MVVM/ViewModelTypeID.h"
#include "UObject/Package.h"

#define UE_API SEQUENCENAVIGATOR_API

struct FSlateBrush;

namespace UE::SequenceNavigator
{

enum class EItemRevisionControlState
{
	None                      = 0,
	SourceControlled          = 1 << 0,
	PartiallySourceControlled = 1 << 1,
};
ENUM_CLASS_FLAGS(EItemRevisionControlState)

class IRevisionControlExtension
{
public:
	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, IRevisionControlExtension)

	IRevisionControlExtension(const UObject* const InObject)
	{
		QueueRevisionControlStatusUpdate(InObject);
	}
	virtual ~IRevisionControlExtension() = default;

	virtual EItemRevisionControlState GetRevisionControlState() const = 0;

	virtual const FSlateBrush* GetRevisionControlStatusIcon() const = 0;

	virtual FText GetRevisionControlStatusText() const = 0;

	static void QueueRevisionControlStatusUpdate(const UObject* const InObject)
	{
		if (!InObject)
		{
			return;
		}

		UPackage* const InPackage = InObject->GetPackage();
		if (!InPackage)
		{
			return;
		}

		const FString PackageFilename = FPackageName::LongPackageNameToFilename(InPackage->GetName()
			, FPackageName::GetAssetPackageExtension());
		ISourceControlModule::Get().QueueStatusUpdate(PackageFilename);
	}
};

} // namespace UE::SequenceNavigator

#undef UE_API
