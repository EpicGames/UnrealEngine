// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NavigationToolDefines.h"
#include "NavigationToolFilterBase.h"
#include "Styling/SlateBrush.h"
#include "NavigationToolBuiltInFilterParams.generated.h"

#define UE_API SEQUENCENAVIGATOR_API

class UObject;

USTRUCT()
struct FNavigationToolBuiltInFilterParams
{
	GENERATED_BODY()

	UE_API static FNavigationToolBuiltInFilterParams CreateSequenceFilter();
	UE_API static FNavigationToolBuiltInFilterParams CreateTrackFilter();
	UE_API static FNavigationToolBuiltInFilterParams CreateBindingFilter();
	UE_API static FNavigationToolBuiltInFilterParams CreateMarkerFilter();

	FNavigationToolBuiltInFilterParams() = default;

	UE_API FNavigationToolBuiltInFilterParams(const FName InFilterId
		, const TSet<UE::Sequencer::FViewModelTypeID>& InItemClasses
		, const TArray<TSubclassOf<UObject>>& InFilterClasses
		, const ENavigationToolFilterMode InMode = ENavigationToolFilterMode::MatchesType
		, const FSlateBrush* const InIconBrush = nullptr
		, const FText& InDisplayName = FText::GetEmpty()
		, const FText& InTooltipText = FText::GetEmpty()
		, const TSharedPtr<FUICommandInfo>& InToggleCommand = nullptr
		, const bool InEnabledByDefault = true
		, const EClassFlags InRequiredClassFlags = CLASS_None
		, const EClassFlags InRestrictedClassFlags = CLASS_None);

	bool HasValidFilterData() const;

	FName GetFilterId() const;

	FText GetDisplayName() const;

	FText GetTooltipText() const;

	const FSlateBrush* GetIconBrush() const;

	ENavigationToolFilterMode GetFilterMode() const;

	bool IsEnabledByDefault() const;

	TSharedPtr<FUICommandInfo> GetToggleCommand() const;

	void SetOverrideIconColor(const FSlateColor& InNewIconColor);

	bool IsValidItemClass(const UE::Sequencer::FViewModelTypeID& InClassTypeId) const;
	bool IsValidObjectClass(const UClass* const InClass) const;

	void SetFilterText(const FText& InText);

	bool PassesFilterText(const UE::SequenceNavigator::FNavigationToolViewModelPtr& InItem) const;

private:
	UPROPERTY(EditAnywhere, Category = "Filter")
	FName FilterId = NAME_None;

	TSet<UE::Sequencer::FViewModelTypeID> ItemClasses;

	UPROPERTY(EditAnywhere, Category = "Filter")
	TArray<TSubclassOf<UObject>> ObjectClasses;

	UPROPERTY(EditAnywhere, Category = "Filter")
	ENavigationToolFilterMode FilterMode = ENavigationToolFilterMode::MatchesType;

	UPROPERTY(EditAnywhere, Category = "Filter|Advanced")
	FText FilterText;

	UPROPERTY(EditAnywhere, Category = "Filter")
	FText DisplayName;

	UPROPERTY(EditAnywhere, Category = "Filter")
	FText TooltipText;

	UPROPERTY(EditAnywhere, Category = "Filter", meta=(EditCondition="bUseOverrideIcon"))
	FSlateBrush OverrideIcon;

	UPROPERTY(EditAnywhere, Category = "Filter", meta=(InlineEditConditionToggle))
	bool bUseOverrideIcon = true;

	UPROPERTY(EditAnywhere, Category = "Filter", meta=(InlineEditConditionToggle))
	bool bEnabledByDefault = true;

	TSharedPtr<FUICommandInfo> ToggleCommand;

	const FSlateBrush* IconBrush = nullptr;

	EClassFlags RequiredClassFlags = CLASS_None;
	EClassFlags RestrictedClassFlags = CLASS_None;
};

#undef UE_API
