// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/FilterBase.h"
#include "Filters/INavigationToolFilterBar.h"
#include "Filters/SequencerFilterBase.h"
#include "Items/NavigationToolItem.h"
#include "GameFramework/Actor.h"
#include "Misc/IFilter.h"
#include "NavigationToolDefines.h"
#include "NavigationToolFilterBase.generated.h"

#define UE_API SEQUENCENAVIGATOR_API

UENUM()
enum class ENavigationToolFilterMode : uint8
{
	None = 0,

	/** Navigation Tool Item type matches the Filter Type */
	MatchesType = 1 << 0,

	/** Navigation Tool Item contains an item (as a descendant/child) of that type*/
	ContainerOfType = 1 << 1,
};
ENUM_CLASS_FLAGS(ENavigationToolFilterMode);

namespace UE::SequenceNavigator
{

class FNavigationToolFilter : public FSequencerFilterBase<FNavigationToolViewModelPtr>
{
public:
	UE_API FNavigationToolFilter(INavigationToolFilterBar& InOutFilterInterface, TSharedPtr<FFilterCategory>&& InCategory = nullptr);

	//~ Begin IFilter
	virtual bool PassesFilter(const FNavigationToolViewModelPtr InItem) const override { return true; };
	//~ End IFilter

	//~ Begin FSequencerFilterBase
	UE_API virtual INavigationToolFilterBar& GetFilterInterface() const override;
	//~ End FSequencerFilterBase
};

//////////////////////////////////////////////////////////////////////////
//

/** Base filter for filtering items based on their item class */
template<typename InItemType>
class FNavigationToolFilter_ItemType : public FNavigationToolFilter
{
public:
	FNavigationToolFilter_ItemType(INavigationToolFilterBar& InOutFilterInterface, TSharedPtr<FFilterCategory> InCategory)
		: FNavigationToolFilter(InOutFilterInterface, MoveTemp(InCategory))
	{}

	//~ Begin IFilter
	virtual bool PassesFilter(const FNavigationToolViewModelPtr InItem) const override
	{
		return InItem.AsModel()->CastThis<InItemType>() != nullptr;
	}
	//~ End IFilter
};

} // namespace UE::SequenceNavigator

#undef UE_API
