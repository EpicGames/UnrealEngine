// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/FilterCollection.h"
#include "Misc/IFilter.h"
#include "NavigationToolDefines.h"

class FFilterCategory;
class ISequencerFilterBar;

namespace UE::SequenceNavigator
{

class FNavigationToolFilter;

/** Some of this classes functionality could be moved to the TFilterCollection base class so other places could take advantage */
class FNavigationToolFilterCollection : public TFilterCollection<FNavigationToolViewModelPtr>
{
public:
	FNavigationToolFilterCollection(ISequencerFilterBar& InFilterInterface);

	// @todo Maybe this should get moved in to TFilterCollection
	bool ContainsFilter(const TSharedRef<FNavigationToolFilter>& InItem) const;

	// @todo Maybe this should get moved in to TFilterCollection
	void RemoveAll();

	/**
	 * Adds the specified Filter to the collection
	 * 
	 * @param InFilter The filter object to add to the collection
	 * 
	 * @return The index in the collection at which the filter was added
	 */
	int32 Add(const TSharedRef<FNavigationToolFilter>& InFilter);

	/**
	 * Removes as many instances of the specified Filter as there are in the collection
	 * 
	 * @param InFilter The filter object to remove from the collection
	 * 
	 * @return The number of Filters removed from the collection
	 */
	int32 Remove(const TSharedRef<FNavigationToolFilter>& InFilter);

	/**
	 * Gets the filter at the specified index
	 * 
	 * @param InIndex The index of the requested filter in the ChildFilters array
	 * 
	 * @return Filter at the specified index
	 */
	TSharedRef<FNavigationToolFilter> GetFilterAtIndex(const int32 InIndex);

	/** Returns the number of Filters in the collection */
	int32 Num() const;

	bool IsEmpty() const;

	/** Sorts the filters by display string */
	void Sort();

	/** Gets all the available track filter names */
	TArray<FText> GetFilterDisplayNames() const;

	TArray<TSharedRef<FNavigationToolFilter>> GetAllFilters(const TArray<TSharedRef<FFilterCategory>>& InCategories = {}) const;

	TSet<TSharedRef<FFilterCategory>> GetCategories(const TSet<TSharedRef<FNavigationToolFilter>>* InFilters = nullptr) const;

	/** @return List of filters that match the specified category */
	TArray<TSharedRef<FNavigationToolFilter>> GetCategoryFilters(const TSharedRef<FFilterCategory>& InCategory) const;

	void ForEachFilter(const TFunctionRef<bool(const TSharedRef<FNavigationToolFilter>&)>& InFunction
		, const TArray<TSharedRef<FFilterCategory>>& InCategories = {}) const;

protected:
	/**
	 *	Called when a child filter restrictions change and broadcasts the FilterChanged delegate
	 *	for the collection
	 */
	void OnChildFilterChanged();

	ISequencerFilterBar& FilterInterface;

public:
	/**
	 * DO NOT USE DIRECTLY
	 * STL-like iterators to enable range-based for loop support.
	 */
	inline TArray<TSharedPtr<IFilter<FNavigationToolViewModelPtr>>>::RangedForIteratorType      begin()       { return ChildFilters.begin(); }
	inline TArray<TSharedPtr<IFilter<FNavigationToolViewModelPtr>>>::RangedForConstIteratorType begin() const { return ChildFilters.begin(); }
	inline TArray<TSharedPtr<IFilter<FNavigationToolViewModelPtr>>>::RangedForIteratorType      end()	      { return ChildFilters.end(); }
	inline TArray<TSharedPtr<IFilter<FNavigationToolViewModelPtr>>>::RangedForConstIteratorType end()   const { return ChildFilters.end(); }
};

} // namespace UE::SequenceNavigator
