// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NavigationToolFilterExtension.generated.h"

class FFilterCategory;

namespace UE::SequenceNavigator
{
	class FNavigationToolFilter;
	class INavigationToolFilterBar;
}

/**
 * Derive from this class to make additional filters available in the Navigation Tool.
 */
UCLASS(MinimalAPI, Abstract)
class UNavigationToolFilterExtension : public UObject
{
	GENERATED_BODY()

public:
	/**
	 * Override to add additional Navigation Tool filters.
	 * @param InOutFilterInterface The filter interface to extend
	 * @param InPreferredCategory  The preferred filter bar category to place any newly added filters to, but can be any category
	 * @param InOutFilterList      Filter list to add additional filters to
	 */
	virtual void AddFilterExtensions(UE::SequenceNavigator::INavigationToolFilterBar& InOutFilterInterface
		, const TSharedRef<FFilterCategory>& InPreferredCategory
		, TArray<TSharedRef<UE::SequenceNavigator::FNavigationToolFilter>>& InOutFilterList) const
	{}
};
