// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/CustomTextFilters.h"
#include "NavigationToolDefines.h"
#include "NavigationToolFilter_Text.h"

namespace UE::SequenceNavigator
{

class INavigationToolFilterBar;

class FNavigationToolFilter_CustomText :
	public ICustomTextFilter<FNavigationToolViewModelPtr>,
	public FNavigationToolFilter_Text
{
public:
	FNavigationToolFilter_CustomText(INavigationToolFilterBar& InFilterInterface);

	//~ Begin ICustomTextFilter
	virtual void SetFromCustomTextFilterData(const FCustomTextFilterData& InFilterData) override;
	virtual FCustomTextFilterData CreateCustomTextFilterData() const override;
	virtual TSharedPtr<FFilterBase<FNavigationToolViewModelPtr>> GetFilter() override;
	//~ End ICustomTextFilter

	//~ Begin FNavigationToolFilter
	virtual bool IsCustomTextFilter() const override;
	virtual FText GetDefaultToolTipText() const override;
	virtual TSharedPtr<FUICommandInfo> GetToggleCommand() const override;
	//~ End FNavigationToolFilter

	//~ Begin FFilterBase
	virtual FText GetDisplayName() const override;
	virtual FText GetToolTipText() const override;
	virtual FLinearColor GetColor() const override;
	//~ End FFilterBase

	//~ Begin FFilterBase
	virtual FString GetName() const override;
	//~ End FFilterBase

protected:
	FText DisplayName = FText::GetEmpty();

	FLinearColor Color = FLinearColor::White;
};

} // namespace UE::SequenceNavigator
