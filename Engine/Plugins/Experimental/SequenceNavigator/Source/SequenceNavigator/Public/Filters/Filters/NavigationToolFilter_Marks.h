// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/Filters/NavigationToolFilterBase.h"
#include "Filters/NavigationToolFilterData.h"

namespace UE::SequenceNavigator
{

class FNavigationToolFilter_Marks : public FNavigationToolFilter
{
public:
	static FString StaticName() { return TEXT("Marks"); }

	FNavigationToolFilter_Marks(INavigationToolFilterBar& InFilterInterface, TSharedPtr<FFilterCategory> InCategory = nullptr);

	//~ Begin FNavigationToolFilter
	virtual FText GetDefaultToolTipText() const override;
	virtual TSharedPtr<FUICommandInfo> GetToggleCommand() const override;
	//~ End FNavigationToolFilter

	//~ Begin FFilterBase
	virtual FText GetDisplayName() const override;
	virtual FSlateIcon GetIcon() const override;
	//~ End FFilterBase

	//~ Begin IFilter
	virtual FString GetName() const override;
	virtual bool PassesFilter(const FNavigationToolViewModelPtr InItem) const override;
	//~ End IFilter
};

} // namespace UE::SequenceNavigator
