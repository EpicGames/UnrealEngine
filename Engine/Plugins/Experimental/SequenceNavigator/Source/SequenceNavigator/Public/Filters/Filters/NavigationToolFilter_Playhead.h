// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/Filters/NavigationToolFilterBase.h"
#include "Filters/NavigationToolFilterData.h"

namespace UE::SequenceNavigator
{

class FNavigationToolFilter_Playhead : public FNavigationToolFilter
{
public:
	static FString StaticName() { return TEXT("Playhead"); }

	FNavigationToolFilter_Playhead(INavigationToolFilterBar& InFilterInterface, TSharedPtr<FFilterCategory> InCategory = nullptr);
	virtual ~FNavigationToolFilter_Playhead() override;

	//~ Begin FNavigationToolFilter
	virtual FText GetDefaultToolTipText() const override;
	virtual TSharedPtr<FUICommandInfo> GetToggleCommand() const override;
	virtual void ActiveStateChanged(const bool bInActive) override;
	//~ End FNavigationToolFilter

	//~ Begin FFilterBase
	virtual FText GetDisplayName() const override;
	virtual FSlateIcon GetIcon() const override;
	//~ End FFilterBase

	//~ Begin IFilter
	virtual FString GetName() const override;
	virtual bool PassesFilter(const FNavigationToolViewModelPtr InItem) const override;
	//~ End IFilter

protected:
	void BindEvents();
	void UnbindEvents();

	void OnPlayEvent();
	void OnStopEvent();

	void OnBeginScrubbingEvent();
	void OnEndScrubbingEvent();

	FTimerManager* GetTimerManager() const;

	void StartRefreshTimer();
	void StopRefreshTimer();

	FTimerHandle TimerHandle;

	TWeakPtr<ISequencer> WeakSequencer;
};

} // namespace UE::SequenceNavigator
