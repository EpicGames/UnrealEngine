// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NavigationToolDefines.h"
#include "NavigationToolFilterBase.h"
#include "Misc/TextFilterExpressionEvaluator.h"

enum class ESequencerTextFilterValueType : uint8;

namespace UE::SequenceNavigator
{

class FNavigationToolFilterTextExpressionContext;
class INavigationToolFilterBar;

class FNavigationToolFilter_Text : public FNavigationToolFilter
{
public:
	FNavigationToolFilter_Text(INavigationToolFilterBar& InFilterInterface);

	//~ Begin FFilterBase
	virtual FText GetDisplayName() const override;
	virtual FText GetToolTipText() const override;
	//~ End FFilterBase

	//~ Begin IFilter
	virtual FString GetName() const override;
	virtual bool PassesFilter(const FNavigationToolViewModelPtr InItem) const override;
	//~ End IFilter

	bool IsActive() const;

	FText GetRawFilterText() const;
	void SetRawFilterText(const FText& InFilterText);

	FText GetFilterErrorText() const;

	const FTextFilterExpressionEvaluator& GetTextFilterExpressionEvaluator() const;
	TArray<TSharedRef<ISequencerTextFilterExpressionContext>> GetTextFilterExpressionContexts() const;

	bool DoesTextFilterStringContainExpressionPair(const ISequencerTextFilterExpressionContext& InExpression) const;

protected:
	static bool IsTokenKey(const FExpressionToken& InToken, const TSet<FName>& InKeys);
	static bool IsTokenOperator(const FExpressionToken& InToken, const ESequencerTextFilterValueType InValueType);
	static bool IsTokenValueValid(const FExpressionToken& InToken, const ESequencerTextFilterValueType InValueType);

	/** Expression evaluator that can be used to perform complex text filter queries */
	FTextFilterExpressionEvaluator TextFilterExpressionEvaluator;

	/** Transient context data, used when calling PassesFilter. Kept around to minimize re-allocations between multiple calls to PassesFilter */
	TArray<TSharedRef<FNavigationToolFilterTextExpressionContext>> TextFilterExpressionContexts;
};

} // namespace UE::SequenceNavigator
