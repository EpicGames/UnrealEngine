// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Items/NavigationToolSequence.h"
#include "Items/NavigationToolTrack.h"
#include "NavigationToolFilterBase.h"

class FFilterCategory;
class UMovieSceneSequence;

namespace UE::SequenceNavigator
{

class INavigationToolFilterBar;

class FNavigationToolFilter_Sequence : public FNavigationToolFilter_ItemType<FNavigationToolSequence>
{
public:
	static FString StaticName() { return TEXT("Sequence"); }

	FNavigationToolFilter_Sequence(INavigationToolFilterBar& InFilterInterface, TSharedPtr<FFilterCategory> InCategory = nullptr);

	//~ Begin IFilter
	virtual FString GetName() const override { return StaticName(); }
	//~ End IFilter

	//~ Begin FFilterBase
	virtual FText GetDisplayName() const override;
	virtual FSlateIcon GetIcon() const override;
	//~ End FFilterBase

	//~ Begin FSequencerTrackFilter
	virtual FText GetDefaultToolTipText() const override;
	virtual TSharedPtr<FUICommandInfo> GetToggleCommand() const override;
	virtual bool SupportsSequence(UMovieSceneSequence* const InSequence) const override;
	//~ End FSequencerTrackFilter
};

//////////////////////////////////////////////////////////////////////////
//

class FNavigationToolFilter_Track : public FNavigationToolFilter_ItemType<FNavigationToolTrack>
{
public:
	static FString StaticName() { return TEXT("Track"); }

	FNavigationToolFilter_Track(INavigationToolFilterBar& InFilterInterface, TSharedPtr<FFilterCategory> InCategory = nullptr);

	//~ Begin IFilter
	virtual FString GetName() const override { return StaticName(); }
	//~ End IFilter

	//~ Begin FFilterBase
	virtual FText GetDisplayName() const override;
	virtual FSlateIcon GetIcon() const override;
	//~ End FFilterBase

	//~ Begin FSequencerTrackFilter
	virtual FText GetDefaultToolTipText() const override;
	virtual TSharedPtr<FUICommandInfo> GetToggleCommand() const override;
	virtual bool SupportsSequence(UMovieSceneSequence* const InSequence) const override;
	//~ End FSequencerTrackFilter
};

} // namespace UE::SequenceNavigator
