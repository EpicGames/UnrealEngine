// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/CustomTextFilters.h"
#include "Filters/ISequencerFilterBar.h"

class FFilterCategory;

namespace UE::SequenceNavigator
{

class FNavigationToolFilter;
class FNavigationToolFilter_CustomText;
struct FNavigationToolFilterData;

class INavigationToolFilterBar : public ISequencerFilterBar
{
public:
	virtual ~INavigationToolFilterBar() override = default;

	virtual TSet<TSharedRef<FFilterCategory>> GetFilterCategories(const TSet<TSharedRef<FNavigationToolFilter>>* InFilters = nullptr) const = 0;

	virtual TArray<TSharedRef<FNavigationToolFilter>> GetCommonFilters(const TArray<TSharedRef<FFilterCategory>>& InCategories = {}) const = 0;

	virtual bool IsFilterEnabled(TSharedRef<FNavigationToolFilter> InFilter) const = 0;
	virtual bool SetFilterEnabled(const TSharedRef<FNavigationToolFilter> InFilter, const bool bInEnabled, const bool bInRequestFilterUpdate = true) = 0;

	virtual bool IsFilterActive(const TSharedRef<FNavigationToolFilter> InFilter) const = 0;
	virtual bool SetFilterActive(const TSharedRef<FNavigationToolFilter>& InFilter, const bool bInActive, const bool bInRequestFilterUpdate = true) = 0;

	virtual bool AddCustomTextFilter(const TSharedRef<FNavigationToolFilter_CustomText>& InFilter, const bool bInAddToConfig) = 0;
	virtual bool RemoveCustomTextFilter(const TSharedRef<FNavigationToolFilter_CustomText>& InFilter, const bool bInAddToConfig) = 0;

	virtual TSharedPtr<FNavigationToolFilter> FindFilterByDisplayName(const FString& InFilterName) const = 0;
	virtual TSharedPtr<FNavigationToolFilter_CustomText> FindCustomTextFilterByDisplayName(const FString& InFilterName) const = 0;

	virtual FNavigationToolFilterData& GetFilterData() = 0;

	virtual void EnableCustomTextFilters(const bool bInEnable, const TArray<TSharedRef<FNavigationToolFilter_CustomText>> InExceptions = {}) = 0;

	virtual void CreateWindow_AddCustomTextFilter(const FCustomTextFilterData& InCustomTextFilterData = FCustomTextFilterData()) = 0;
	virtual void CreateWindow_EditCustomTextFilter(const TSharedPtr<FNavigationToolFilter_CustomText>& InCustomTextFilter) = 0;
};

} // namespace UE::SequenceNavigator
