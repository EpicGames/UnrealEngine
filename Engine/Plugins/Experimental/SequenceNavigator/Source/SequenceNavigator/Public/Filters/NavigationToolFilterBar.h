// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/CustomTextFilters.h"
#include "Filters/INavigationToolFilterBar.h"
#include "Filters/NavigationToolFilterData.h"
#include "Filters/SequencerTrackFilterBase.h"
#include "NavigationToolDefines.h"

class FUICommandList;
class ISequencer;
class SComboButton;
class SSequencerSearchBox;
class UNavigationToolSettings;
enum class EFilterBarLayout : uint8;

namespace UE::SequenceNavigator
{

class FNavigationTool;
class FNavigationToolFilter;
class FNavigationToolFilter_CustomText;
class FNavigationToolFilter_Text;
class FNavigationToolFilterCollection;
class FNavigationToolFilterMenu;
class INavigationToolView;
class SNavigationToolFilterBar;

enum class ENavigationToolFilterChange : uint8
{
	Enable,
	Disable,
	Activate,
	Deactivate
};

/** Holds the Sequencer track filter collection, the current text filter, and hidden/isolated lists. */
class FNavigationToolFilterBar : public INavigationToolFilterBar
{
public:
	/** An identifier shared by all filter bars, used to save and load settings common to every instance */
	static const FName SharedIdentifier;

	DECLARE_EVENT_TwoParams(ISequencerFilterBar, FNavigationToolFiltersChanged, const ENavigationToolFilterChange /*InChangeType*/, const TSharedRef<FNavigationToolFilter>& /*InFilter*/);
	DECLARE_EVENT_OneParam(ISequencerFilterBar, FOnLayoutChanged, EFilterBarLayout /** InNewLayout */);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnVisibilityChanged, bool /*bIsVisible*/)
	
	static FCustomTextFilterData DefaultNewCustomTextFilterData(const FText& InFilterString);

	FNavigationToolFilterBar(FNavigationTool& InTool);
	virtual ~FNavigationToolFilterBar() override;

	void Init();

	void BindCommands(const TSharedPtr<FUICommandList>& InBaseCommandList);

	TSharedPtr<ICustomTextFilter<FNavigationToolViewModelPtr>> CreateTextFilter();

	TSharedRef<SSequencerSearchBox> GetOrCreateSearchBoxWidget();

	TSharedRef<SNavigationToolFilterBar> GenerateWidget();

	virtual bool AreFiltersMuted() const override;
	virtual void MuteFilters(const bool bInMute) override;
	void ToggleMuteFilters();

	virtual bool CanResetFilters() const override;
	virtual void ResetFilters() override;

	FNavigationToolFiltersChanged& OnFiltersChanged() { return FiltersChangedEvent; }
	FOnLayoutChanged& OnLayoutChanged() { return LayoutChangedEvent; }
	FSimpleMulticastDelegate& OnActivationStateChanged() { return ActivationStateChangedEvent; }
	FOnVisibilityChanged& OnVisibilityChanged() { return VisibilityChangedEvent; }
	virtual FSimpleMulticastDelegate& OnRequestUpdate() override { return RequestUpdateEvent; }
	virtual FSimpleMulticastDelegate& OnTextFilterTextChanged() override { return TextFilterTextChangedEvent; }
	virtual FMuteStateChangedDelegate& OnMuteFiltersChanged() override { return MuteFiltersChangedEvent; }

	TSharedRef<FNavigationToolFilter_Text> GetTextFilter() const;
	FText GetFilterErrorText() const;

	virtual TSharedPtr<FNavigationToolFilter> FindFilterByDisplayName(const FString& InFilterName) const override;
	virtual TSharedPtr<FNavigationToolFilter_CustomText> FindCustomTextFilterByDisplayName(const FString& InFilterName) const override;

	bool HasAnyFiltersEnabled() const;

	//~ Begin ISequencerFilterBar

	virtual FName GetIdentifier() const override;

	virtual ISequencer& GetSequencer() const override;

	virtual TSharedPtr<FUICommandList> GetCommandList() const override;

	virtual FText GetTextFilterText() const;
	virtual FString GetTextFilterString() const override;
    virtual void SetTextFilterString(const FString& InText) override;

	virtual bool DoesTextFilterStringContainExpressionPair(const ISequencerTextFilterExpressionContext& InExpression) const override;

	virtual void RequestFilterUpdate() override;

	virtual void EnableAllFilters(const bool bInEnable, const TArray<FString>& InExceptionFilterNames) override;

	virtual void ActivateCommonFilters(const bool bInActivate, const TArray<FString>& InExceptionFilterNames) override;

	virtual bool AreAllEnabledFiltersActive(const bool bInActive, const TArray<FString> InExceptionFilterNames) const override;
	virtual void ActivateAllEnabledFilters(const bool bInActivate, const TArray<FString> InExceptionFilterNames) override;
	virtual void ToggleActivateAllEnabledFilters();

	virtual bool IsFilterActiveByDisplayName(const FString& InFilterName) const override;
	virtual bool IsFilterEnabledByDisplayName(const FString& InFilterName) const override;
	virtual bool SetFilterActiveByDisplayName(const FString& InFilterName, const bool bInActive, const bool bInRequestFilterUpdate = true) override;
	virtual bool SetFilterEnabledByDisplayName(const FString& InFilterName, const bool bInEnabled, const bool bInRequestFilterUpdate = true) override;

	virtual TArray<FText> GetFilterDisplayNames() const override;
	virtual TArray<FText> GetCustomTextFilterNames() const override;

	virtual int32 GetTotalDisplayNodeCount() const override;
	virtual int32 GetFilteredDisplayNodeCount() const override;

	virtual const FTextFilterExpressionEvaluator& GetTextFilterExpressionEvaluator() const override;
	virtual TArray<TSharedRef<ISequencerTextFilterExpressionContext>> GetTextFilterExpressionContexts() const override;
	virtual FString GenerateTextFilterStringFromEnabledFilters() const override;

	//~ End ISequencerFilterBar

	//~ Begin INavigationToolFilterBar
	virtual FNavigationToolFilterData& GetFilterData() override;
	//~ End INavigationToolFilterBar

	/** Active Filter Functions */

	bool AnyCommonFilterActive() const;
	virtual bool HasAnyFilterActive(const bool bCheckTextFilter = true
		, const bool bInCheckHideIsolateFilter = true
		, const bool bInCheckCommonFilters = true
		, const bool bInCheckInternalFilters = true
		, const bool bInCheckCustomTextFilters = true) const override;
	virtual bool IsFilterActive(const TSharedRef<FNavigationToolFilter> InFilter) const override;
	virtual bool SetFilterActive(const TSharedRef<FNavigationToolFilter>& InFilter, const bool bInActive, const bool bInRequestFilterUpdate = true) override;
	void ActivateCommonFilters(const bool bInActivate
		, const TArray<TSharedRef<FFilterCategory>>& InMatchCategories
		, const TArray<TSharedRef<FNavigationToolFilter>>& InExceptions);
	TArray<TSharedRef<FNavigationToolFilter>> GetActiveFilters() const;

	/** Enabled Filter Functions */

	bool HasEnabledCommonFilters() const;
	bool HasEnabledFilter(const TArray<TSharedRef<FNavigationToolFilter>>& InFilters = {}) const;
	virtual bool HasAnyFilterEnabled() const override;
	virtual bool IsFilterEnabled(const TSharedRef<FNavigationToolFilter> InFilter) const override;
	virtual bool SetFilterEnabled(const TSharedRef<FNavigationToolFilter> InFilter, const bool bInEnabled, const bool bInRequestFilterUpdate = true) override;
	void EnableFilters(const bool bInEnable
		, const TArray<TSharedRef<FFilterCategory>> InMatchCategories = {}
		, const TArray<TSharedRef<FNavigationToolFilter>> InExceptions = {});
	void ToggleFilterEnabled(const TSharedRef<FNavigationToolFilter> InFilter);
	TArray<TSharedRef<FNavigationToolFilter>> GetEnabledFilters() const;

	/** Filter Functions */

	bool HasAnyCommonFilters() const;
	bool AddFilter(const TSharedRef<FNavigationToolFilter>& InFilter);
	bool RemoveFilter(const TSharedRef<FNavigationToolFilter>& InFilter);

	virtual TArray<TSharedRef<FNavigationToolFilter>> GetCommonFilters(const TArray<TSharedRef<FFilterCategory>>& InCategories = {}) const override;

	/** Custom Text Filter Functions */

	bool AnyCustomTextFilterActive() const;
	bool HasEnabledCustomTextFilters() const;
	TArray<TSharedRef<FNavigationToolFilter_CustomText>> GetAllCustomTextFilters() const;
	virtual bool AddCustomTextFilter(const TSharedRef<FNavigationToolFilter_CustomText>& InFilter, const bool bInAddToConfig) override;
	virtual bool RemoveCustomTextFilter(const TSharedRef<FNavigationToolFilter_CustomText>& InFilter, const bool bInRemoveFromConfig) override;
	void ActivateCustomTextFilters(const bool bInActivate, const TArray<TSharedRef<FNavigationToolFilter_CustomText>> InExceptions = {});
	virtual void EnableCustomTextFilters(const bool bInEnable, const TArray<TSharedRef<FNavigationToolFilter_CustomText>> InExceptions = {}) override;
	TArray<TSharedRef<FNavigationToolFilter_CustomText>> GetEnabledCustomTextFilters() const;

	/** Filter Category Functions */

	virtual TSet<TSharedRef<FFilterCategory>> GetFilterCategories(const TSet<TSharedRef<FNavigationToolFilter>>* InFilters = nullptr) const override;
	TSet<TSharedRef<FFilterCategory>> GetConfigCategories() const;
	TSharedRef<FFilterCategory> GetClassTypeCategory() const;
	TSharedRef<FFilterCategory> GetComponentTypeCategory() const;
	TSharedRef<FFilterCategory> GetMiscCategory() const;

	/** Misc Functions */

	const FNavigationToolFilterData& FilterNodes();

	bool ShouldUpdateOnTrackValueChanged() const;

	TSharedRef<SComboButton> MakeAddFilterButton();

	bool ShouldShowFilterBarWidget() const;
	bool IsFilterBarVisible() const;
	void ToggleFilterBarVisibility();

	bool IsFilterBarLayout(const EFilterBarLayout InLayout) const;
	void SetToVerticalLayout();
	void SetToHorizontalLayout();
	void ToggleFilterBarLayout();

	virtual void CreateWindow_AddCustomTextFilter(const FCustomTextFilterData& InCustomTextFilterData = FCustomTextFilterData()) override;
	virtual void CreateWindow_EditCustomTextFilter(const TSharedPtr<FNavigationToolFilter_CustomText>& InCustomTextFilter) override;

	bool IsFilterSupported(const TSharedRef<FNavigationToolFilter>& InFilter) const;
    bool IsFilterSupported(const FString& InFilterName) const;

	virtual void OpenTextExpressionHelp() override;
	virtual void SaveCurrentFilterSetAsCustomTextFilter() override;
	virtual void CreateNewTextFilter() override;

	virtual bool DoesCommonFilterPass(const Sequencer::TViewModelPtr<Sequencer::FViewModel>& InViewModel, const FString& InFilterName) const override;

	virtual void GetCommonFilterKeySuggestions(TArray<FSequencerFilterSuggestion>& OutSuggestions) const override;
	virtual void GetCommonFilterValueSuggestions(const FString& InKey, TArray<FSequencerFilterSuggestion>& OutSuggestions) const override;

	virtual bool IsCommonFilterName(const FString& InFilterName) const override;

protected:
	virtual void GetCommonFiltersImpl(TArray<TSharedPtr<void>>& OutTypeErasedFilters) const override;

	void CreateDefaultFilters();
	void CreateCustomTextFiltersFromConfig();

	bool PassesAnyCommonFilter(const FNavigationToolViewModelPtr& InNode);
	bool PassesAllCustomTextFilters(const FNavigationToolViewModelPtr& InNode);

	TArray<TSharedRef<FNavigationToolFilter>> GetFilterList(const bool bInIncludeCustomTextFilters = false) const;

	void OnSearchTextChanged(const FText& InFilterText);
	void OnSearchTextCommitted(const FText& InFilterText, const ETextCommit::Type InCommitType);
	void OnSearchTextSaved(const FText& InFilterText);

	void BroadcastFiltersChanged(const TSharedRef<FNavigationToolFilter>& InFilter, const ENavigationToolFilterChange InChangeType);

	/** The Navigation Tool this filter bar is interacting with */
	FNavigationTool& Tool;

	TSharedRef<FUICommandList> CommandList;

	/** Global override to enable/disable all filters */
	bool bFiltersMuted = false;

	TSharedRef<FFilterCategory> ClassTypeCategory;
	TSharedRef<FFilterCategory> ComponentTypeCategory;
	TSharedRef<FFilterCategory> MiscCategory;
	TSharedRef<FFilterCategory> TransientCategory;

	TSharedRef<FNavigationToolFilterCollection> CommonFilters;

	TSharedRef<FNavigationToolFilter_Text> TextFilter;
	//TSharedRef<FSequencerTrackFilter_Selected> SelectedFilter;
	
	TArray<TSharedRef<FNavigationToolFilter_CustomText>> CustomTextFilters;

	TSharedPtr<FNavigationToolFilterMenu> FilterMenu;

	FNavigationToolFilterData FilterData;

	TWeakPtr<SSequencerSearchBox> WeakSearchBoxWidget;

	TSharedPtr<SNavigationToolFilterBar> FilterBarWidget;

	FNavigationToolFiltersChanged FiltersChangedEvent;
	FOnLayoutChanged LayoutChangedEvent;
	FSimpleMulticastDelegate ActivationStateChangedEvent;
	FOnVisibilityChanged VisibilityChangedEvent;
	FSimpleMulticastDelegate RequestUpdateEvent;
	FSimpleMulticastDelegate TextFilterTextChangedEvent;
	FMuteStateChangedDelegate MuteFiltersChangedEvent;

private:
	static int32 InstanceCount;

	/** Do not call directly! Should only be called by FilterNodes(). */
	bool FilterNodesRecursive(INavigationToolView& InToolView
		, const UNavigationToolSettings& InSettings
		, const bool bInHasActiveFilter
		, const FNavigationToolViewModelWeakPtr& InWeakStartNode);

	bool CheckFilterNameValidity(const FString& InNewFilterName, const FString& InOldFilterName, const bool bInIsEdit, FText& OutErrorText) const;

	bool TryCreateCustomTextFilter(const FCustomTextFilterData& InNewFilterData, const FString& InOldFilterName, const bool bInApply, FText& OutErrorText);
	bool TryModifyCustomTextFilter(const FCustomTextFilterData& InNewFilterData, const FString& InOldFilterName, FText& OutErrorText);
	bool TryDeleteCustomTextFilter(const FString& InFilterName, FText& OutErrorText);
	
	void HandleTextFilterChanged();
};

} // namespace UE::SequenceNavigator
