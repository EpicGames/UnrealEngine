// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Items/NavigationToolItem.h"
#include "NavigationToolDefines.h"
#include "UObject/WeakObjectPtr.h"

#define UE_API SEQUENCENAVIGATOR_API

namespace UE::SequenceNavigator
{

/** Represents a cache between nodes for a filter operation. */
struct FNavigationToolFilterData
{
	UE_API FNavigationToolFilterData(const FString& InRawFilterText);

	UE_API bool operator==(const FNavigationToolFilterData& InRhs) const;
	UE_API bool operator!=(const FNavigationToolFilterData& InRhs) const;

	UE_API void Reset();

	UE_API FString GetRawFilterText() const;

	UE_API uint32 GetDisplayNodeCount() const;
	UE_API uint32 GetTotalNodeCount() const;

	UE_API uint32 GetFilterInCount() const;
	UE_API uint32 GetFilterOutCount() const;

	UE_API void IncrementTotalNodeCount();

	UE_API void FilterInNode(const FNavigationToolViewModelWeakPtr& InWeakNode);
	UE_API void FilterOutNode(const FNavigationToolViewModelWeakPtr& InWeakNode);

	UE_API void FilterInParentChildNodes(const FNavigationToolViewModelWeakPtr& InWeakNode
		, const bool bInIncludeSelf, const bool bInIncludeParents, const bool bInIncludeChildren = false);

	UE_API void FilterInNodeWithAncestors(const FNavigationToolViewModelWeakPtr& InWeakNode);

	UE_API bool ContainsFilterInNodes(const FNavigationToolFilterData& InOtherData) const;

	UE_API bool IsFilteredIn(const FNavigationToolViewModelWeakPtr& InWeakNode) const;
	UE_API bool IsFilteredOut(const FNavigationToolViewModelWeakPtr& InWeakNode) const;

protected:
	FString RawFilterText;

	uint32 TotalNodeCount = 0;

	/** Nodes to be displayed in the UI */
	TSet<FNavigationToolViewModelWeakPtr> FilterInNodes;
};

} // namespace UE::SequenceNavigator

#undef UE_API
