// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/ISequencerTextFilterExpressionContext.h"
#include "Filters/INavigationToolFilterBar.h"
#include "Filters/SequencerTrackFilterBase.h"
#include "NavigationToolDefines.h"

#define UE_API SEQUENCENAVIGATOR_API

class UMovieScene;
class UMovieSceneSequence;

namespace UE::SequenceNavigator
{

class FNavigationToolFilterTextExpressionContext : public ISequencerTextFilterExpressionContext
{ 
public:
	UE_API FNavigationToolFilterTextExpressionContext(INavigationToolFilterBar& InFilterInterface);

	void SetFilterItem(const FNavigationToolViewModelPtr& InFilterItem);

	//~ Begin ITextFilterExpressionContext

	UE_API virtual bool TestBasicStringExpression(const FTextFilterString& InValue
		, const ETextFilterTextComparisonMode InTextComparisonMode) const override;

	UE_API virtual bool TestComplexExpression(const FName& InKey
		, const FTextFilterString& InValue
		, const ETextFilterComparisonOperation InComparisonOperation
		, const ETextFilterTextComparisonMode InTextComparisonMode) const override;

	//~ End ITextFilterExpressionContext

protected:
	UE_API UMovieSceneSequence* GetFocusedMovieSceneSequence() const;
	UE_API UMovieScene* GetFocusedGetMovieScene() const;

	INavigationToolFilterBar& FilterInterface;

	FNavigationToolViewModelWeakPtr WeakFilterItem;
};

} // namespace UE::SequenceNavigator

#undef UE_API
