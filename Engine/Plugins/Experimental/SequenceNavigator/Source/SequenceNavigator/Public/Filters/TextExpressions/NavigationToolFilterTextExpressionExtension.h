// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NavigationToolFilterTextExpressionExtension.generated.h"

class FNavigationToolFilterTextExpressionContext;

namespace UE::SequenceNavigator
{

class INavigationToolFilterBar;
class FNavigationToolFilterTextExpressionContext;

}

/**
 * Derive from this class to make additional track filter text expressions available in Sequencer.
 */
UCLASS(MinimalAPI, Abstract)
class UNavigationToolFilterTextExpressionExtension : public UObject
{
	GENERATED_BODY()

public:
	/**
	 * Override to add additional Sequencer track filter text expressions.
	 * @param InOutFilterInterface The filter interface to extend
	 * @param InOutExpressionList  Expression list to add additional text expressions to
	 */
	virtual void AddFilterTextExpressionExtensions(UE::SequenceNavigator::INavigationToolFilterBar& InOutFilterInterface
		, TArray<TSharedRef<UE::SequenceNavigator::FNavigationToolFilterTextExpressionContext>>& InOutExpressionList) const
	{}
};
