// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Containers/ContainersFwd.h"
#include "Containers/StringFwd.h"
#include "Delegates/Delegate.h"
#include "ItemProxies/INavigationToolItemProxyFactory.h"
#include "ItemProxies/NavigationToolItemProxyRegistry.h"
#include "Items/NavigationToolItem.h"
#include "Items/NavigationToolItemId.h"
#include "Items/NavigationToolItemProxy.h"
#include "Modules/ModuleManager.h"
#include "MVVM/ViewModelPtr.h"
#include "NavigationToolDefines.h"
#include "NavigationToolExtender.h"
#include "Providers/NavigationToolProvider.h"
#include "SequencerCoreFwd.h"
#include "Templates/SharedPointer.h"

class FUICommandList;
class ISequencer;
template<typename OptionalType> struct TOptional;

namespace UE::SequenceNavigator
{

class INavigationToolItemAction;
class INavigationToolView;
struct FNavigationToolItemId;

/** 
 * The Navigation Tool Object that is commonly instanced once per Sequencer
 * (unless for advanced use where there are different Navigation Tool instances with different item ordering and behaviors).
 * This is the object that dictates core Navigation Tool behavior like how items are sorted, which items are allowed, etc.
 * Views are the objects that take this core behavior and show a part of it (e.g. through filters).
 */
class INavigationTool : public TSharedFromThis<INavigationTool>
{
public:
	virtual ~INavigationTool() = default;

	DECLARE_MULTICAST_DELEGATE(FOnToolLoaded);
	virtual FOnToolLoaded& GetOnToolLoaded() = 0;

	virtual TSharedPtr<ISequencer> GetSequencer() const = 0;

	virtual bool IsToolTabVisible() const = 0;
	virtual void ShowHideToolTab(const bool bInVisible) = 0;
	virtual void ToggleToolTabVisible() = 0;

	virtual void ForEachProvider(const TFunction<bool(const TSharedRef<FNavigationToolProvider>& InToolProvider)>& InPredicate) const = 0;

	/** Gets the Command List that the Navigation Tool Views will use to append their Command Lists to */
	virtual TSharedPtr<FUICommandList> GetBaseCommandList() const = 0;

	/** Register a new Navigation Tool View to the Navigation Tool to the given id, replacing the old view that was bound to the given id */
	virtual TSharedPtr<INavigationToolView> RegisterToolView(const int32 InToolViewId) = 0;

	/** Gets the Navigation Tool View bound to the given id */
	virtual TSharedPtr<INavigationToolView> GetToolView(const int32 InToolViewId) const = 0;

	/** Gets the outliner view that was most recently used (i.e. called FNavigationTool::UpdateRecentToolViews) */
	virtual TSharedPtr<INavigationToolView> GetMostRecentToolView() const = 0;

	/** Instantiates a new Item and automatically registers it to the Navigation Tool */
	template<typename InItemType, typename = typename TEnableIf<TIsDerivedFrom<InItemType, INavigationToolItem>::IsDerived>::Type, typename ...InArgTypes>
	FNavigationToolViewModelPtr FindOrAdd(const TSharedRef<FNavigationToolProvider>& InProvider, InArgTypes&&... InArgs)
	{
		const Sequencer::TViewModelPtr<InItemType> Item = MakeShared<InItemType>(*this, Forward<InArgTypes>(InArgs)...);
		Item->SetProvider(InProvider);

		// If an existing item already exists and has a valid state, use that and forget about the newly created
		const FNavigationToolItemId ItemId = Item->GetItemId();
		const FNavigationToolViewModelPtr ExistingItem = FindItem(ItemId);
		if (ExistingItem.IsValid() && ExistingItem->IsItemValid() && ExistingItem.AsModel()->IsA<InItemType>())
		{
			return ExistingItem;
		}

		if (Item->IsAllowedInTool())
		{
			RegisterItem(Item);
		}

		return Item;
	}

	/** Tries to find the Item Proxy Factory for the given Item Proxy Type Name */
	template<typename InItemProxyType, typename = typename TEnableIf<TIsDerivedFrom<InItemProxyType, FNavigationToolItemProxy>::IsDerived>::Type>
	INavigationToolItemProxyFactory* GetItemProxyFactory() const
	{
		// First look for the Registry in Navigation Tool
		if (INavigationToolItemProxyFactory* Factory = GetItemProxyRegistry().GetItemProxyFactory<InItemProxyType>())
		{
			return Factory;
		}

		// Fallback to finding the Factory in the Module if the Navigation Tool did not find it
		return FNavigationToolExtender::GetItemProxyRegistry().GetItemProxyFactory<InItemProxyType>();
	}

	/**
	 * Tries to get the Item Proxy Factory for the given Item Proxy type, first trying the Navigation Tool Registry then the Module's
	 * then returns an existing item proxy created via the factory, or creates one if there's no existing item proxy
	 * @returns the Item Proxy created by the Factory. Can be null if no factory was found or if the factory intentionally returns null
	 */
	template<typename InItemProxyType, typename = typename TEnableIf<TIsDerivedFrom<InItemProxyType, FNavigationToolItemProxy>::IsDerived>::Type, typename ...InArgTypes>
	TSharedPtr<FNavigationToolItemProxy> GetOrCreateItemProxy(const FNavigationToolViewModelPtr& InParentItem, InArgTypes&&... InArgs)
	{
		if (!InParentItem.IsValid() || !InParentItem->IsAllowedInTool())
		{
			return nullptr;
		}

		INavigationToolItemProxyFactory* const Factory = GetItemProxyFactory<InItemProxyType>();
		if (!Factory)
		{
			return nullptr;
		}

		TSharedPtr<FNavigationToolItemProxy> OutItemProxy;
		if (const FNavigationToolViewModelPtr ExistingItemProxy = FindItem(FNavigationToolItemId(InParentItem, *Factory)))
		{
			check(ExistingItemProxy.AsModel()->IsA<FNavigationToolItemProxy>());
			ExistingItemProxy->SetParent(InParentItem);
			OutItemProxy = ExistingItemProxy.ImplicitCast();
		}
		else
		{
			OutItemProxy = Factory->CreateItemProxy(*this, InParentItem);
		}
		RegisterItem(OutItemProxy);
		return OutItemProxy;
	}

	/** Gathers the Type Names of all the Item Proxies that are registered both in the outliner proxy registry and the module's */
	virtual TArray<FName> GetRegisteredItemProxyTypeNames() const = 0;

	/**
	 * Determines whether the given actor can be presented in the Navigation Tool, at all.
	 * This is a permanent check unlike filters that are temporary.
	 */
	virtual bool IsObjectAllowedInTool(const UObject* const InObject) const = 0;

	/** Registers the given Item, replacing the old one. */
	virtual void RegisterItem(const FNavigationToolViewModelPtr& InItem) = 0;

	/** Unregisters the Item having the given ItemId */
	virtual void UnregisterItem(const FNavigationToolItemId& InItemId) = 0;

	/** Ensures that the next time Refresh is called in tick, Refresh will be called */
	virtual void RequestRefresh() = 0;

	/**
	 * Flushes the Pending Actions from the Queue while also updating the state of the Navigation Tool.
	 * Calling it directly is forcing it to happen.
	 * If a refresh is needed it will be called on the next tick automatically.
	 */
	virtual void Refresh() = 0;

	/** Gets the tree root item of the Navigation Tool */
	virtual FNavigationToolViewModelWeakPtr GetTreeRoot() const = 0;

	/**
	 * Finds the registered item that has the given Id
	 * @returns The item with the given Id, or null if the item does not exist or was not registered to the Navigation Tool
	 */
	virtual FNavigationToolViewModelPtr FindItem(const FNavigationToolItemId& InItemId) const = 0;

	/**
	 * Attempts to find all items associated with a Sequencer outliner view model
	 * @param InSequencerOutlinerViewModel The Sequencer view model to find the items for
	 * @return The list of Navigation Tool items found
	 */
	virtual TArray<FNavigationToolViewModelPtr> TryFindItems(const Sequencer::FViewModelPtr& InSequencerOutlinerViewModel) const = 0;

	/**
 	 * Adds or Removes the Ignore Notify Flags to prevent certain actions from automatically happening when they're triggered
 	 * @param InFlag the ignore flag to add or remove
 	 * @param bInIgnore whether to add (true) or remove (false) the flag
 	 */
	virtual void SetIgnoreNotify(const ENavigationToolIgnoreNotifyFlags InFlag, const bool bInIgnore) = 0;

	/** Called when the Sequencer selection has changed */
	virtual void OnSequencerSelectionChanged() = 0;

	/** Returns the currently selected items in the most recent outliner view (since this list can vary between outliner views) */
	virtual TArray<FNavigationToolViewModelWeakPtr> GetSelectedItems(const bool bInNormalizeToTopLevelSelections = false) const = 0;

	/**
	 * Selects the given Items on all Navigation Tool Views
	 * @param InWeakItems the items to select
	 * @param InFlags how the items should be selected (appended, notify of selections, etc)
	 */
	virtual void SelectItems(const TArray<FNavigationToolViewModelWeakPtr>& InWeakItems
		, const ENavigationToolItemSelectionFlags InFlags = ENavigationToolItemSelectionFlags::SignalSelectionChange) const = 0;

	/**
	 * Clears the Item Selection from all Navigation Tool Views
	 * @param bInSignalSelectionChange whether to notify the change in selection
	 */
	virtual void ClearItemSelection(const bool bInSignalSelectionChange) const = 0;

	/** Pairs the Item with the given color name, overriding the inherited color if different */
	virtual void SetItemColor(const FNavigationToolViewModelPtr& InItem, const FColor& InColor) = 0;

	/** Removes the Color pairing of the given Item (can still have an inherited color though) */
	virtual void RemoveItemColor(const FNavigationToolViewModelPtr& InItem) = 0;

	/**
	 * Gets the color related to the Item
	 * @param InItem the item to query
	 * @param bRecurseParent whether to get the color of the parent (recursively) if the given item does not have a color by itself
	 * @returns the matching color pair or unset if item is invalid or no color could be found.
	 */
	virtual TOptional<FColor> FindItemColor(const FNavigationToolViewModelPtr& InItem, bool bRecurseParent = true) const = 0;

	/**
	 * Instantiates a new item action without adding it to the Pending Actions Queue.
	 * This should only be used directly when planning to enqueue multiple actions.
	 * @see FNavigationTool::EnqueueItemActions
	 */
	template<typename InItemActionType, typename = typename TEnableIf<TIsDerivedFrom<InItemActionType, INavigationToolItemAction>::IsDerived>::Type, typename ...InArgTypes>
	TSharedRef<InItemActionType> NewItemAction(InArgTypes&&... InArgs)
	{
		return MakeShared<InItemActionType>(Forward<InArgTypes>(InArgs)...);
	}

	/**
	 * Instantiates a single new item action and immediately adds it to the Pending Actions Queue.
	 * Ideal for when dealing with a single action.
	 * For multiple actions use FNavigationTool::EnqueueItemActions.
	 */
	template<typename InItemActionType, typename = typename TEnableIf<TIsDerivedFrom<InItemActionType, INavigationToolItemAction>::IsDerived>::Type, typename ...InArgTypes>
	void EnqueueItemAction(InArgTypes&&... InArgs)
	{
		EnqueueItemActions({ NewItemAction<InItemActionType>(Forward<InArgTypes>(InArgs)...) });
	}

	/** Adds the given actions to the Pending Action Queue */
	virtual void EnqueueItemActions(TArray<TSharedPtr<INavigationToolItemAction>>&& InItemActions) noexcept = 0;

	/** Called when an Item has been Renamed */
	virtual void NotifyToolItemRenamed(const FNavigationToolViewModelPtr& InItem) = 0;

	/** Called when an Item has been deleted */
	virtual void NotifyToolItemDeleted(const FNavigationToolViewModelPtr& InItem) = 0;

protected:
	virtual const FNavigationToolItemProxyRegistry& GetItemProxyRegistry() const = 0;
};

} // namespace UE::SequenceNavigator
