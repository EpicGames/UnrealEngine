// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Optional.h"
#include "NavigationToolDefines.h"
#include "Templates/SharedPointer.h"

class FDragDropEvent;
class FReply;
class ISequencer;
class SWidget;
enum class EItemDropZone;
struct FGeometry;
struct FPointerEvent;
enum class ENavigationToolItemViewMode : uint8;

namespace UE::SequenceNavigator
{

class INavigationTool;
class INavigationToolColumn;

class INavigationToolView : public TSharedFromThis<INavigationToolView>
{
public:
	virtual ~INavigationToolView() = default;

	virtual TSharedPtr<INavigationTool> GetOwnerTool() const  = 0;

	/** Returns the Navigation Tool Widget. Can be null widget */
	virtual TSharedPtr<SWidget> GetToolWidget() const = 0;

	virtual TSharedPtr<ISequencer> GetSequencer() const = 0;

	/** Marks the Navigation Tool View to be refreshed on Next Tick */
	virtual void RequestRefresh() = 0;

	DECLARE_MULTICAST_DELEGATE(FOnToolViewRefreshed);
	virtual FOnToolViewRefreshed& GetOnToolViewRefreshed() = 0;

	virtual void SetKeyboardFocus() = 0;

	virtual ENavigationToolItemViewMode GetItemDefaultViewMode() const = 0;

	virtual ENavigationToolItemViewMode GetItemProxyViewMode() const = 0;

	virtual bool IsColumnVisible(const TSharedPtr<INavigationToolColumn>& InColumn) const = 0;

	virtual TSharedRef<SWidget> GetColumnMenuContent(const FName InColumnId) = 0;

	virtual void GetChildrenOfItem(const FNavigationToolViewModelWeakPtr InWeakItem
		, TArray<FNavigationToolViewModelWeakPtr>& OutWeakChildren) const = 0;

	/**
	 * Gets the children of a given item. Can recurse if the immediate child is hidden (the children of these hidden items should still be given the opportunity to show up)
	 * @param InWeakItem The item to get the children of
	 * @param OutWeakChildren The visible children in the give view mode
	 * @param InViewMode The view mode(s) that the children should support to be added to OutChildren
	 * @param InWeakRecursionDisallowedItems The items where recursion should not be performed
	 */
	virtual void GetChildrenOfItem(const FNavigationToolViewModelWeakPtr InWeakItem
		, TArray<FNavigationToolViewModelWeakPtr>& OutWeakChildren
		, const ENavigationToolItemViewMode InViewMode
		, const TSet<FNavigationToolViewModelWeakPtr>& InWeakRecursionDisallowedItems) const = 0;

	/** Whether the given item is explicitly marked as Read-only in the Navigation Tool */
	virtual bool IsItemReadOnly(const FNavigationToolViewModelWeakPtr& InWeakItem) const = 0;

	/** Selection State */

	/** Whether the given item can be selected in this Navigation Tool View / Widget */
	virtual bool CanSelectItem(const FNavigationToolViewModelWeakPtr& InWeakItem) const = 0;

	/** Selects the item in this Navigation Tool View */
	virtual void SelectItems(TArray<FNavigationToolViewModelWeakPtr> InWeakItems
		, const ENavigationToolItemSelectionFlags InFlags = ENavigationToolItemSelectionFlags::SignalSelectionChange) = 0;

	/** Clears the currently selected items in the Navigation Tool View */
	virtual void ClearItemSelection(const bool bInSignalSelectionChange = true) = 0;

	/** Whether the current item is selected in this Navigation Tool View */
	virtual bool IsItemSelected(const FNavigationToolViewModelWeakPtr& InWeakItem) const = 0;

	/** Gets the currently selected items in the Tree View */
	virtual TArray<FNavigationToolViewModelWeakPtr> GetSelectedItems() const = 0;

	/** Drag Drop */
#if 1==2
	/** Called a drag is attempted for the selected items in view */
	virtual FReply OnDragDetected(const FGeometry& InGeometry
		, const FPointerEvent& InPointerEvent
		, const FNavigationToolViewModelWeakPtr InWeakTargetItem) = 0;

	/** Processes the drag and drop for the given item if valid, else it will process it for the root item */
	virtual FReply OnDrop(const FDragDropEvent& InDragDropEvent
		, const EItemDropZone InDropZone
		, const FNavigationToolViewModelWeakPtr InWeakTargetItem) = 0;

	/** Determines whether the drag and drop can be processed by the given item */
	virtual TOptional<EItemDropZone> OnCanDrop(const FDragDropEvent& InDragDropEvent
		, const EItemDropZone InDropZone
		, const FNavigationToolViewModelWeakPtr InWeakTargetItem) const = 0;
#endif
	/** Expansion State */

	virtual bool IsItemExpanded(const FNavigationToolViewModelWeakPtr& InWeakItem
		, const bool bInUseFilter = true) const = 0;

	/** Expands / Collapses the given item */
	virtual void SetItemExpansion(const FNavigationToolViewModelWeakPtr& InWeakItem
		, const bool bInExpand, const bool bInUseFilter = true) = 0;

	/** Expands / Collapses the given item and its children recursively */
	virtual void SetItemExpansionRecursive(const FNavigationToolViewModelWeakPtr InWeakItem, const bool bInExpand) = 0;

	/** Change the expansion state of the parents (recursively) of the given item */
	virtual void SetParentItemExpansions(const FNavigationToolViewModelWeakPtr& InWeakItem, const bool bInExpand) = 0;

	/** @return True if all items in the tool tree view can be expanded */
	virtual bool CanExpandAll() const = 0;

	/** Expands all items in the tool tree view */
	virtual void ExpandAll() = 0;

	/** @return True if all items in the tool tree view can be collapsed */
	virtual bool CanCollapseAll() const = 0;

	/** Collapses all items in the tool tree view */
	virtual void CollapseAll() = 0;
};

} // namespace UE::SequenceNavigator
