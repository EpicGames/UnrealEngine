// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/ICastable.h"
#include "MVVM/ViewModels/ViewModel.h"

#define UE_API SEQUENCENAVIGATOR_API

class UObject;

namespace UE::SequenceNavigator
{

class FNavigationTool;

/**
 * Interface class for an action in the Navigation Tool (e.g. Add/Delete/Move tree item)
 */ 
class INavigationToolItemAction
	: public Sequencer::FViewModel
{
public:
	UE_SEQUENCER_DECLARE_CASTABLE_API(UE_API, INavigationToolItemAction
		, Sequencer::FViewModel)

	/** Determines whether the given action modifies its objects and should transact */
	virtual bool ShouldTransact() const { return false; }

	/** The action to execute on the given Navigation Tool */
	virtual void Execute(FNavigationTool& InTool) = 0;

	/** Replace any objects that might be held in this action that has been killed and replaced by a new object (e.g. BP Components) */
	virtual void OnObjectsReplaced(const TMap<UObject*, UObject*>& InReplacementMap, const bool bInRecursive) = 0;
};

} // namespace UE::SequenceNavigator

#undef UE_API
