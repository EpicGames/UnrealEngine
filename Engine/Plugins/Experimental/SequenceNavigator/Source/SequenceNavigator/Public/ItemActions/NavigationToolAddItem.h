// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "INavigationToolItemAction.h"
#include "Items/NavigationToolItemParameters.h"

#define UE_API SEQUENCENAVIGATOR_API

namespace UE::SequenceNavigator
{

class FNavigationTool;

/**
 * Item action responsible for adding an item to the tree under a given optional parent.
 * If Parent is null, it is added as a Top Level Item.
 */
class FNavigationToolAddItem
		: public INavigationToolItemAction
{
public:
	UE_SEQUENCER_DECLARE_CASTABLE_API(UE_API, FNavigationToolAddItem
		, INavigationToolItemAction)

	UE_API FNavigationToolAddItem(const FNavigationToolAddItemParams& InAddItemParams);
	virtual ~FNavigationToolAddItem() = default;

	//~ Begin INavigationToolAction
	UE_API virtual bool ShouldTransact() const override;
	UE_API virtual void Execute(FNavigationTool& InTool) override;
	UE_API virtual void OnObjectsReplaced(const TMap<UObject*, UObject*>& InReplacementMap, bool bRecursive) override;
	//~ End INavigationToolAction

protected:
	FNavigationToolAddItemParams AddParams;
};

} // namespace UE::SequenceNavigator

#undef UE_API
