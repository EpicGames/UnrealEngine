// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "INavigationToolItemAction.h"
#include "Items/NavigationToolItemParameters.h"

#define UE_API SEQUENCENAVIGATOR_API

namespace UE::SequenceNavigator
{

class FNavigationTool;

/**
 * Item Action responsible for removing/unregistering items from the tree
 */
class FNavigationToolRemoveItem
	: public INavigationToolItemAction
{
public:
	UE_SEQUENCER_DECLARE_CASTABLE_API(UE_API, FNavigationToolRemoveItem
		, INavigationToolItemAction)

	FNavigationToolRemoveItem(const FNavigationToolRemoveItemParams& InRemoveItemParams);
	virtual ~FNavigationToolRemoveItem() = default;

	//~ Begin INavigationToolAction
	virtual void Execute(FNavigationTool& InTool) override;
	virtual void OnObjectsReplaced(const TMap<UObject*, UObject*>& InReplacementMap, const bool bInRecursive) override;
	//~ End INavigationToolAction

protected:
	FNavigationToolRemoveItemParams RemoveParams;
};

} // namespace UE::SequenceNavigator

#undef UE_API
