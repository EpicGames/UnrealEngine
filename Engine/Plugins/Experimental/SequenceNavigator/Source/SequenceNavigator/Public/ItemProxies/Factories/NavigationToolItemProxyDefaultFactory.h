// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ItemProxies/INavigationToolItemProxyFactory.h"
#include "Items/NavigationToolItemProxy.h"

namespace UE::SequenceNavigator
{

/**
 * Default Template Item Proxy Factory classes to create the Item Proxy without having to write it out for all classes 
 * that don't need special behavior or custom constructors
 */
template<typename InItemProxyType, uint32 InItemProxyPriority = 0
	, typename = typename TEnableIf<TIsDerivedFrom<InItemProxyType, FNavigationToolItemProxy>::IsDerived>::Type>
class TNavigationToolItemProxyDefaultFactoryBase : public INavigationToolItemProxyFactory
{
public:
	virtual FName GetItemProxyTypeName() const override
	{
		return Sequencer::TAutoRegisterViewModelTypeID<InItemProxyType>::GetTypeTable().GetTypeName();
	}

protected:
	template<typename... InArgTypes>
	TSharedRef<FNavigationToolItemProxy> DefaultCreateItemProxy(InArgTypes&&... InArgs)
	{
		const TSharedRef<FNavigationToolItemProxy> ItemProxy = MakeShared<InItemProxyType>(Forward<InArgTypes>(InArgs)...);
		ItemProxy->SetPriority(InItemProxyPriority);
		return ItemProxy;
	}
};

template<typename InItemProxyType, uint32 InItemProxyPriority = 0
	, typename = typename TEnableIf<TIsDerivedFrom<InItemProxyType, FNavigationToolItemProxy>::IsDerived>::Type>
class TNavigationToolItemProxyDefaultFactory : public TNavigationToolItemProxyDefaultFactoryBase<InItemProxyType, InItemProxyPriority>
{
public:
	virtual TSharedPtr<FNavigationToolItemProxy> CreateItemProxy(INavigationTool& InTool, const FNavigationToolViewModelPtr& InParentItem) override
	{
		return this->DefaultCreateItemProxy(InTool, InParentItem);
	}
};

} // namespace UE::SequenceNavigator
