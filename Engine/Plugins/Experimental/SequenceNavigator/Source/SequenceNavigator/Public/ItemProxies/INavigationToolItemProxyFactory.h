// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Items/NavigationToolItem.h"
#include "NavigationToolDefines.h"
#include "Templates/SharedPointer.h"

class FName;

namespace UE::SequenceNavigator
{

class FNavigationToolItemProxy;
class INavigationTool;

/**
 * Item Proxy Factories are the classes that instance or get the existing Navigation Tool Item Proxies for a given Item
 * @see INavigationToolItemProxy
 */
class INavigationToolItemProxyFactory
{
public:
	virtual ~INavigationToolItemProxyFactory() = default;

	/** Gets the Type Name of the Item Proxy the Factory creates */
	virtual FName GetItemProxyTypeName() const = 0;

	/** Returns a newly created instance of the Relevant Item Proxy if successful */
	virtual TSharedPtr<FNavigationToolItemProxy> CreateItemProxy(INavigationTool& InTool
		, const FNavigationToolViewModelPtr& InParentItem) = 0;
};

} // namespace UE::SequenceNavigator
