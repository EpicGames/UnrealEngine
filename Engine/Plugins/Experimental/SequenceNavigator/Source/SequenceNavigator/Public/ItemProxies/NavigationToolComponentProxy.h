// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Items/NavigationToolItemProxy.h"

#define UE_API SEQUENCENAVIGATOR_API

namespace UE::SequenceNavigator
{

class FNavigationToolComponentProxy
	: public FNavigationToolItemProxy
{
public:
	UE_SEQUENCER_DECLARE_CASTABLE_API(UE_API, FNavigationToolComponentProxy
		, FNavigationToolItemProxy)

	UE_API FNavigationToolComponentProxy(INavigationTool& InTool, const FNavigationToolViewModelPtr& InParentItem);

	//~ Begin INavigationToolItem
	UE_API virtual FText GetDisplayName() const override;
	UE_API virtual FSlateIcon GetIcon() const override;
	UE_API virtual FText GetIconTooltipText() const override;
	UE_API virtual ENavigationToolItemViewMode GetSupportedViewModes(const INavigationToolView& InToolView) const override;
	//~ End INavigationToolItem

	//~ Begin FNavigationToolItemProxy
	UE_API virtual void GetProxiedItems(const FNavigationToolViewModelPtr& InParent
		, TArray<FNavigationToolViewModelWeakPtr>& OutWeakChildren, const bool bInRecursive) override;
	//~ End FNavigationToolItemProxy
};

} // namespace UE::SequenceNavigator

#undef UE_API
