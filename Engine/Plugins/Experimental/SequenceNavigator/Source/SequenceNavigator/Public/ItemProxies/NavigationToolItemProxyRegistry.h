// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "INavigationToolItemProxyFactory.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Factories/NavigationToolItemProxyDefaultFactory.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

namespace UE::SequenceNavigator
{

/**
 * Handles registering a Navigation Tool item type with a Navigation Tool item proxy factory that creates the respective INavigationToolItemProxy
 * @see INavigationToolItemProxy
 */
class FNavigationToolItemProxyRegistry
{
public:
	template<typename InItemProxyFactoryType
		, typename = typename TEnableIf<TIsDerivedFrom<InItemProxyFactoryType, INavigationToolItemProxyFactory>::IsDerived>::Type
		, typename... InArgTypes>
	void RegisterItemProxyFactory(InArgTypes&&... InArgs)
	{
		const TSharedRef<INavigationToolItemProxyFactory> Factory = MakeShared<InItemProxyFactoryType>(Forward<InArgTypes>(InArgs)...);
		const FName ItemProxyTypeName = Factory->GetItemProxyTypeName();
		ItemProxyFactories.Add(ItemProxyTypeName, Factory);
	}

	/** Registers an Item Proxy Type with the Default Factory */
	template<typename InItemProxyType
		, uint32 InItemProxyPriority = 0
		, typename InType = typename TEnableIf<TIsDerivedFrom<InItemProxyType, FNavigationToolItemProxy>::IsDerived>::Type
		, typename... InArgTypes>
	void RegisterItemProxyWithDefaultFactory()
	{
		RegisterItemProxyFactory<TNavigationToolItemProxyDefaultFactory<InItemProxyType, InItemProxyPriority>>();
	}

	/** Unregisters the given Item Type from having an Item Proxy Factory */
	template<typename InItemProxyType, typename = typename TEnableIf<TIsDerivedFrom<InItemProxyType, FNavigationToolItemProxy>::IsDerived>::Type>
	void UnregisterItemProxyFactory()
	{
		ItemProxyFactories.Remove(Sequencer::TAutoRegisterViewModelTypeID<InItemProxyType>::GetTypeTable().GetTypeName());
	}

	void UnregisterItemProxyFactory(FName InItemProxyTypeName)
	{
		ItemProxyFactories.Remove(InItemProxyTypeName);
	}

	/** Unregisters all the Item Proxy Factories for this Instance */
	void UnregisterAllItemProxyFactories()
	{
		ItemProxyFactories.Empty();
	}

	/** Gets the Item Proxy Factory for the given Item Proxy Type Name. Returns nullptr if not found */
	INavigationToolItemProxyFactory* GetItemProxyFactory(const FName InItemProxyTypeName) const
	{
		if (const TSharedRef<INavigationToolItemProxyFactory>* const FoundFactory = ItemProxyFactories.Find(InItemProxyTypeName))
		{
			return &(FoundFactory->Get());
		}
		return nullptr;
	}

	/** Gets the Item Proxy Factory if it was registered with the Item Proxy Type Name. Returns nullptr if not found */
	template<typename InItemProxyType, typename = typename TEnableIf<TIsDerivedFrom<InItemProxyType, FNavigationToolItemProxy>::IsDerived>::Type>
	INavigationToolItemProxyFactory* GetItemProxyFactory() const
	{
		return GetItemProxyFactory(InItemProxyType::ID.GetTypeTable()->GetTypeName());
	}

	/** Gets all the Item Proxy Type Names that exist in this registry */
	void GetRegisteredItemProxyTypeNames(TSet<FName>& OutItemProxyTypeNames) const
	{
		ItemProxyFactories.GetKeys(OutItemProxyTypeNames);
	}

private:
	/** Map of the item proxy type name and its item proxy factory */
	TMap<FName, TSharedRef<INavigationToolItemProxyFactory>> ItemProxyFactories;
};

} // namespace UE::SequenceNavigator
