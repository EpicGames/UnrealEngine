// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "EditorSequenceNavigationDefs.h"
#include "MVVM/ViewModelTypeID.h"
#include "NavigationToolDefines.h"
#include "NavigationToolSettings.h"

#define UE_API SEQUENCENAVIGATOR_API

class FDragDropEvent;
class FReply;
class SWidget;
class UObject;
enum class EItemDropZone;
struct FLinearColor;
struct FSlateBrush;
struct FSlateColor;
struct FSlateIcon;

namespace UE::SequenceNavigator
{

class FNavigationToolItemProxy;
class FNavigationToolProvider;
class FNavigationToolScopedSelection;
class INavigationTool;
class INavigationToolView;
class SNavigationToolTreeRow;
struct FNavigationToolAddItemParams;
struct FNavigationToolItemId;
struct FNavigationToolRemoveItemParams;

/*
 * An Navigation Tool Item is the class that represents a single element (i.e. node) in the Navigation Tool tree.
 * This can be an item that represents an object (e.g. actor, component) or a folder, or something else.
*/
class INavigationToolItem
{
	using IndexType = TArray<INavigationToolItem>::SizeType;

public:
	UE_SEQUENCER_DECLARE_VIEW_MODEL_TYPE_ID_API(UE_API, INavigationToolItem)

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnExpansionChanged, const TSharedPtr<INavigationToolView>&, bool);
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnRenameAction, ENavigationToolRenameAction, const TSharedPtr<INavigationToolView>&);

	virtual ~INavigationToolItem() = default;

	/** Determines whether the Item properties are in a valid state */
	virtual bool IsItemValid() const = 0;

	/** Gets the UObject that this item represents. May be null due if no item association or multiple. */
	virtual UObject* GetItemObject() const { return nullptr; }

	/** Gets the module provider responsible for the creation of this item */
	virtual TSharedPtr<FNavigationToolProvider> GetProvider() const = 0;

	/** Gets the providers saved state */
	virtual FNavigationToolSaveState* GetProviderSaveState() const = 0;

	/** Used to signal the Scoped Selection that this Item should be Selected */
	virtual void Select(FNavigationToolScopedSelection& InSelection) const {}

	/** Determines whether the given Item is selected in the given Scoped Selection */
	virtual bool IsSelected(const FNavigationToolScopedSelection& InSelection) const { return false; }

	/** Whether the Item can be selected in Navigation Tool at all */
	virtual bool IsSelectable() const { return true; }

	/** Called when an Item is selected */
	virtual void OnSelect() {}

	/** Called when an Item is double-clicked with the mouse */
	virtual void OnDoubleClick() {}

	/** Gets the Navigation Tool that owns this item */
	virtual INavigationTool& GetOwnerTool() const = 0;

	/** Called when the Item has been registered into the Navigation Tool */
	virtual void OnItemRegistered() {}

	/** Called when the Item has been unregistered from the Navigation Tool */
	virtual void OnItemUnregistered() {}

	/** Called when the Item been selected/deselected from the Tree View */
	virtual void OnItemSelectionChanged(const bool bInIsSelected) {}

	/** Refreshes what the Parent and Children are of this Item. (not recursive!) */
	virtual void RefreshChildren() = 0;

	/** Resets both the cached visible children and children (before doing so, sets all child's parents to null) */
	virtual void ResetChildren() = 0;

	/**
	 * Determines whether this item can be sorted by the Navigation Tool or not.
	 * Unsorted Items usually mean that they have their own way of sorting that Navigation Tool's Item Sorting Data should not interfere with.
	 * Note: Unsorted Children go before Sorted Items (e.g. Item Proxies go first before Actors below a Parent)
	 */
	virtual bool ShouldSort() const = 0;

	/** Determines whether the given Child is supported and can be added under this Item */
	virtual bool CanAddChild(const FNavigationToolViewModelPtr& InChild) const = 0;

	/**
	 * Adds another Child under this Item if such Item is supported.
	 * Returns true if it did, false if it could not add it (e.g. item not supported).
	 */
	virtual bool AddChild(const FNavigationToolAddItemParams& InAddItemParams) = 0;

	/**
	 * Removes the given child from this Item if it was ever indeed a child.
	 * Returns true if the removal did happen.
	 */
	virtual bool RemoveChild(const FNavigationToolRemoveItemParams& InRemoveItemParams) = 0;

	/** Finds this items children. This is only relevant for items that do have that functionality (e.g. components or actors) */
	virtual void FindChildren(TArray<FNavigationToolViewModelWeakPtr>& OutWeakChildren, const bool bInRecursive) = 0;

	virtual void FindValidChildren(TArray<FNavigationToolViewModelWeakPtr>& OutWeakChildren, const bool bInRecursive) = 0;

	/** Gets the Item Proxies for this Item (e.g. Component Item that represent Primitives add in a Material Proxy to display) */
	virtual void GetItemProxies(TArray<TSharedPtr<FNavigationToolItemProxy>>& OutItemProxies) {}

	/*
	 * Tries to find the first path of descendants (not including self) that lead to a given item in the set.
	 * The last item is the item in the set that was found, so the path might be "A/B/C/.../ItemInSet", where A is the child of This.
	 * Returns The empty array if failed.
	*/
	virtual TArray<FNavigationToolViewModelPtr> FindPath(const TArray<FNavigationToolViewModelWeakPtr>& InWeakItems) const = 0;

	/** Returns the path to this item in the tree. Ex. "RootId,ChildId,ChildId" */
	virtual FString GetFullPath() const = 0;

	/** Gets the current Child Items of this item */
	virtual const TArray<FNavigationToolViewModelWeakPtr>& GetChildren() const = 0;
	
	/** Gets the current Child Items of this Item */
	virtual TArray<FNavigationToolViewModelWeakPtr>& GetChildrenMutable() = 0;

	/**
	 * Gets the Index that the given Child Item is at.
	 * NOTE: This includes HIDDEN items as Item Visibility is relative to each Navigation Tool View.
	 * To consider only Visible Items, use FNavigationToolView::GetVisibleChildAt.
	 */
	virtual IndexType GetChildIndex(const FNavigationToolViewModelWeakPtr& InWeakChildItem) const = 0;

	/*
	 * Gets the Child Item at the given Index.
	 * NOTE: This includes HIDDEN items as Item Visibility is relative to each Navigation Tool View.
	 * To consider only Visible Items, use FNavigationToolView::GetVisibleChildAt.
	 */
	virtual FNavigationToolViewModelWeakPtr GetChildAt(const IndexType InIndex) const = 0;

	/**
	 * Gets the Parent of this item.
	 * Should only be null prior to registering it in Navigation Tool or if its root item.
	 */
	virtual FNavigationToolViewModelPtr GetParent() const = 0;

	/** Gets the list of all parents up the tree from this Item */
	virtual TSet<FNavigationToolViewModelPtr> GetParents(const bool bInIncludeRoot = false) const = 0;

	/** Sets the parent. Note that the parent must've already had this instance as a child (check is done) */
	virtual void SetParent(FNavigationToolViewModelPtr InParent) = 0;

	/** Gets the Id of this item */
	virtual FNavigationToolItemId GetItemId() const = 0;

	/**
	 * Whether this item can be at the Top Level just beneath the Root, or it needs to always be under some other item
	 * E.g. Actors can be Top Level, but Components or Materials can't
	 */
	virtual bool CanBeTopLevel() const = 0;
	
	/** Returns whether this item (and what it represents) should be allowed to be registered in Navigation Tool */
	virtual bool IsAllowedInTool() const = 0;
	
	/** Gets the display name text of the item */
	virtual FText GetDisplayName() const = 0;

	/** Gets the Class/Type of this Item (e.g. for Items that represent UObjects, it will be the UObject class) */
	virtual FText GetClassName() const = 0;

	/** Gets the color to use for the item label */
	virtual FSlateColor GetItemLabelColor() const = 0;

	/** Gets the color to use to tint the entire item row (all column content) */
	virtual FLinearColor GetItemTintColor() const = 0;

	/** Gets the slate icon for the item */
	virtual FSlateIcon GetIcon() const = 0;

	virtual const FSlateBrush* GetDefaultIconBrush() const { return nullptr; }

	virtual const FSlateBrush* GetIconBrush() const = 0;

	/** Gets the color for the item icon */
	virtual FSlateColor GetIconColor() const = 0;

	/** Gets the tooltip text for the item icon */
	virtual FText GetIconTooltipText() const = 0;

	/** Gets the View Modes that this Item Supports */
	virtual ENavigationToolItemViewMode GetSupportedViewModes(const INavigationToolView& InToolView) const = 0;

	/** Whether this Item should be visualized in the given View Mode, for the given Navigation Tool View */
	virtual bool IsViewModeSupported(const ENavigationToolItemViewMode InViewMode, const INavigationToolView& InToolView) const = 0;

	/**
	 * Called when objects have been replaced on the Engine side. Used to replace any UObjects used by this item
	 * @param InReplacementMap the map of old object that is garbage to the new object that replaced it
	 * @param bInRecursive     whether to recurse this same function to children items
	 */
	virtual void OnObjectsReplaced(const TMap<UObject*, UObject*>& InReplacementMap, const bool bInRecursive) {}

	/** Function responsible of Generating the Label Widget for this Item (i.e. the column containing the Icon and the Name) */
	virtual TSharedRef<SWidget> GenerateLabelWidget(const TSharedRef<SNavigationToolTreeRow>& InRow) = 0;

	/** Whether this Item supports Visibility for the Given Type */
	virtual bool ShowVisibility() const = 0;

	/** Whether a change in Parent Visibility should also affect this Item's Visibility */
	virtual bool CanReceiveParentVisibilityPropagation() const = 0;

	/** Whether this Item is currently visible or not for the Given Type */
	virtual bool GetVisibility() const = 0;

	/** Called when the Visibility on Item has been changed on the Navigation Tool side */
	virtual void OnVisibilityChanged(const bool bInNewVisibility) {}

	/**
	 * Delegate signature for when the Item Expansion Changes
	 * const TSharedPtr<FNavigationToolView>& - The Navigation Tool View where this Expansion happened
	 * bool - Whether the Item is expanded
	 */
	//using FOnExpansionChanged = TMulticastDelegate<void(const TSharedPtr<INavigationToolView>&, bool)>;

	/** Called when Expansion state (Expanded/Collapsed) has been changed */
	//virtual FOnExpansionChanged& OnExpansionChanged() = 0;

	/** Whether the Item is able to expand when AutoExpand functionality is enabled */
	virtual bool CanAutoExpand() const = 0;

	/** Whether this Item can be deleted or not */
	virtual bool CanDelete() const = 0;

	/** The implementation to delete the item */
	virtual bool Delete() = 0;

	virtual void AddFlags(const ENavigationToolItemFlags InFlag) = 0;
	virtual void RemoveFlags(const ENavigationToolItemFlags InFlag) = 0;

	virtual bool HasAnyFlags(const ENavigationToolItemFlags InFlag) const = 0;
	virtual bool HasAllFlags(const ENavigationToolItemFlags InFlag) const = 0;

	virtual void SetFlags(const ENavigationToolItemFlags InFlags) = 0;
	virtual ENavigationToolItemFlags GetFlags() const = 0;

	/** Gets the Tags found for this item (e.g. for Actors, actor tags and for Components, component tags) */
	virtual TArray<FName> GetTags() const { return TArray<FName>(); }

	/** Returns the item's height in the tree. Root item should return 0 as it has no parent. */
	//virtual int32 GetItemTreeHeight() const = 0;

	/**
	 * Delegate signature for relaying an Item Rename action
	 * ENavigationToolRenameAction the type of action being relayed (e.g. request a rename, or notify rename complete, etc)
	 * const TSharedPtr<INavigationToolView>& - The Navigation Tool View where the rename action is taking place
	 */
	//using FOnRenameAction = TMulticastDelegate<void(ENavigationToolRenameAction, const TSharedPtr<INavigationToolView>&)>;

	/** Broadcasts whenever a rename action takes place from a given view (e.g. when pressing "F2" to rename, or committing the rename text) */
	//virtual FOnRenameAction& OnRenameAction() = 0;

	/** Determines if and where the incoming Drag Drop Event can be processed by this item */
	virtual TOptional<EItemDropZone> CanAcceptDrop(const FDragDropEvent& InDragDropEvent, const EItemDropZone InDropZone) = 0;

	/** Processes the Drag and Drop Event for this Item */
	virtual  FReply AcceptDrop(const FDragDropEvent& InDragDropEvent, const EItemDropZone InDropZone) = 0;

	/** Whether Ignoring Pending Kill. Useful to get underlying UObjects that are pending kill and get the pointer to it and not a null value */
	virtual bool IsIgnoringPendingKill() const = 0;

	/** Gets whether this item is expanded */
	virtual bool IsExpanded() const = 0;

	/** Sets the expansion state of this item */
	virtual void SetExpansion(const bool bInIsExpanded) = 0;

	/** Converts this Navigation Tool item to a serialized item that can be saved in the sequence */
	virtual FNavigationToolSerializedItem MakeSerializedItem() = 0;

	FOnRenameAction& OnRenameAction() { return OnRenameActionDelegate; }

	FOnExpansionChanged& OnExpansionChanged() { return OnExpansionChangedDelegate; }

protected:
	/** Delegate for when expansion changes in the item */
	FOnExpansionChanged OnExpansionChangedDelegate;
	
	/** The delegate for renaming */
	FOnRenameAction OnRenameActionDelegate;
};

} // namespace UE::SequenceNavigator

#undef UE_API
