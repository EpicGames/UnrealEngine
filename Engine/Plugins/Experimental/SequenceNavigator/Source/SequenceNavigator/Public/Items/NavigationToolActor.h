// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/ViewModelTypeID.h"
#include "NavigationToolBinding.h"
#include "NavigationToolDefines.h"

#define UE_API SEQUENCENAVIGATOR_API

class AActor;

namespace UE::SequenceNavigator
{

class INavigationToolView;

/**
 * Navigation Tool Item representing an AActor binding
 */
class FNavigationToolActor
	: public FNavigationToolBinding
{
public:
	UE_SEQUENCER_DECLARE_CASTABLE(FNavigationToolActor
		, FNavigationToolBinding)

	UE_API FNavigationToolActor(INavigationTool& InTool
		, const FNavigationToolViewModelPtr& InParentItem
		, const TSharedPtr<FNavigationToolSequence>& InParentSequenceItem
		, const FMovieSceneBinding& InBinding);

	//~ Begin INavigationToolItem

	UE_API virtual void FindChildren(TArray<FNavigationToolViewModelWeakPtr>& OutWeakChildren, const bool bInRecursive) override;

	UE_API virtual bool IsAllowedInTool() const override;
	UE_API virtual ENavigationToolItemViewMode GetSupportedViewModes(const INavigationToolView& InToolView) const override;
	virtual bool CanReceiveParentVisibilityPropagation() const override { return true; }

	UE_API virtual TArray<FName> GetTags() const override;

	virtual bool ShowVisibility() const override { return true; }
	UE_API virtual bool GetVisibility() const override;
	UE_API virtual void OnVisibilityChanged(const bool bInNewVisibility) override;

	//~ End INavigationToolItem

	//~ Begin IRenameableExtension
	UE_API virtual bool CanRename() const override;
	UE_API virtual void Rename(const FText& InNewName) override;
	//~ End IRenameableExtension

	UE_API AActor* GetActor() const;
};

} // namespace UE::SequenceNavigator

#undef UE_API
