// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Extensions/IIdExtension.h"
#include "Extensions/IPlayheadExtension.h"
#include "MovieSceneBinding.h"
#include "MVVM/Extensions/IRenameableExtension.h"
#include "MVVM/ViewModelTypeID.h"
#include "MVVM/ViewModels/ObjectBindingModel.h"
#include "NavigationToolItem.h"
#include "Styling/SlateColor.h"
#include "Textures/SlateIcon.h"

#define UE_API SEQUENCENAVIGATOR_API

namespace UE::Sequencer
{
class FObjectBindingModel;
}

namespace UE::SequenceNavigator
{

class FNavigationToolSequence;

/**
 * Navigation Tool Item representing a Sequence binding
 */
class FNavigationToolBinding
	: public FNavigationToolItem
	, public Sequencer::IRenameableExtension
	//, public ISequenceLockableExtension
	, public IPlayheadExtension
	, public IIdExtension
{
public:
	UE_SEQUENCER_DECLARE_CASTABLE_API(UE_API, FNavigationToolBinding
		, FNavigationToolItem
		, Sequencer::IRenameableExtension
		//, ISequenceLockableExtension
		, IPlayheadExtension
		, IIdExtension);

	UE_API FNavigationToolBinding(INavigationTool& InTool
		, const FNavigationToolViewModelPtr& InParentItem
		, const TSharedPtr<FNavigationToolSequence>& InParentSequenceItem
		, const FMovieSceneBinding& InBinding);

	//~ Begin INavigationToolItem

	UE_API virtual bool IsItemValid() const override;
	UE_API virtual UObject* GetItemObject() const override;
	UE_API virtual bool IsAllowedInTool() const override;

	UE_API virtual void FindChildren(TArray<FNavigationToolViewModelWeakPtr>& OutWeakChildren, const bool bInRecursive) override;

	virtual bool CanBeTopLevel() const override { return false; }
	virtual bool ShouldSort() const override { return false; }

	UE_API virtual FText GetDisplayName() const override;
	UE_API virtual FText GetClassName() const override;
	UE_API virtual FSlateIcon GetIcon() const override;
	UE_API virtual FText GetIconTooltipText() const override;
	UE_API virtual FSlateColor GetIconColor() const override;

	UE_API virtual bool IsSelected(const FNavigationToolScopedSelection& InSelection) const override;
	UE_API virtual void Select(FNavigationToolScopedSelection& InSelection) const override;
	UE_API virtual void OnSelect() override;
	UE_API virtual void OnDoubleClick() override;

	UE_API virtual bool CanDelete() const override;
	UE_API virtual bool Delete() override;

	UE_API virtual FNavigationToolItemId CalculateItemId() const override;

	//~ End INavigationToolItem

	//~ Begin IRenameableExtension
	UE_API virtual bool CanRename() const override;
	UE_API virtual void Rename(const FText& InNewName) override;
	//~ End IRenameableExtension

	//~ Begin ISequenceLockableExtension
	//virtual EItemSequenceLockState GetLockState() const override;
	//virtual void SetIsLocked(const bool bInIsLocked) override;
	//~ End ISequenceLockableExtension

	//~ Begin IIdExtension
	UE_API virtual FText GetId() const override;
	//~ End IIdExtension

	//~ Begin IPlayheadExtension
	UE_API virtual EItemContainsPlayhead ContainsPlayhead() const override;
	//~ End IPlayheadExtension

	UE_API const FMovieSceneBinding& GetBinding() const;

	/** @return The Sequence this binding belongs to */
	UE_API UMovieSceneSequence* GetSequence() const;

	/** @return The Movie Scene this binding belongs to */
	UMovieScene* GetMovieScene() const;

	/** @return The cached object that is bound in sequencer. */
	UE_API UObject* GetCachedBoundObject() const;

	/** Caches the object if it's not already been cached and returns the result. */
	UE_API UObject* CacheBoundObject();

	UE_API UE::Sequencer::TViewModelPtr<UE::Sequencer::FObjectBindingModel> GetViewModel() const;

protected:
	Sequencer::TWeakViewModelPtr<FNavigationToolSequence> WeakParentSequenceItem;

	FMovieSceneBinding Binding;

	TWeakObjectPtr<const UClass> WeakBoundObjectClass;
	TWeakObjectPtr<> WeakBoundObject;

	FSlateIcon Icon;
	FSlateColor IconColor;
};

} // namespace UE::SequenceNavigator

#undef UE_API
