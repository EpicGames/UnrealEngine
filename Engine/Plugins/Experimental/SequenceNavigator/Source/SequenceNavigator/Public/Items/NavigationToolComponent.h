// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MVVM/ViewModelTypeID.h"
#include "NavigationToolBinding.h"

#define UE_API SEQUENCENAVIGATOR_API

class UActorComponent;

namespace UE::SequenceNavigator
{

/**
 * Navigation Tool Item representing an Actor Component binding
 */
class FNavigationToolComponent
	: public FNavigationToolBinding
{
public:
	UE_SEQUENCER_DECLARE_CASTABLE_API(UE_API, FNavigationToolComponent
		, FNavigationToolBinding)

	UE_API FNavigationToolComponent(INavigationTool& InTool
		, const FNavigationToolViewModelPtr& InParentItem
		, const TSharedPtr<FNavigationToolSequence>& InParentSequenceItem
		, const FMovieSceneBinding& InBinding);

	//~ Begin INavigationToolItem

	UE_API virtual void FindChildren(TArray<FNavigationToolViewModelWeakPtr>& OutWeakChildren, const bool bInRecursive) override;
	UE_API virtual void GetItemProxies(TArray<TSharedPtr<FNavigationToolItemProxy>>& OutItemProxies) override;

	UE_API virtual bool IsAllowedInTool() const override;
	UE_API virtual ENavigationToolItemViewMode GetSupportedViewModes(const INavigationToolView& InToolView) const override;
	virtual bool CanReceiveParentVisibilityPropagation() const override { return false; }
	UE_API virtual TSharedRef<SWidget> GenerateLabelWidget(const TSharedRef<SNavigationToolTreeRow>& InRow) override;

	UE_API virtual FLinearColor GetItemTintColor() const override;
	UE_API virtual TArray<FName> GetTags() const override;

	virtual bool ShowVisibility() const override { return true; }
	UE_API virtual bool GetVisibility() const override;
	UE_API virtual void OnVisibilityChanged(const bool bInNewVisibility) override;

	//~ End INavigationToolItem

	//~ Begin IRenameableExtension
	UE_API virtual bool CanRename() const override;
	UE_API virtual void Rename(const FText& InNewName) override;
	//~ End IRenameableExtension

	UE_API UActorComponent* GetComponent() const;
};

} // namespace UE::SequenceNavigator

#undef UE_API
