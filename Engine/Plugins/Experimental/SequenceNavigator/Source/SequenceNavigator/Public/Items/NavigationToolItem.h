// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorSequenceNavigationDefs.h"
#include "Extensions/IColorExtension.h"
#include "Items/INavigationToolItem.h"
#include "Items/NavigationToolItemId.h"
#include "Misc/Optional.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/ViewModels/OutlinerItemModel.h"
#include "NavigationToolSettings.h"

#define UE_API SEQUENCENAVIGATOR_API

class FText;

namespace UE::SequenceNavigator
{

class FNavigationToolProvider;
class FNavigationToolTreeRoot;
class INavigationTool;
class INavigationToolView;
class SNavigationToolTreeRow;
struct FNavigationToolAddItemParams;
struct FNavigationToolRemoveItemParams;

/** Base implementation of INavigationToolItem */
class FNavigationToolItem
	: public Sequencer::TOutlinerModelMixin<Sequencer::FViewModel>
	, public INavigationToolItem
	, public IColorExtension
{
	friend class FNavigationTool;

public:
	using IndexType = TArray<INavigationToolItem>::SizeType;

	UE_SEQUENCER_DECLARE_CASTABLE_API(UE_API, FNavigationToolItem
		, Sequencer::TOutlinerModelMixin<Sequencer::FViewModel>
		, INavigationToolItem
		, IColorExtension)

	DECLARE_DELEGATE_RetVal_OneParam(FNavigationToolSaveState*, FNavigationToolGetToolState
		, const FNavigationToolViewModelPtr /*InItem*/);

	UE_API FNavigationToolItem(INavigationTool& InTool, const FNavigationToolViewModelPtr& InParentItem);

	//~ Begin INavigationToolItem

	UE_API virtual INavigationTool& GetOwnerTool() const final override;
	UE_API virtual TSharedPtr<FNavigationToolProvider> GetProvider() const final override;
	UE_API virtual FNavigationToolSaveState* GetProviderSaveState() const override;
	UE_API virtual bool IsItemValid() const override;

	UE_API virtual void RefreshChildren() override;
	UE_API virtual void ResetChildren() override;

	virtual const TArray<FNavigationToolViewModelWeakPtr>& GetChildren() const override { return WeakChildren; }
	virtual TArray<FNavigationToolViewModelWeakPtr>& GetChildrenMutable() override { return WeakChildren; }

	UE_API virtual void FindChildren(TArray<FNavigationToolViewModelWeakPtr>& OutWeakChildren
		, const bool bInRecursive) override;
	UE_API virtual void FindValidChildren(TArray<FNavigationToolViewModelWeakPtr>& OutWeakChildren
		, const bool bInRecursive) override;

	UE_API virtual TArray<FNavigationToolViewModelPtr> FindPath(const TArray<FNavigationToolViewModelWeakPtr>& InWeakItems) const override final;
	UE_API virtual FString GetFullPath() const override final;

	UE_API virtual IndexType GetChildIndex(const FNavigationToolViewModelWeakPtr& InWeakChildItem) const override;
	UE_API virtual FNavigationToolViewModelWeakPtr GetChildAt(const IndexType InIndex) const override;

	UE_API virtual FNavigationToolViewModelPtr GetParent() const override;
	UE_API virtual TSet<FNavigationToolViewModelPtr> GetParents(const bool bInIncludeRoot = false) const override;
	UE_API virtual void SetParent(FNavigationToolViewModelPtr InParent) override;

	virtual bool ShouldSort() const override { return false; }

	UE_API virtual bool CanAddChild(const FNavigationToolViewModelPtr& InChild) const override;
	UE_API virtual bool AddChild(const FNavigationToolAddItemParams& InAddItemParams) override;
	UE_API virtual bool RemoveChild(const FNavigationToolRemoveItemParams& InRemoveItemParams) override;

	UE_API virtual FNavigationToolItemId GetItemId() const override final;
	virtual bool CanBeTopLevel() const override { return false; }
	virtual bool IsAllowedInTool() const override { return true; }
	virtual FText GetDisplayName() const override { return FText::GetEmpty(); }
	virtual FText GetClassName() const override { return FText::GetEmpty(); }

	UE_API virtual ENavigationToolItemViewMode GetSupportedViewModes(const INavigationToolView& InToolView) const override;
	UE_API virtual bool IsViewModeSupported(const ENavigationToolItemViewMode InViewMode, const INavigationToolView& InToolView) const override;

	UE_API virtual FSlateColor GetItemLabelColor() const override;
	UE_API virtual FLinearColor GetItemTintColor() const override;
	UE_API virtual const FSlateBrush* GetIconBrush() const override;
	UE_API virtual FSlateColor GetIconColor() const override;

	UE_API virtual TSharedRef<SWidget> GenerateLabelWidget(const TSharedRef<SNavigationToolTreeRow>& InRow) override;

	virtual bool ShowVisibility() const override { return false; }
	virtual bool CanReceiveParentVisibilityPropagation() const override { return false; }
	virtual bool GetVisibility() const override { return false; }
	virtual bool CanAutoExpand() const override { return true; }
	virtual bool CanDelete() const override { return false; }
	UE_API virtual bool Delete() override;

	UE_API virtual void AddFlags(ENavigationToolItemFlags Flags) override;
	UE_API virtual void RemoveFlags(ENavigationToolItemFlags Flags) override;
	UE_API virtual bool HasAnyFlags(ENavigationToolItemFlags Flags) const override;
	UE_API virtual bool HasAllFlags(ENavigationToolItemFlags Flags) const override;
	virtual void SetFlags(ENavigationToolItemFlags InFlags) override { ItemFlags = InFlags; }
	virtual ENavigationToolItemFlags GetFlags() const override { return ItemFlags; }

	virtual TArray<FName> GetTags() const override { return TArray<FName>(); }

	UE_API virtual TOptional<EItemDropZone> CanAcceptDrop(const FDragDropEvent& InDragDropEvent, const EItemDropZone InDropZone) override;
	UE_API virtual FReply AcceptDrop(const FDragDropEvent& InDragDropEvent, const EItemDropZone InDropZone) override;

	UE_API virtual bool IsIgnoringPendingKill() const override final;

	UE_API virtual void OnObjectsReplaced(const TMap<UObject*, UObject*>& InReplacementMap, const bool bInRecursive) override;

	UE_API virtual bool IsExpanded() const override;
	UE_API virtual void SetExpansion(const bool bInExpand) override;

	UE_API virtual FNavigationToolSerializedItem MakeSerializedItem() override final;

	//~ End INavigationToolItem

	//~ Begin IColorExtension
	UE_API virtual TOptional<FColor> GetColor() const override;
	UE_API virtual void SetColor(const TOptional<FColor>& InColor) override;
	//~ End IColorExtension

	//~ Begin IOutlinerExtension
	UE_API virtual Sequencer::FOutlinerSizing GetOutlinerSizing() const override;
	//~ End IOutlinerExtension

	Sequencer::TViewModelPtr<FNavigationToolItem> AsItemViewModel()
	{
		const TSharedRef<FNavigationToolItem> SharedThisRef = SharedThis(this);
		return Sequencer::CastViewModel<FNavigationToolItem>(SharedThisRef);
	}

	Sequencer::TViewModelPtr<FNavigationToolItem> AsItemViewModelConst() const
	{
		const TSharedRef<FNavigationToolItem> SharedThisRef = SharedThis(const_cast<FNavigationToolItem*>(this));
		return Sequencer::CastViewModel<FNavigationToolItem>(SharedThisRef);
	}

protected:
	friend class INavigationTool;

	UE_API void SetProvider(const TWeakPtr<FNavigationToolProvider>& InWeakProvider);

	/** Gets the Item Id with the latest information (e.g. parent, object, etc)*/
	virtual FNavigationToolItemId CalculateItemId() const = 0;

	/** Sets the ItemId member var to what CalculateItemId returns */
	UE_API void RecalculateItemId();
	
	/** The actual implementation of putting the given item under the children array */
	UE_API void AddChildChecked(const FNavigationToolAddItemParams& InAddItemParams);

	/** The actual implementation of removing the given item from the Children array */
	UE_API bool RemoveChildChecked(const FNavigationToolRemoveItemParams& InRemoveItemParams);
	
	/** Careful handling of multiple children being detected and added to this item children array */
	UE_API void HandleNewSortableChildren(TArray<FNavigationToolViewModelWeakPtr> InWeakSortableChildren);

	/** Reference to the Owning Navigation Tool */
	INavigationTool& Tool;

	/** Tool provider that is responsible for the creation of this item */
	TWeakPtr<FNavigationToolProvider> WeakProvider;

	/** Weak pointer to the Parent Item. Can be null, but if valid, the Parent should have this item in the Children Array */
	FNavigationToolViewModelWeakPtr WeakParent;

	/** Array of Shared pointers to the Child Items. These Items should have their ParentWeak pointing to this item */
	TArray<FNavigationToolViewModelWeakPtr> WeakChildren;

	/** The current flags set for this item */
	ENavigationToolItemFlags ItemFlags;

	FNavigationToolItemId ItemId;
};

/** Adds scoped item flags and removes them when out of scope. Useful for temp checks like IgnorePendingKill */
struct FNavigationToolItemFlagGuard
{
	FNavigationToolItemFlagGuard(const FNavigationToolViewModelPtr& InItem, const ENavigationToolItemFlags InItemFlags)
	{
		if (InItem.IsValid())
		{
			WeakItem = InItem;
			OldItemFlags = InItem->GetFlags();
			InItem->SetFlags(InItemFlags);
		}
	}

	~FNavigationToolItemFlagGuard()
	{
		if (const FNavigationToolViewModelPtr Item = WeakItem.Pin())
		{
			Item->SetFlags(OldItemFlags);
		}
	}

protected:
	FNavigationToolViewModelWeakPtr WeakItem;
	
	ENavigationToolItemFlags OldItemFlags;
};

} // namespace UE::SequenceNavigator

#undef UE_API
