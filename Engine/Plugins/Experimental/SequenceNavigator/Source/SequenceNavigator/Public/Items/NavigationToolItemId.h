// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

//#include "ItemProxies/INavigationToolItemProxyFactory.h"
#include "NavigationToolDefines.h"
#include "SequencerCoreFwd.h"

#define UE_API SEQUENCENAVIGATOR_API

class UMovieSceneSequence;
class UMovieSceneSection;
class UObject;

namespace UE::SequenceNavigator
{

class INavigationToolItem;
class INavigationToolItemProxyFactory;

/** Struct to identify an item in the Navigation Tool */
struct FNavigationToolItemId
{
	static UE_API FString GetObjectPath(const UObject* const InObject);

	static UE_API void AddSeparatedSegment(FString& OutString, const FString& InSegment);

	static UE_API FNavigationToolItemId RootId;

	static constexpr const TCHAR* Separator = TEXT(",");

	/** Default Ctor. Does not run any CalculateTypeHash so this instance remains invalid */
	FNavigationToolItemId() = default;

	/** More flexible option to just specify the string directly. Could be used for folders (e.g. for a nested folder C could be "A/B/C" */
	UE_API FNavigationToolItemId(const FString& InUniqueId);

	/**
	 * Ctor used for Objects that are expected to appear multiple times in the Navigation Tool (e.g. a Material Ref)
	 *
	 * Example Id #1:
	 * [Component Path], [Full Path of Material Asset], [Slot Index]
	 * "/Game/World.World:PersistentLevel.StaticMeshActor_0.StaticMeshComponent,/Game/Materials/M_TestMaterial.M_TestMaterial,[Slot 0]"
	 *
	 * Example Id #2:
	 * [Component Path], [Material Instance Dynamic Path], [Slot Index]
	 * "/Game/World.World:PersistentLevel.StaticMeshActor_0.StaticMeshComponent,/Game/World.World:PersistentLevel.StaticMeshActor_0.StaticMeshComponent.MaterialInstanceDynamic_0,[Slot 0]"
	 * 
	 * @param InObject The Object being referenced multiple times (e.g. a Material)
	 * @param InReferencingItem The Item holding a reference to the object (e.g. a Primitive Comp referencing a Material)
	 * @param InReferencingId The Id to differentiate this reference from other references within the same object (e.g. can be slot or property name)
	 */
	UE_API FNavigationToolItemId(const UObject* InObject
		, const FNavigationToolViewModelPtr& InReferencingItem
		, const FString& InReferencingId);

	/**
	 * Ctor used for making the Item Id for an item proxy that will be under the given Parent Item.
	 * Used when the actual Item Proxy is not created yet, but know it's factory and want to know whether the item proxy already exists
	 * @param InParentItem The parent item holding the item proxy
	 * @param InItemProxyFactory The factory responsible for creating the item proxy
	 */
	UE_API FNavigationToolItemId(const FNavigationToolViewModelPtr& InParentItem
		, const INavigationToolItemProxyFactory& InItemProxyFactory);

	/**
	 * Ctor used for making the Item Id for an Item Proxy under the given Parent Item.
	 * @param InParentItem The parent item holding the item proxy
	 * @param InItemProxy The item proxy under the parent item
	 */
	UE_API FNavigationToolItemId(const FNavigationToolViewModelPtr& InParentItem
		, const FNavigationToolViewModelPtr& InItemProxy);

	/**
	 * Ctor used for making the Item Id for most sequence items
	 * @param InParentItem The parent item holding the item proxy
	 * @param InSequence The parent sequence object
	 * @param InSection The parent section this sequence belongs to, if any
	 * @param InSectionIndex The parent section index, if any. Should be specified if InSection is used.
	 * @param InReferenceId The optional additional reference string
	 */
	UE_API FNavigationToolItemId(const FNavigationToolViewModelPtr& InParentItem
		, const UMovieSceneSequence* const InSequence
		, const UMovieSceneSection* const InSection = nullptr
		, const int32 InSectionIndex = 0
		, const FString& InReferenceId = FString());

	/**
	 * Ctor used for making the Item Id from a Sequencer view model
	 * @param InViewModel The view model to construct the Id from
	 */
	UE_API FNavigationToolItemId(const UE::Sequencer::FViewModelPtr& InViewModel);

	UE_API FNavigationToolItemId(const FNavigationToolItemId& Other);
	UE_API FNavigationToolItemId(FNavigationToolItemId&& Other) noexcept;

	UE_API FNavigationToolItemId& operator=(const FNavigationToolItemId& Other);
	UE_API FNavigationToolItemId& operator=(FNavigationToolItemId&& Other) noexcept;

	UE_API bool operator==(const FNavigationToolItemId& Other) const;

	friend uint32 GetTypeHash(const FNavigationToolItemId& InItemId)
	{
		return InItemId.CachedHash;
	}

	/** Returns whether this Id has a cached hash (i.e. ran any ctor except the default one) */
	UE_API bool IsValidId() const;

	UE_API FString GetStringId() const;

private:
	/**
	 * Constructs an Item Id for most sequence items
	 * @param InParentItem The parent item holding the item proxy
	 * @param InSequence The parent sequence object
	 * @param InSection The parent section this sequence belongs to, if any
	 * @param InSectionIndex The parent section index, if any. Should be specified if InSection is used.
	 * @param InReferenceId The optional additional reference string
	 */
	UE_API void ConstructId(const FNavigationToolViewModelPtr& InParentItem
		, const UMovieSceneSequence* const InSequence
		, const UMovieSceneSection* const InSection = nullptr
		, const int32 InSectionIndex = 0
		, const FString& InReferenceId = FString());

	UE_API void CalculateTypeHash();

	FString Id;

	uint32 CachedHash = 0;
	bool bHasCachedHash = false;
};

} // namespace UE::SequenceNavigator

#undef UE_API
