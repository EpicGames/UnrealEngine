// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineTypes.h"
#include "Misc/Optional.h"
#include "NavigationToolDefines.h"
#include "Widgets/Views/STableRow.h"

namespace UE::SequenceNavigator
{

class FNavigationToolItem;

enum class ENavigationToolAddItemFlags : uint8
{
	None        = 0,
	/** Also add the children of the given item even if they were not made into their own Add Item Action */
	AddChildren = 1 << 0,
	/** Select this Item on Add */
	Select      = 1 << 1,
	/** Make a Transaction for this Action */
	Transact    = 1 << 2,
};
ENUM_CLASS_FLAGS(ENavigationToolAddItemFlags);

struct FNavigationToolAddItemParams
{
	FNavigationToolAddItemParams(const FNavigationToolViewModelPtr& InItem = nullptr
			, const ENavigationToolAddItemFlags InFlags = ENavigationToolAddItemFlags::None
			, const FNavigationToolViewModelPtr& InRelativeItem = nullptr
			, const TOptional<EItemDropZone>& InRelativeDropZone = TOptional<EItemDropZone>())
		: WeakItem(InItem)
		, WeakRelativeItem(InRelativeItem)
		, RelativeDropZone(InRelativeDropZone)
		, Flags(InFlags)
		, SelectionFlags(ENavigationToolItemSelectionFlags::None)
	{}

	/** The Item to Add */
	FNavigationToolViewModelWeakPtr WeakItem;

	/** The Item to use as base in where to place the Item */
	FNavigationToolViewModelWeakPtr WeakRelativeItem;

	/** The Placement Order from the Relative Item (Onto/Inside, Above, Below) */
	TOptional<EItemDropZone> RelativeDropZone;

	/** Some Extra Flags for what to do when Adding or After Adding the Items */
	ENavigationToolAddItemFlags Flags;

	/** Flags to Indicate how we should Select the Item. This only applies if bool bSelectItem is true */
	ENavigationToolItemSelectionFlags SelectionFlags;
};

struct FNavigationToolRemoveItemParams
{
	FNavigationToolRemoveItemParams(const FNavigationToolViewModelPtr& InItem = nullptr)
		: WeakItem(InItem)
	{}

	FNavigationToolViewModelWeakPtr WeakItem;
};

} // namespace UE::SequenceNavigator
