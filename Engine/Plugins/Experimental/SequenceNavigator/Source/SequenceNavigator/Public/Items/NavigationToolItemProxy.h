// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Items/INavigationToolItem.h"
#include "Items/NavigationToolItem.h"
#include "MVVM/ICastable.h"
#include "NavigationToolDefines.h"

#define UE_API SEQUENCENAVIGATOR_API

namespace UE::SequenceNavigator
{

class INavigationTool;
class INavigationToolView;

/**
 * Item Proxies are Navigation Tool Items that with the sole purpose to group and hold common items together.
 * The description or name of such commonality between these items should be the name of the Proxy that holds them.
 * 
 * NOTE: Although Item proxies by default require a parent to be visible in Navigation Tool,
 * they can be created without a parent as a means to override behavior (e.g. DisplayName, Icon, etc)
 */
class FNavigationToolItemProxy
	: public FNavigationToolItem
{
public:
	UE_SEQUENCER_DECLARE_CASTABLE_API(UE_API, FNavigationToolItemProxy
		, FNavigationToolItem)

	UE_API FNavigationToolItemProxy(INavigationTool& InTool, const FNavigationToolViewModelPtr& InParentItem);

	//~ Begin INavigationToolItem
	UE_API virtual bool IsItemValid() const override;
	UE_API virtual void FindChildren(TArray<FNavigationToolViewModelWeakPtr>& OutWeakChildren
		, const bool bInRecursive) override final;
	UE_API virtual void SetParent(FNavigationToolViewModelPtr InParent) override;
	UE_API virtual ENavigationToolItemViewMode GetSupportedViewModes(const INavigationToolView& InToolView) const override;
	UE_API virtual bool CanAutoExpand() const override;
	virtual FText GetClassName() const override { return FText::GetEmpty(); }
	//~ End INavigationToolItem

	uint32 GetPriority() const { return Priority; }
	void SetPriority(const uint32 InPriority) { Priority = InPriority; }

	/** Gets the items that this item proxy is representing / holding (ie. children) */
	virtual void GetProxiedItems(const FNavigationToolViewModelPtr& InParent
		, TArray<FNavigationToolViewModelWeakPtr>& OutWeakChildren, const bool bInRecursive) = 0;

protected:
	//~ Begin FNavigationToolItem
	UE_API virtual FNavigationToolItemId CalculateItemId() const override;
	//~ End FNavigationToolItem

private:
	/** This item proxy's order priority (ie. Highest priority is placed topmost or leftmost (depending on Orientation).
	 * Priority 0 is the lowest priority. */
	uint32 Priority = 0;
};

} // namespace UE::SequenceNavigator

#undef UE_API
