// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/AllOf.h"
#include "Containers/ContainersFwd.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "NavigationToolDefines.h"
#include "NavigationToolSequence.h"

#define UE_API SEQUENCENAVIGATOR_API

class ISequencer;
class UMovieSceneMetaData;
class UMovieSceneSubSection;

namespace UE::SequenceNavigator
{
class INavigationTool;
class FNavigationToolItem;
class FNavigationToolSequence;
}

namespace UE::SequenceNavigator::ItemUtils
{

/**
 * Compares the absolute order of the items in the Navigation Tool and returns true if A comes before B in the Navigation Tool.
 * Useful to use when sorting items.
 */
bool CompareToolItemOrder(const FNavigationToolViewModelPtr& InA, const FNavigationToolViewModelPtr& InB);

/** Returns two subset arrays of Items: one is containing only Sortable Items and the other Non Sortable Items */
void SplitSortableAndUnsortableItems(const TArray<FNavigationToolViewModelWeakPtr>& InWeakItems
	, TArray<FNavigationToolViewModelWeakPtr>& OutWeakSortable
	, TArray<FNavigationToolViewModelWeakPtr>& OutWeakUnsortable);

/**  */
UMovieSceneSubSection* GetSequenceItemSubSection(const FNavigationToolViewModelPtr& InItem);

/**  */
UE_API UMovieSceneMetaData* GetSequenceItemMetaData(const FNavigationToolViewModelPtr& InItem);

/**
 * Removes the parent sequence prefix from the display name of a child sequence item.
 * @param InOutDisplayName The display name to modify, which will have the parent prefix removed if found.
 * @param InSequenceItem The child sequence item whose display name is being adjusted.
 */
void RemoveSequenceDisplayNameParentPrefix(FText& InOutDisplayName
	, const Sequencer::TViewModelPtr<FNavigationToolSequence>& InSequenceItem);

/**
 * Appends the dirty symbol to the given display name.
 * If the associated package is dirty, a "*" tag is appended to the display name.
 * @param InOutDisplayName The display name to be appended with the status.
 * @param InSequence The sequence whose package is checked for a dirty status.
 */
void AppendSequenceDisplayNameDirtyStatus(FText& InOutDisplayName, const UMovieSceneSequence& InSequence);

/** The result of an item comparison */
enum class ENavigationToolCompareState
{
	NoneTrue    = 0,
	AllTrue     = 1 << 0,
	PartialTrue = 1 << 1,
};
ENUM_CLASS_FLAGS(ENavigationToolCompareState)

/**
 * Evaluates an items children based on a predicate function and determines
 * whether all, none, or some of the children match the condition.
 * @param InItem The item which to evaluate the children of.
 * @param InTrueFunction A function that defines the true condition.
 * @param InFalseFunction A function that defines the false condition.
 * @return ENavigationToolCompareState::AllTrue if all items satisfy the condition
 *         ENavigationToolCompareState::NoneTrue if no items satisfy the condition
 *         ENavigationToolCompareState::PartialTrue if some items satisfy the condition
 */
template<typename InItemType>
ENavigationToolCompareState CompareChildrenItemState(const FNavigationToolViewModelPtr& InItem
	, const TFunctionRef<bool(const Sequencer::TViewModelPtr<InItemType>)>& InTrueFunction
	, const TFunctionRef<bool(const Sequencer::TViewModelPtr<InItemType>)>& InFalseFunction
	, const ENavigationToolCompareState InDefaultState = ENavigationToolCompareState::NoneTrue)
{
	using namespace Sequencer;

	if (!InItem.IsValid())
	{
		return InDefaultState;
	}

	const TViewModelListIterator<InItemType> WeakChildrenIter = InItem.AsModel()->GetChildrenOfType<InItemType>();
	if (!WeakChildrenIter)
	{
		return InDefaultState;
	}

	const TArray<TViewModelPtr<InItemType>> WeakChildren = WeakChildrenIter.ToArray();

	const bool bAllMatch = Algo::AllOf(WeakChildren,
		[&InTrueFunction](const TViewModelPtr<InItemType>& InItem)
		{
			return InTrueFunction(InItem);
		});
	if (bAllMatch)
	{
		return ENavigationToolCompareState::AllTrue;
	}

	const bool bAllNoMatch = Algo::AllOf(WeakChildren,
		[&InFalseFunction](const Sequencer::TViewModelPtr<InItemType>& InItem)
		{
			return InFalseFunction(InItem);
		});
	if (bAllNoMatch)
	{
		return ENavigationToolCompareState::NoneTrue;
	}

	return ENavigationToolCompareState::PartialTrue;
}

/**
 * Evaluates an items children based on a predicate function and determines
 * whether all, none, or some of the children match the condition.
 * @param InItem The item which to evaluate the children of.
 * @param InTrueFunction A function that defines the true condition.
 * @return ENavigationToolCompareState::AllTrue if all items satisfy the condition
 *         ENavigationToolCompareState::NoneTrue if no items satisfy the condition
 *         ENavigationToolCompareState::PartialTrue if some items satisfy the condition
 */
template<typename InItemType>
ENavigationToolCompareState CompareChildrenItemStateSimple(const FNavigationToolViewModelPtr& InItem
	, const TFunctionRef<bool(const InItemType* const)>& InTrueFunction)
{
	using namespace Sequencer;

	if (!InItem)
	{
		return ENavigationToolCompareState::NoneTrue;
	}

	const TViewModelListIterator<const InItemType> WeakChildrenIter = InItem.AsModel()->GetChildrenOfType<const InItemType>();
	if (!WeakChildrenIter)
	{
		return ENavigationToolCompareState::NoneTrue;
	}

	const TArray<TViewModelPtr<const InItemType>> WeakChildren = WeakChildrenIter.ToArray();

	const bool bAllMatch = Algo::AllOf(WeakChildren,
		[&InTrueFunction](const InItemType* const InItem)
		{
			return InTrueFunction(InItem);
		});
	if (bAllMatch)
	{
		return ENavigationToolCompareState::AllTrue;
	}

	const bool bAllNoMatch = Algo::AllOf(WeakChildren,
		[&InTrueFunction](const InItemType* const InItem)
		{
			return !InTrueFunction(InItem);
		});
	if (bAllNoMatch)
	{
		return ENavigationToolCompareState::NoneTrue;
	}

	return ENavigationToolCompareState::PartialTrue;
}

/**
 * Evaluates the state of items in an array based on a predicate function
 * and determines whether all, none, or some of the items match the condition.
 * @param InArray The array of items to evaluate.
 * @param InTrueFunction A function that defines the true condition.
 * @param InFalseFunction A function that defines the false condition.
 * @return ENavigationToolCompareState::AllTrue if all items satisfy the condition
 *         ENavigationToolCompareState::NoneTrue if no items satisfy the condition
 *         ENavigationToolCompareState::PartialTrue if some items satisfy the condition
 */
template<typename InItemType>
ENavigationToolCompareState CompareArrayState(const TArray<InItemType*>& InArray
	, const TFunctionRef<bool(const InItemType* const)>& InTrueFunction
	, const TFunctionRef<bool(const InItemType* const)>& InFalseFunction)
{
	if (InArray.IsEmpty())
	{
		return ENavigationToolCompareState::AllTrue;
	}

	const bool bAllMatch = Algo::AllOf(InArray,
		[&InTrueFunction](const InItemType* const InItem)
		{
			return InTrueFunction(InItem);
		});
	if (bAllMatch)
	{
		return ENavigationToolCompareState::AllTrue;
	}

	const bool bAllNoMatch = Algo::AllOf(InArray,
		[&InFalseFunction](const InItemType* const InItem)
		{
			return InFalseFunction(InItem);
		});
	if (bAllNoMatch)
	{
		return ENavigationToolCompareState::NoneTrue;
	}

	return ENavigationToolCompareState::PartialTrue;
}

/**
 * Evaluates the state of items in an array based on a predicate function
 * and determines whether all, none, or some of the items match the condition.
 * @param InArray The array of items to evaluate.
 * @param InTrueFunction A function that defines the true condition.
 * @return ENavigationToolCompareState::AllTrue if all items satisfy the condition
 *         ENavigationToolCompareState::NoneTrue if no items satisfy the condition
 *         ENavigationToolCompareState::PartialTrue if some items satisfy the condition
 */
template<typename InItemType>
ENavigationToolCompareState CompareArrayStateSimple(const TArray<InItemType*>& InArray
	, const TFunctionRef<bool(const InItemType* const)>& InTrueFunction)
{
	if (InArray.IsEmpty())
	{
		return ENavigationToolCompareState::AllTrue;
	}

	const bool bAllMatch = Algo::AllOf(InArray,
		[&InTrueFunction](const InItemType* const InItem)
		{
			return InTrueFunction(InItem);
		});
	if (bAllMatch)
	{
		return ENavigationToolCompareState::AllTrue;
	}

	const bool bAllNoMatch = Algo::AllOf(InArray,
		[&InTrueFunction](const InItemType* const InItem)
		{
			return !InTrueFunction(InItem);
		});
	if (bAllNoMatch)
	{
		return ENavigationToolCompareState::NoneTrue;
	}

	return ENavigationToolCompareState::PartialTrue;
}

FSlateColor GetItemBindingColor(const ISequencer& InSequencer
	, UMovieSceneSequence& InSequence
	, const FGuid& InObjectGuid
	, const FSlateColor& DefaultColor = FSlateColor::UseForeground());

} // namespace UE::SequenceNavigator

#undef UE_API
