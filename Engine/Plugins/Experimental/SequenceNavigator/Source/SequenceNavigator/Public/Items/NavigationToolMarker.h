// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Extensions/IInTimeExtension.h"
#include "MVVM/Extensions/IRenameableExtension.h"
#include "MVVM/ICastable.h"
#include "NavigationToolItem.h"

#define UE_API SEQUENCENAVIGATOR_API

class UMovieScene;
struct FMovieSceneMarkedFrame;

namespace UE::SequenceNavigator
{

class FNavigationToolSequence;

/**
 * Navigation Tool Item representing a sequence marker
 */
class FNavigationToolMarker
	: public FNavigationToolItem
	, public Sequencer::IRenameableExtension
	, public IInTimeExtension
{
public:
	UE_SEQUENCER_DECLARE_CASTABLE_API(UE_API, FNavigationToolMarker
		, FNavigationToolItem
		, Sequencer::IRenameableExtension
		, IInTimeExtension)

	UE_API FNavigationToolMarker(INavigationTool& InTool
		, const FNavigationToolViewModelPtr& InParentItem
		, const TSharedPtr<FNavigationToolSequence>& InParentSequenceItem
		, const int32 InMarkedFrameIndex);

	//~ Begin INavigationToolItem

	UE_API virtual bool IsItemValid() const override;
	UE_API virtual bool IsAllowedInTool() const override;

	UE_API virtual ENavigationToolItemViewMode GetSupportedViewModes(const INavigationToolView& InToolView) const override;

	UE_API virtual FText GetDisplayName() const override;
	UE_API virtual FText GetClassName() const override;
	UE_API virtual FSlateIcon GetIcon() const override;
	UE_API virtual const FSlateBrush* GetIconBrush() const override;
	UE_API virtual FSlateColor GetIconColor() const override;
	UE_API virtual FText GetIconTooltipText() const override;

	UE_API virtual bool IsSelected(const FNavigationToolScopedSelection& InSelection) const override;
	UE_API virtual void Select(FNavigationToolScopedSelection& InSelection) const override;
	UE_API virtual void OnSelect() override;
	UE_API virtual void OnDoubleClick() override;

	UE_API virtual bool CanDelete() const override;
	UE_API virtual bool Delete() override;

	//~ End INavigationToolItem

	//~ Begin IRenameableExtension
	UE_API virtual bool CanRename() const override;
	UE_API virtual void Rename(const FText& InNewName) override;
	//~ End IRenameableExtension

	//~ Begin IInTimeExtension
	UE_API virtual FFrameNumber GetInTime() const override;
	UE_API virtual void SetInTime(const FFrameNumber& InTime) override;
	//~ End IInTimeExtension

	UE_API int32 GetMarkedFrameIndex() const;

	UE_API FMovieSceneMarkedFrame* GetMarkedFrame() const;

protected:
	//~ Begin INavigationToolItem
	UE_API virtual FNavigationToolItemId CalculateItemId() const override;
	//~ End INavigationToolItem

	UE_API UMovieSceneSequence* GetParentSequence() const;
	UE_API UMovieScene* GetParentMovieScene() const;

	Sequencer::TWeakViewModelPtr<FNavigationToolSequence> WeakParentSequenceItem;

	int32 MarkedFrameIndex = INDEX_NONE;
};

} // namespace UE::SequenceNavigator

#undef UE_API
