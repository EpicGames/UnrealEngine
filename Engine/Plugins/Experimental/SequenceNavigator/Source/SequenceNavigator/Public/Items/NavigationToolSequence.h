// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Extensions/IInTimeExtension.h"
#include "Extensions/IIdExtension.h"
#include "Extensions/IMarkerVisibilityExtension.h"
#include "Extensions/IOutTimeExtension.h"
#include "Extensions/IPlayheadExtension.h"
#include "Extensions/IRevisionControlExtension.h"
#include "MVVM/Extensions/IDeactivatableExtension.h"
#include "MVVM/Extensions/ILockableExtension.h"
#include "MVVM/Extensions/IRenameableExtension.h"
#include "MVVM/ViewModelTypeID.h"
#include "NavigationToolItem.h"

#define UE_API SEQUENCENAVIGATOR_API

class UMovieScene;
class UMovieSceneSequence;
class UMovieSceneSubSection;
struct FMovieSceneBinding;

namespace UE::Sequencer
{
class FSectionModel;
}

namespace UE::SequenceNavigator
{

/**
 * Item in Navigation Tool representing a Sequence.
 */
class FNavigationToolSequence
	: public FNavigationToolItem
	, public Sequencer::IRenameableExtension
	, public Sequencer::IDeactivatableExtension
	, public Sequencer::ILockableExtension
	, public IMarkerVisibilityExtension
	, public IRevisionControlExtension
	, public IPlayheadExtension
	, public IIdExtension
	, public IInTimeExtension
	, public IOutTimeExtension
{
public:
	UE_SEQUENCER_DECLARE_CASTABLE_API(UE_API, FNavigationToolSequence
		, FNavigationToolItem
		, Sequencer::IRenameableExtension
		, Sequencer::IDeactivatableExtension
		, Sequencer::ILockableExtension
		, IMarkerVisibilityExtension
		, IRevisionControlExtension
		, IPlayheadExtension
		, IIdExtension
		, IInTimeExtension
		, IOutTimeExtension)

	UE_API FNavigationToolSequence(INavigationTool& InTool
		, const FNavigationToolViewModelPtr& InParentItem
		, UMovieSceneSequence* const InSequence
		, UMovieSceneSubSection* const InSubSection
		, const int32 InSubSectionIndex);

	//~ Begin INavigationToolItem

	UE_API virtual bool IsItemValid() const override;
	UE_API virtual UObject* GetItemObject() const override;
	UE_API virtual bool IsAllowedInTool() const override;

	UE_API virtual void FindChildren(TArray<FNavigationToolViewModelWeakPtr>& OutWeakChildren, const bool bInRecursive) override;
	UE_API virtual void GetItemProxies(TArray<TSharedPtr<FNavigationToolItemProxy>>& OutItemProxies) override;

	UE_API virtual bool AddChild(const FNavigationToolAddItemParams& InAddItemParams) override;
	UE_API virtual bool RemoveChild(const FNavigationToolRemoveItemParams& InRemoveItemParams) override;

	UE_API virtual ENavigationToolItemViewMode GetSupportedViewModes(const INavigationToolView& InToolView) const override;
	virtual bool CanBeTopLevel() const override { return true; }

	UE_API virtual FText GetDisplayName() const override;
	UE_API virtual FText GetClassName() const override;
	UE_API virtual FSlateIcon GetIcon() const override;
	UE_API virtual FText GetIconTooltipText() const override;

	UE_API virtual bool IsSelected(const FNavigationToolScopedSelection& InSelection) const override;
	UE_API virtual void Select(FNavigationToolScopedSelection& InSelection) const override;

	UE_API virtual void OnSelect() override;
	UE_API virtual void OnDoubleClick() override;

	UE_API virtual void OnObjectsReplaced(const TMap<UObject*, UObject*>& InReplacementMap, bool bInRecursive) override;

	//~ End INavigationToolItem

	//~ Begin Sequencer::IRenameableExtension
	UE_API virtual bool CanRename() const override;
	UE_API virtual void Rename(const FText& InNewName) override;
	//~ End Sequencer::IRenameableExtension

	//~ Begin Sequencer::IDeactivatableExtension
	UE_API virtual bool IsDeactivated() const override;
	UE_API virtual void SetIsDeactivated(const bool bInIsDeactivated) override;
	//~ End Sequencer::IDeactivatableExtension

	//~ Begin Sequencer::ILockableExtension
	UE_API virtual Sequencer::ELockableLockState GetLockState() const override;
	UE_API virtual void SetIsLocked(const bool bInIsLocked) override;
	//~ End Sequencer::ILockableExtension

	//~ Begin IMarkerVisibilityExtension
	UE_API virtual EItemMarkerVisibility GetMarkerVisibility() const override;
	UE_API virtual void SetMarkerVisibility(const bool bInVisible) override;
	//~ End IMarkerVisibilityExtension

	//~ Begin IColorExtension
	UE_API virtual TOptional<FColor> GetColor() const override;
	UE_API virtual void SetColor(const TOptional<FColor>& InColor) override;
	//~ End IColorExtension

	//~ Begin IIdExtension
	UE_API virtual FText GetId() const override;
	//~ End IIdExtension

	//~ Begin IRevisionControlExtension
	UE_API virtual EItemRevisionControlState GetRevisionControlState() const override;
	UE_API virtual const FSlateBrush* GetRevisionControlStatusIcon() const override;
	UE_API virtual FText GetRevisionControlStatusText() const override;
	//~ End IRevisionControlExtension

	//~ Begin IPlayheadExtension
	UE_API virtual EItemContainsPlayhead ContainsPlayhead() const override;
	//~ End IPlayheadExtension

	//~ Begin IInTimeExtension
	UE_API virtual FFrameNumber GetInTime() const override;
	UE_API virtual void SetInTime(const FFrameNumber& InTime) override;
	//~ End IInTimeExtension

	//~ Begin IOutTimeExtension
	UE_API virtual FFrameNumber GetOutTime() const override;
	UE_API virtual void SetOutTime(const FFrameNumber& InTime) override;
	//~ End IOutTimeExtension

	UE_API UMovieSceneSequence* GetSequence() const;

	UE_API UMovieSceneSubSection* GetSubSection() const;
	UE_API int32 GetSubSectionIndex() const;

	UE_API UMovieScene* GetSequenceMovieScene() const;

	UE_API TArray<FMovieSceneBinding> GetSortedBindings() const;

	Sequencer::TViewModelPtr<Sequencer::FSectionModel> GetViewModel() const;

protected:
	//~Begin FNavigationToolItem
	UE_API virtual FNavigationToolItemId CalculateItemId() const override;
	//~End FNavigationToolItem

	TWeakObjectPtr<UMovieSceneSubSection> WeakSubSection;
	int32 SubSectionIndex = 0;
	TWeakObjectPtr<UMovieSceneSequence> WeakSequence;
};

} // namespace UE::SequenceNavigator

#undef UE_API
