// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Extensions/IMarkerVisibilityExtension.h"
#include "MVVM/Extensions/IDeactivatableExtension.h"
#include "MVVM/ViewModelTypeID.h"
#include "NavigationToolTrack.h"

#define UE_API SEQUENCENAVIGATOR_API

class UMovieSceneSequence;
class UMovieSceneSection;
class UMovieSceneSubTrack;

namespace UE::SequenceNavigator
{

class FNavigationToolSubTrack
	: public FNavigationToolTrack
	, public Sequencer::IDeactivatableExtension
	, public IMarkerVisibilityExtension
{
public:
	UE_SEQUENCER_DECLARE_CASTABLE_API(UE_API, FNavigationToolSubTrack
		, FNavigationToolTrack
		, Sequencer::IDeactivatableExtension
		, IMarkerVisibilityExtension)

	UE_API FNavigationToolSubTrack(INavigationTool& InTool
		, const FNavigationToolViewModelPtr& InParentItem
		, UMovieSceneSubTrack* const InSubTrack
		, const TWeakObjectPtr<UMovieSceneSequence>& InSequence
		, const TWeakObjectPtr<UMovieSceneSection>& InSection
        , const int32 InSubSectionIndex);

	//~ Begin INavigationToolItem
	UE_API virtual void FindChildren(TArray<FNavigationToolViewModelWeakPtr>& OutWeakChildren, const bool bInRecursive) override;
	UE_API virtual TOptional<EItemDropZone> CanAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone) override;
	UE_API virtual FReply AcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone) override;
	//~ End INavigationToolItem

	UE_API UMovieSceneSubTrack* GetSubTrack() const;

	//~ Begin Sequencer::IDeactivatableExtension
	UE_API virtual bool IsDeactivated() const override;
	UE_API virtual void SetIsDeactivated(const bool bInIsDeactivated) override;
	//~ End Sequencer::IDeactivatableExtension

	//~ Begin IMarkerVisibilityExtension
	UE_API virtual EItemMarkerVisibility GetMarkerVisibility() const override;
	UE_API virtual void SetMarkerVisibility(const bool bInVisible) override;
	//~ End IMarkerVisibilityExtension
};

} // namespace UE::SequenceNavigator

#undef UE_API
