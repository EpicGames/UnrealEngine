// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Extensions/IPlayheadExtension.h"
#include "MVVM/Extensions/ILockableExtension.h"
#include "MVVM/ViewModelTypeID.h"
#include "NavigationToolItem.h"
#include "Textures/SlateIcon.h"
#include "UObject/WeakObjectPtrTemplates.h"

#define UE_API SEQUENCENAVIGATOR_API

class UMovieSceneSequence;
class UMovieSceneSection;
class UMovieSceneTrack;

namespace UE::Sequencer
{
class FTrackModel;
}

namespace UE::SequenceNavigator
{

class FNavigationToolTrack
	: public FNavigationToolItem
	, public Sequencer::ILockableExtension
	, public IPlayheadExtension
{
public:
	UE_SEQUENCER_DECLARE_CASTABLE_API(UE_API, FNavigationToolTrack
		, FNavigationToolItem
		, Sequencer::ILockableExtension
		, IPlayheadExtension)

	FNavigationToolTrack(INavigationTool& InTool
		, const FNavigationToolViewModelPtr& InParentItem
		, UMovieSceneTrack* const InTrack
		, const TWeakObjectPtr<UMovieSceneSequence>& InSequence
		, const TWeakObjectPtr<UMovieSceneSection>& InSection
		, const int32 InSectionIndex);

	//~ Begin INavigationToolItem

	virtual bool IsItemValid() const override;
	virtual UObject* GetItemObject() const override;
	virtual bool IsAllowedInTool() const override;

	virtual void FindChildren(TArray<FNavigationToolViewModelWeakPtr>& OutWeakChildren, const bool bInRecursive) override;

	virtual bool CanBeTopLevel() const override { return false; }

	virtual FText GetDisplayName() const override;
	virtual FText GetClassName() const override;
	virtual const FSlateBrush* GetDefaultIconBrush() const override;
	virtual FSlateIcon GetIcon() const override;
	virtual FText GetIconTooltipText() const override;

	virtual bool IsSelected(const FNavigationToolScopedSelection& InSelection) const override;
	virtual void Select(FNavigationToolScopedSelection& InSelection) const override;

	virtual void OnSelect() override;
	virtual void OnDoubleClick() override;

	virtual void OnObjectsReplaced(const TMap<UObject*, UObject*>& InReplacementMap, bool bInRecursive) override;

	//~ End INavigationToolItem

	//~ Begin ILockableExtension
	virtual Sequencer::ELockableLockState GetLockState() const override;
	virtual void SetIsLocked(const bool bInIsLocked) override;
	//~ End ILockableExtension

	//~ Begin IPlayheadExtension
	virtual EItemContainsPlayhead ContainsPlayhead() const override;
	//~ End IPlayheadExtension

	UMovieSceneTrack* GetTrack() const;

	UMovieSceneSequence* GetSequence() const;
	UMovieSceneSection* GetSection() const;
	int32 GetSectionIndex() const;

	UE::Sequencer::TViewModelPtr<UE::Sequencer::FTrackModel> GetViewModel() const;

protected:
	//~ Begin FNavigationToolItem
	virtual FNavigationToolItemId CalculateItemId() const override;
	//~ End FNavigationToolItem

	void OnTrackObjectChanged();

	TWeakObjectPtr<UMovieSceneSequence> WeakSequence;
	TWeakObjectPtr<UMovieSceneSection> WeakSection;
	int32 SectionIndex = 0;
	TWeakObjectPtr<UMovieSceneTrack> WeakTrack;

	FSlateIcon Icon;
};

} // namespace UE::SequenceNavigator

#undef UE_API
