// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/INavigationToolFilterBar.h"
#include "Templates/SharedPointer.h"
#include "NavigationToolFilterBarContext.generated.h"

class UToolMenu;

DECLARE_DELEGATE_OneParam(FOnPopulateFilterBarMenu, UToolMenu*);

UCLASS()
class UNavigationToolFilterBarContext : public UObject
{
	GENERATED_BODY()

public:
	void Init(const TWeakPtr<UE::SequenceNavigator::INavigationToolFilterBar>& InFilterBarWeak)
	{
		WeakFilterBar = InFilterBarWeak;
	}

	TSharedPtr<UE::SequenceNavigator::INavigationToolFilterBar> GetFilterBar() const
	{
		return WeakFilterBar.IsValid() ? WeakFilterBar.Pin() : nullptr;
	}

	FOnPopulateFilterBarMenu OnPopulateMenu;

protected:
	TWeakPtr<UE::SequenceNavigator::INavigationToolFilterBar> WeakFilterBar;
};
