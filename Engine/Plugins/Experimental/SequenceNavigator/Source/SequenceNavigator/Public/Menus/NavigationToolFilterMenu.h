// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Delegates/Delegate.h"
#include "Styling/SlateTypes.h"

class FFilterCategory;
class UToolMenu;
struct FToolMenuSection;

namespace UE::SequenceNavigator
{

class FNavigationToolFilter;
class INavigationToolFilterBar;

class FNavigationToolFilterMenu : public TSharedFromThis<FNavigationToolFilterMenu>
{
public:
	static TSharedRef<SWidget> ConstructCustomMenuItemWidget(const TAttribute<FText>& InItemText
		, const FSimpleDelegate& InOnItemClicked
		, const TAttribute<ECheckBoxState>& InIsChecked
		, const FSlateBrush* const InButtonImage
		, const FSimpleDelegate& InOnButtonClicked
		, const bool bInRadioButton = false);

	TSharedRef<SWidget> CreateMenu(const TSharedRef<INavigationToolFilterBar>& InFilterBar);

protected:
	void PopulateMenu(UToolMenu* const InMenu);

	void PopulateCustomsSection(UToolMenu& InMenu);
	void PopulateFilterOptionsSection(UToolMenu& InMenu);
	void PopulateCommonFilterSections(UToolMenu& InMenu);

	void FillCustomTextFiltersMenu(UToolMenu* const InMenu);

	void FillFiltersMenuCategory(FToolMenuSection& InOutSection, const TSharedRef<FFilterCategory> InMenuCategory);
	void FillFiltersMenuCategory(UToolMenu* const InMenu, const TSharedRef<FFilterCategory> InMenuCategory);

 	void OnFilterCategoryClicked(const TSharedRef<FFilterCategory> InMenuCategory);
 	ECheckBoxState GetFilterCategoryCheckedState(const TSharedRef<FFilterCategory> InMenuCategory) const;

 	void OnFilterClicked(const TSharedRef<FNavigationToolFilter> InFilter);

	void OnCustomTextFilterClicked(const FText InFilterLabel);
	ECheckBoxState GetCustomTextFilerCheckState(const FText InFilterLabel) const;
	void OnEditCustomTextFilterClicked(const FText InFilterLabel);

	bool CanResetFilters() const;
	void ResetFilters();

private:
	TWeakPtr<INavigationToolFilterBar> WeakFilterBar;
};

} // namespace UE::SequenceNavigator
