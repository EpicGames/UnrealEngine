// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/Filters/NavigationToolFilterBase.h"
#include "Menus/NavigationToolFilterBarContext.h"
#include "Templates/SharedPointer.h"
#include "NavigationToolFilterMenuContext.generated.h"

UCLASS()
class UNavigationToolFilterMenuContext : public UObject
{
	GENERATED_BODY()

public:
	void Init(const TWeakPtr<UE::SequenceNavigator::FNavigationToolFilter>& InFilter)
	{
		WeakFilter = InFilter;
	}

	TSharedPtr<UE::SequenceNavigator::FNavigationToolFilter> GetFilter() const
	{
		return WeakFilter.Pin();
	}

	FOnPopulateFilterBarMenu OnPopulateMenu;

protected:
	TWeakPtr<UE::SequenceNavigator::FNavigationToolFilter> WeakFilter;
};
