// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ContainersFwd.h"
#include "NavigationToolDefines.h"
#include "Templates/SharedPointer.h"
#include "UObject/Object.h"
#include "NavigationToolItemMenuContext.generated.h"

namespace UE::SequenceNavigator
{
class INavigationTool;
}

UCLASS(MinimalAPI)
class UNavigationToolItemMenuContext : public UObject
{
	GENERATED_BODY()

	friend class FNavigationToolView;

public:
	void Init(const TWeakPtr<UE::SequenceNavigator::INavigationTool>& InTool
		, const TArray<UE::Sequencer::TWeakViewModelPtr<UE::SequenceNavigator::INavigationToolItem>>& InWeakItemList)
	{
		WeakTool = InTool;
		WeakItemList.Append(InWeakItemList);
	}

	TSharedPtr<UE::SequenceNavigator::INavigationTool> GetTool() const
	{
		return WeakTool.Pin();
	}

	TConstArrayView<UE::Sequencer::TWeakViewModelPtr<UE::SequenceNavigator::INavigationToolItem>> GetItems() const
	{
		return WeakItemList;
	}

private:
	TWeakPtr<UE::SequenceNavigator::INavigationTool> WeakTool;

	TArray<UE::Sequencer::TWeakViewModelPtr<UE::SequenceNavigator::INavigationToolItem>> WeakItemList;
};
