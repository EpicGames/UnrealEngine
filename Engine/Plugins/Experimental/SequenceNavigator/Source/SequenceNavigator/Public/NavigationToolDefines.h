// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "MVVM/ViewModelPtr.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectMacros.h"

#define UE_API SEQUENCENAVIGATOR_API

namespace UE::SequenceNavigator
{

class INavigationToolItem;

using FNavigationToolViewModelPtr = Sequencer::TViewModelPtr<INavigationToolItem>;
using FNavigationToolViewModelWeakPtr = Sequencer::TWeakViewModelPtr<INavigationToolItem>;

using FNavigationToolSharedPtr = TSharedPtr<Sequencer::TViewModelPtr<INavigationToolItem>::ViewModelType>;
using FNavigationToolSharedRef = TSharedRef<Sequencer::TViewModelPtr<INavigationToolItem>::ViewModelType>;

UE_API FName GetToolBarMenuName();
UE_API FName GetItemContextMenuName();

} // namespace UE::SequenceNavigator

/** Flags specifying how an Item should be Selected */
enum class ENavigationToolItemSelectionFlags : uint8
{
	None = 0,
	/** Append to the current selection*/
	AppendToCurrentSelection = 1 << 0,
	/** Signal selection change (e.g. trigger on selection change delegate) */
	SignalSelectionChange = 1 << 1,
	/** Auto-include the items' children in the selection */
	IncludeChildren = 1 << 2,
	/** Whether to scroll first item into view */
	ScrollIntoView = 1 << 3,
};
ENUM_CLASS_FLAGS(ENavigationToolItemSelectionFlags);

enum class ENavigationToolIgnoreNotifyFlags : uint8
{
	None = 0,
	/** Ignores automatically handling actor spawns, usually so that it is manually handled */
	Spawn = 1 << 0,
	/** Ignores automatically handling actor duplications, usually so that it is manually handled */
	Duplication = 1 << 1,
	All = 0xFF,
};
ENUM_CLASS_FLAGS(ENavigationToolIgnoreNotifyFlags);

enum class ENavigationToolRenameAction : uint8
{
	None,
	Requested,
	Completed,
	Cancelled,
};

enum class ENavigationToolExtensionPosition
{
	Before,
	After,
};

enum class ENavigationToolDragDropActionType : uint8
{
	Move,
	Copy,
};

#undef UE_API
