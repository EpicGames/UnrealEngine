// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DragDropOps/NavigationToolItemDragDropOp.h"
#include "Features/IModularFeature.h"
#include "ItemProxies/NavigationToolItemProxyRegistry.h"
#include "Providers/NavigationToolProvider.h"

#define UE_API SEQUENCENAVIGATOR_API

namespace UE::SequenceNavigator
{

class INavigationTool;
class INavigationToolIconCustomization;

enum class ENavigationToolProvidersChangeType
{
	Add,
	Remove
};

DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnNavigationToolProvidersChanged
	, const FName /*InToolId*/
	, const TSharedRef<FNavigationToolProvider>& /*InProvider*/
	, const ENavigationToolProvidersChangeType /*InChangeType*/);

/** Called when the FNavigationToolItemDragDropOp has been created and Initialized in FNavigationToolItemDragDropOp::Init */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnItemDragDropOpInitialized, FNavigationToolItemDragDropOp&);

/***
 * Singleton class for extending the Sequencer Navigation Tool.
 * 
 * This class will watch for Sequencers being created and if a Navigation Tool has been registered
 * for that specific sequencer, it will create a Navigation Tool instance for that Sequencer instance.
 */
class FNavigationToolExtender : IModularFeature
{
public:
	UE_API static FNavigationToolExtender& Get();

	FNavigationToolExtender();
	virtual ~FNavigationToolExtender();

	static FName GetModularFeatureName();

	/**
	 * Retrieves the unique name for the provided Sequencer instance.
	 * The Navigation Tool instance name is based on its associated sequencer settings
	 * object name. This provides a unique instance name for each Sequencer type.
	 * @param InSequencer The Sequencer instance for which the name is being retrieved.
	 * @return The name of the Navigation Tool instance if available; otherwise, NAME_None.
	 */
	UE_API static FName GetToolInstanceId(const ISequencer& InSequencer);
	UE_API static FName GetToolInstanceId(const INavigationTool& InTool);

	/** Find the Navigation Tool instance associated with the given Sequencer instance */
	UE_API static TSharedPtr<INavigationTool> FindNavigationTool(const TSharedRef<ISequencer>& InSequencer);

	/**
	 * Registers a Navigation Tool provider for a specified Navigation Tool instance.
	 * This method ensures that the provider is associated with the given tool and replaces any
	 * existing provider with the same identifier, if one exists.
	 * @param InSequencer The Sequencer instance to create the Navigation Tool instance for.
	 * @param InProvider The Navigation Tool provider being registered.
	 * @return True if the provider was successfully registered, false otherwise.
	 */
	UE_API static bool RegisterToolProvider(const TSharedRef<ISequencer>& InSequencer
		, const TSharedRef<FNavigationToolProvider>& InProvider);

	/**
	 * Unregisters a Navigation Tool provider associated with a specific instance and identifier.
	 * @param InToolId The Id of the Navigation Tool from which the provider needs to be unregistered.
	 * @param InProviderId The identifier of the provider to be unregistered.
	 * @return True if the provider was successfully unregistered; false otherwise.
	 */
	UE_API static bool UnregisterToolProvider(const FName InToolId, const FName InProviderId);

	/**  */
	UE_API static TSharedPtr<FNavigationToolProvider> FindToolProvider(const FName InToolId, const FName InProviderId);

	/**
	 * Finds the tool providers associated with the given Navigation Tool instance.
	 * @param InToolId The Navigation Tool Id for which to find associated providers.
	 * @param OutProviders A set that will be populated with the found tool providers as weak pointers.
	 * @return True if tool providers were found and populated into OutProviders; otherwise, false.
	 */
	UE_API static bool FindToolProviders(const FName InToolId
		, TSet<TSharedRef<FNavigationToolProvider>>& OutProviders);

	/**
	 * Iterates over each Navigation Tool provider and executes the predicate function.
	 * The provided predicate function determines whether the iteration continues or stops.
	 * @param InPredicate A function that accepts a reference to an INavigationTool and a shared reference
	 *                    to FNavigationToolProvider. Should return true to continue iteration or false to stop.
	 */
	UE_API static void ForEachProvider(const TFunction<bool(const FName InToolId
		, const TSharedRef<FNavigationToolProvider>& InProvider)>& InPredicate);

	/**
	 * Iterates over each Navigation Tool provider and executes the predicate function associated with the given
	 * Navigation Tool instance. The iteration is controlled by the provided predicate function, which determines
	 * whether it continues or stops.
	 * @param InToolId The Navigation Tool Id whose providers are to be iterated over.
	 * @param InPredicate A predicate function that takes a shared reference to a Navigation Tool provider as input.
	 *                    The function should return true to continue iteration or false to stop it.
	 */
	UE_API static void ForEachToolProvider(const FName InToolId
		, const TFunction<bool(const TSharedRef<FNavigationToolProvider>& InProvider)>& InPredicate);

	/** Event called when a provider has been added or removed from a tool instance. */
	static FOnNavigationToolProvidersChanged& OnProvidersChanged()
	{
		return Get().ProvidersChangedDelegate;
	}

	/** Event called when the FNavigationToolItemDragDropOp has been created and initialized in FNavigationToolItemDragDropOp::Init. */
	static FOnItemDragDropOpInitialized& OnItemDragDropOpInitialized()
	{
		return Get().ItemDragDropOpInitializedDelegate;
	}

	static FNavigationToolItemProxyRegistry& GetItemProxyRegistry()
	{
		return Get().ItemProxyRegistry;
	}

	/**
	 * Get the custom icon for the item if a customization for it exists.
	 * @param InItem The item to find the icon of
	 * @return The custom icon for the item if a customization is found, FSlateIcon() otherwise
	 */
	UE_API static FSlateIcon FindOverrideIcon(const FNavigationToolViewModelPtr& InItem);

	static bool AnyProviderSupportsSequencer(const FName InToolId, const ISequencer& InSequencer);

protected:
	struct FNavigationToolInstance
	{
		/** The Sequencer type name to register the provider for */
		FName ToolId;

		TSharedPtr<FNavigationTool> Instance;

		/** The sequencer instance for this Navigation Tool instance. May be null if the Sequencer is not currently open */
		TWeakPtr<ISequencer> WeakSequencer;

		FDelegateHandle ActivateSequenceHandle;
		FDelegateHandle SequencerClosedHandle;

		/** List of providers registered to this tool instance */
		TSet<TSharedRef<FNavigationToolProvider>> Providers;
	};

	static TSet<TSharedRef<FNavigationToolProvider>>* FindToolProviders(const FName InToolId);

	FNavigationToolInstance* FindOrAddToolInstance_Internal(const TSharedRef<ISequencer>& InSequencer);

	void OnSequencerCreated(const TSharedRef<ISequencer> InSequencer);
	void OnSequencerActivated(const FName InToolId, FMovieSceneSequenceIDRef InSequenceId);
	void OnSequencerClosed(const FName InToolId, const TSharedRef<ISequencer> InSequencer);

	void RegisterOverridenIcon_Internal(const Sequencer::FViewModelTypeID& InItemTypeId, const TSharedRef<INavigationToolIconCustomization>& InIconCustomization);
	void UnregisterOverriddenIcon_Internal(const Sequencer::FViewModelTypeID& InItemTypeId, const FName& InSpecializationIdentifier);

	/**
	 * Get the customization for the given item if any.
	 * @param InItem Item to search the customization for.
	 * @return The customization for the item if any; nullptr otherwise.
	 */
	TSharedPtr<INavigationToolIconCustomization> GetCustomizationForItem(const FNavigationToolViewModelPtr& InItem) const;

	FDelegateHandle SequencerCreatedHandle;

	static void AddSequencerToolBarExtension(const FNavigationToolInstance& InToolInstance);
	static void RemoveSequencerToolBarExtension(const FNavigationToolInstance& InToolInstance);

	/** Registered Sequencer types to Navigation Tool instances */
	TMap<FName, FNavigationToolInstance> ToolInstances;

	FOnNavigationToolProvidersChanged ProvidersChangedDelegate;

	FOnItemDragDropOpInitialized ItemDragDropOpInitializedDelegate;

	FNavigationToolItemProxyRegistry ItemProxyRegistry;

	/** Hold the key of the map containing the IconCustomizations */
	struct FIconCustomizationKey
	{
		/** Navigation Tool item class FName (ex. NavigationToolItem) */
		//FNavigationToolItemTypeId ItemTypeId = FNavigationToolItemTypeId::Invalid();
		uint32 ItemTypeId = 0;

		/** Specialization identifier (ex. AvaShapeActor class FName for ActorIcon customization) */
		FName CustomizationSpecializationIdentifier;

		bool operator==(const FIconCustomizationKey& InOther) const
		{
			return ItemTypeId == InOther.ItemTypeId
				&& CustomizationSpecializationIdentifier == InOther.CustomizationSpecializationIdentifier;
		}

		friend uint32 GetTypeHash(const FIconCustomizationKey& InCustomizationKey)
		{
			return HashCombine(GetTypeHash(InCustomizationKey.ItemTypeId)
				, GetTypeHash(InCustomizationKey.CustomizationSpecializationIdentifier));
		}
	};

	TMap<FIconCustomizationKey, TSharedPtr<INavigationToolIconCustomization>> IconRegistry;
};

} // namespace UE::SequenceNavigator

#undef UE_API
