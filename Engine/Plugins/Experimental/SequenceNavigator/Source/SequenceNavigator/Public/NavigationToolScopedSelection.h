// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "MovieSceneSequence.h"

#define UE_API SEQUENCENAVIGATOR_API

class ISequencer;
class UMovieSceneSection;
class UMovieSceneTrack;
class UObject;

namespace UE::SequenceNavigator
{

enum class ENavigationToolScopedSelectionPurpose
{
	/** At the end of the Scope, it will set whatever has been added to the Selected List to be the new Selection */
	Sync,
	/** Used only to check for whether an Object is Selected or not. Cannot execute "Select" */
	Read,
};

/** Handler to Sync Selection from Navigation Tool to the Sequencer */
class FNavigationToolScopedSelection
{
public:
	explicit FNavigationToolScopedSelection(ISequencer& InSequencer
		, ENavigationToolScopedSelectionPurpose InPurpose = ENavigationToolScopedSelectionPurpose::Read);

	~FNavigationToolScopedSelection();

	UE_API void Select(const FGuid& InObjectGuid);
	UE_API void Select(UMovieSceneSection* const InSection);
	UE_API void Select(UMovieSceneTrack* const InTrack);
	UE_API void Select(UMovieSceneSequence* const InSequence
		, const int32 InMarkedFrameIndex);

	UE_API bool IsSelected(const UObject* const InObject) const;
	UE_API bool IsSelected(const FGuid& InObjectGuid) const;
	UE_API bool IsSelected(UMovieSceneSection* const InSection) const;
	UE_API bool IsSelected(UMovieSceneTrack* const InTrack) const;
	UE_API bool IsSelected(UMovieSceneSequence* const InSequence
		, const int32 InMarkedFrameIndex) const;

	UE_API ISequencer& GetSequencer() const;

private:
	void SyncSelections();

	ISequencer& Sequencer;

	/** All Objects Selected (Sections, Tracks, Objects) */
	TSet<const UObject*> ObjectsSet;

	TArray<FGuid> SelectedObjectGuids;
	TArray<UMovieSceneSection*> SelectedSections;
	TArray<UMovieSceneTrack*> SelectedTracks;
	TMap<UMovieSceneSequence*, TSet<int32>> SelectedMarkedFrames;

	ENavigationToolScopedSelectionPurpose Purpose;
};

} // namespace UE::SequenceNavigator

#undef UE_API
