// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "Filters/SequencerFilterBarConfig.h"
#include "NavigationToolSettings.generated.h"

/** The type of visualization being done to the item */
UENUM(BlueprintType, meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class ENavigationToolItemViewMode : uint8
{
	None = 0 UMETA(Hidden),

	/** Navigation Tool Tree Hierarchy View of the Items */
	ItemTree = 1 << 0,
	/** Flattened Horizontal List of Nested Items shown in the "Items" column */
	HorizontalItemList = 1 << 1,

	/** All the Views */
	All = ItemTree | HorizontalItemList
};
ENUM_CLASS_FLAGS(ENavigationToolItemViewMode);

USTRUCT()
struct FNavigationToolColumnView
{
	GENERATED_BODY()

	FNavigationToolColumnView() = default;
	FNavigationToolColumnView(const FText& InViewName)
		: ViewName(InViewName)
	{}

	UPROPERTY()
	FText ViewName;

	UPROPERTY()
	TSet<FName> VisibleColumns;

	bool operator==(const FNavigationToolColumnView& InOtherView) const
	{
		return ViewName.EqualTo(InOtherView.ViewName);
	}

	friend uint32 GetTypeHash(const FNavigationToolColumnView& InOtherView)
	{
		return GetTypeHash(InOtherView.ViewName.ToString());
	} 
};

UCLASS(config=EditorPerProjectUserSettings, meta=(DisplayName="Sequence Navigation"))
class UNavigationToolSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	static void OpenEditorSettings();

	UNavigationToolSettings();

	bool ShouldApplyDefaultColumnView() const;
	void SetApplyDefaultColumnView(const bool bInApplyDefaultColumnView);

	bool ShouldUseShortNames() const;
	void SetUseShortNames(const bool bInUseShortNames);

	bool ShouldUseMutedHierarchy() const;
	void SetUseMutedHierarchy(const bool bInUseMutedHierarchy);

	bool ShouldAutoExpandToSelection() const;
	void SetAutoExpandToSelection(const bool bInAutoExpandToSelection);

	bool ShouldAlwaysShowLockState() const;
	void SetAlwaysShowLockState(const bool bInAlwaysShowLockState);

	ENavigationToolItemViewMode GetItemDefaultViewMode() const;
	ENavigationToolItemViewMode GetItemProxyViewMode() const;

	void ToggleViewModeSupport(ENavigationToolItemViewMode& InOutViewMode, const ENavigationToolItemViewMode InFlags);
	void ToggleItemDefaultViewModeSupport(const ENavigationToolItemViewMode InFlags);
	void ToggleItemProxyViewModeSupport(const ENavigationToolItemViewMode InFlags);

	TSet<FNavigationToolColumnView>& GetCustomColumnViews() { return CustomColumnViews; }
	FNavigationToolColumnView* FindCustomColumnView(const FText& InColumnViewName);

	const TSet<FName>& GetEnabledBuiltInFilters() const { return EnabledBuiltInFilters; }
	void SetBuiltInFilterEnabled(const FName InFilterName, const bool bInEnabled);

	FSequencerFilterBarConfig& FindOrAddFilterBar(const FName InIdentifier, const bool bInSaveConfig);
	FSequencerFilterBarConfig* FindFilterBar(const FName InIdentifier);
	bool RemoveFilterBar(const FName InIdentifier);

	bool ShouldAutoExpandNodesOnFilterPass() const;
	void SetAutoExpandNodesOnFilterPass(const bool bInAutoExpand);

	bool ShouldUseFilterSubmenusForCategories() const;
	void SetUseFilterSubmenusForCategories(const bool bInUseFilterSubmenusForCategories);

	bool IsFilterBarVisible() const;
	void SetFilterBarVisible(const bool bInVisible);

	EFilterBarLayout GetFilterBarLayout() const;
	void SetFilterBarLayout(const EFilterBarLayout InLayout);

	float GetLastFilterBarSizeCoefficient() const;
	void SetLastFilterBarSizeCoefficient(const float bInSizeCoefficient, const bool bInSaveConfig);

	bool ShouldSyncSelectionToNavigationTool() const;
	void SetSyncSelectionToNavigationTool(const bool bInSync, const bool bInSaveConfig = true);

	bool ShouldSyncSelectionToSequencer() const;
	void SetSyncSelectionToSequencer(const bool bInSync, const bool bInSaveConfig = true);

private:
	/** Applies the default column view set by the provider when a view is loaded */
	UPROPERTY(Config, EditAnywhere, Category = "Editor UX")
	bool bApplyDefaultColumnView = true;

	/** Shortens child item names that contain their parents name as a prefix by excluding it from the displayed child name */
	UPROPERTY(Config, EditAnywhere, Category = "Editor UX")
	bool bUseShortNames = true;

	/** Whether to show the parent of the shown items, even if the parents are filtered out */
	UPROPERTY(Config, EditAnywhere, Category = "Editor UX")
	bool bUseMutedHierarchy = true;

	/** Whether to auto expand the hierarchy to show the item when selected */
	UPROPERTY(Config, EditAnywhere, Category = "Editor UX")
	bool bAutoExpandToSelection = true;

	/** Whether to show the lock state always, rather than only showing when the item is locked or hovered */
	UPROPERTY(Config, EditAnywhere, Category = "Editor UX")
	bool bAlwaysShowLockState = false;

	/** The View Mode a Non-Actor / Non-Component Item supports by default */
	UPROPERTY(Config, EditAnywhere, Category = "Editor UX", meta=(Bitmask, BitmaskEnum="/Script/Sequencer.ENavigationToolItemViewMode"))
	ENavigationToolItemViewMode ItemDefaultViewMode = ENavigationToolItemViewMode::None;

	/** The View Mode a Proxy Item supports by default */
	UPROPERTY(Config, EditAnywhere, Category = "Editor UX", meta=(Bitmask, BitmaskEnum="/Script/Sequencer.ENavigationToolItemViewMode"))
	ENavigationToolItemViewMode ItemProxyViewMode = ENavigationToolItemViewMode::None;

	UPROPERTY(Config)
	TSet<FNavigationToolColumnView> CustomColumnViews;

	UPROPERTY(Config)
	TSet<FName> EnabledBuiltInFilters;

	/** Saved settings for each unique filter bar instance mapped by instance identifier */
	UPROPERTY(Config)
	TMap<FName, FSequencerFilterBarConfig> FilterBars;

	/** Automatically expand tracks that pass filters */
	UPROPERTY(Config, EditAnywhere, Category = "Filtering")
	bool bAutoExpandNodesOnFilterPass = false;
	
	/** Display the filter menu categories as submenus instead of sections */
	UPROPERTY(Config, EditAnywhere, Category = "Filtering")
	bool bUseFilterSubmenusForCategories = false;

	/** Last saved visibility of the filter bar to restore after closed */
	UPROPERTY(Config, EditAnywhere, Category = "Filtering")
	bool bFilterBarVisible = false;

	/** Last saved layout orientation of the filter bar to restore after closed */
	UPROPERTY(Config, EditAnywhere, Category = "Filtering")
	EFilterBarLayout LastFilterBarLayout = EFilterBarLayout::Horizontal;

	/** Last saved size of the filter bar to restore after closed */
	UPROPERTY(Config)
	float LastFilterBarSizeCoefficient = 0.05f;

	/** If true, syncs selections in Sequencer to Navigation Tool */
	UPROPERTY(Config, EditAnywhere, Category = "Editor UX")
	bool bSyncSelectionToNavigationTool = true;

	/** If true, syncs selections in Navigation Tool to Sequencer */
	UPROPERTY(Config, EditAnywhere, Category = "Editor UX")
	bool bSyncSelectionToSequencer = true;
};
