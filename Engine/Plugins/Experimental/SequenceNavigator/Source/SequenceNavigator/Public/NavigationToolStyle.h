// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

class FNavigationToolStyle final : public FSlateStyleSet
{
public:
	static FNavigationToolStyle& Get()
	{
		static FNavigationToolStyle Instance;
		return Instance;
	}

	FNavigationToolStyle();
	virtual ~FNavigationToolStyle() override;
};
