// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"
#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"
#include "Internationalization/Text.h"
#include "Filters/Filters/NavigationToolBuiltInFilterParams.h"
#include "NavigationToolDefines.h"

class FDragDropEvent;
class FReply;
class UMovieSceneSequence;
struct FNavigationToolColumnView;
struct FNavigationToolSaveState;
enum class EItemDropZone;

namespace UE::SequenceNavigator
{

class FNavigationToolColumnExtender;
class FNavigationToolItemProxy;
class INavigationTool;

/**
 * Provides the Navigation Tool with functionality it is not responsible for.
 */
class INavigationToolProvider : public TSharedFromThis<INavigationToolProvider>
{
public:
	virtual ~INavigationToolProvider() = default;

	/** @return Unique identifier name used to distinguish this provider from others */
	virtual FName GetIdentifier() const = 0;

	/** Gets the sequence classes that this provider is designed to support */
	virtual TSet<TSubclassOf<UMovieSceneSequence>> GetSupportedSequenceClasses() const = 0;

	/** @return Saved editor only state data to be restored */
	virtual FNavigationToolSaveState* GetSaveState(const INavigationTool& InTool) const = 0;

	/** Saves the editor only state data to be restored */
	virtual void SetSaveState(const INavigationTool& InTool, const FNavigationToolSaveState& InSaveState) const = 0;

	/** Event called when this provider is activated. A provider is activated when a Sequencer opens a sequence. */
	virtual void OnActivate() {}

	/** Event called when this provider is deactivated. A provider is deactivated when a Sequencer closes. */
	virtual void OnDeactivate() {}

	virtual void BindCommands(const TSharedRef<FUICommandList>& InCommandList) {}

	/**
	 * Extends the available list of columns that can be displayed in the Navigation Tool instance.
	 * Users can override this method to add or modify columns in the Navigation Tool interface.
	 * @param OutExtender A reference to the column extender used to register additional custom columns.
	 */
	virtual void OnExtendColumns(FNavigationToolColumnExtender& OutExtender) {}

	/**
	 * Extends the available list of column views that can be displayed in the Navigation Tool instance.
	 * Users can override this method to add or modify column views in the Navigation Tool interface.
	 * @param OutColumnViews A set of column views that can be customized by the user.
	 */
	virtual void OnExtendColumnViews(TSet<FNavigationToolColumnView>& OutColumnViews) {}

	/**
	 * Extends the children of a specified item in the Navigation Tool instance.
	 * Users can override this method to add or manipulate the child items of a given parent item.
	 * @param InTool The Navigation Tool instance where the item resides.
	 * @param InParentItem The item whose children are to be extended.
	 * @param OutWeakChildren The array to which additional child items will be added.
	 * @param bInRecursive Indicates whether to recursively find children for the provided item.
	 */
	virtual void OnExtendItemChildren(INavigationTool& InTool
		, const FNavigationToolViewModelPtr& InParentItem
		, TArray<FNavigationToolViewModelWeakPtr>& OutWeakChildren
		, const bool bInRecursive) {}

	/**
	 * Extends the list of item proxies for a specific item in a Navigation Tool instance.
	 * Users can override this method to add or manipulate item proxies associated with a given item.
	 * @param InTool The Navigation Tool instance where the item resides.
	 * @param InItem The item for which proxies are being extended.
	 * @param OutItemProxies The array to which additional item proxies will be added.
	 */
	virtual void OnExtendItemProxiesForItem(INavigationTool& InTool
		, const FNavigationToolViewModelPtr& InItem
		, TArray<TSharedPtr<FNavigationToolItemProxy>>& OutItemProxies) {}

	/**
	 * Extends the built-in filters available in the Navigation Tool instance.
	 * Users can override this method to add or customize filters in the tool's filtering system.
	 * @param OutFilterParams An array to which additional built-in filter parameters will be added.
	 */
	virtual void OnExtendBuiltInFilters(TArray<FNavigationToolBuiltInFilterParams>& OutFilterParams) {}

	/** Determines whether the Navigation Tool should be read-only */
	virtual bool ShouldLockTool() const = 0;

	/** An extended check to determine whether Item should be hidden in the Navigation Tool */
	virtual bool ShouldHideItem(const FNavigationToolViewModelPtr& InItem) const = 0;

	/** @return The name of the default column view to load when this provider is registered  */
	virtual FText GetDefaultColumnView() const { return FText::GetEmpty(); }

	/** Determines whether an external Drag Drop event (i.e. not an Navigation Tool one) can be accepted by the Navigation Tool for a given Target Item */
	virtual TOptional<EItemDropZone> OnToolItemCanAcceptDrop(const FDragDropEvent& InDragDropEvent
		, const EItemDropZone InDropZone
		, const FNavigationToolViewModelPtr& InTargetItem) const = 0;

	/** Processes an external Drag Drop event (i.e. not an Navigation Tool one) for a given Target item */
	virtual FReply OnToolItemAcceptDrop(const FDragDropEvent& InDragDropEvent
		, const EItemDropZone InDropZone
		, const FNavigationToolViewModelPtr& InTargetItem) = 0;

	virtual void UpdateItemIdContexts(const INavigationTool& InTool) = 0;

	/** Event called when an Item has been renamed */
	virtual void NotifyToolItemRenamed(const FNavigationToolViewModelPtr& InItem) {}

	/** Event called when an Item has been deleted */
	virtual void NotifyToolItemDeleted(const FNavigationToolViewModelPtr& InItem) {}
};

} // namespace UE::SequenceNavigator
