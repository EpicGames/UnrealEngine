// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NavigationToolDefines.h"
#include "Providers/INavigationToolProvider.h"

#define UE_API SEQUENCENAVIGATOR_API

class UMovieSceneSequence;
struct FNavigationToolBuiltInFilterParams;
struct FNavigationToolSerializedTree;
struct FNavigationToolViewSaveState;

namespace UE::SequenceNavigator
{

class FNavigationTool;
class INavigationTool;

/** Base Navigation Tool Provider to extend from */
class FNavigationToolProvider : public INavigationToolProvider
{
public:
	//~ Begin INavigationToolProvider

	UE_API virtual void OnExtendColumnViews(TSet<FNavigationToolColumnView>& OutColumnViews) override;
	UE_API virtual void OnExtendBuiltInFilters(TArray<FNavigationToolBuiltInFilterParams>& OutFilterParams) override;

	virtual bool ShouldLockTool() const override { return false; }
	virtual bool ShouldHideItem(const FNavigationToolViewModelPtr& InItem) const override { return false; }

	UE_API virtual void UpdateItemIdContexts(const INavigationTool& InTool) final override;

	UE_API virtual TOptional<EItemDropZone> OnToolItemCanAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, const FNavigationToolViewModelPtr& InTargetItem) const override;
	UE_API virtual FReply OnToolItemAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, const FNavigationToolViewModelPtr& InTargetItem) override;

	//~ End INavigationToolProvider

	/** @return Editor only saved data for a specific view */
	FNavigationToolViewSaveState* GetViewSaveState(const INavigationTool& InTool, const int32 InToolViewId) const;

	/** Ensure saved editor only data contains an entry for a specific view, creating it if necessary */
	void EnsureToolViewCount(const INavigationTool& InTool, const int32 InToolViewId);

	UE_API bool IsSequenceSupported(UMovieSceneSequence* const InSequence) const;

private:
	friend class FNavigationToolExtender;
	friend class FNavigationTool;

	void Activate(FNavigationTool& InTool);
	void Deactivate(FNavigationTool& InTool);

	void SaveState(FNavigationTool& InTool);
	void LoadState(FNavigationTool& InTool);

	void SaveSerializedTree(FNavigationTool& InTool, const bool bInResetTree);

	void SaveSerializedTreeRecursive(const FNavigationToolViewModelPtr& InParentItem
		, FNavigationToolSerializedTree& InSerializedTree);

	void LoadSerializedTree(const FNavigationToolViewModelPtr& InParentItem
		, FNavigationToolSerializedTree* const InSerializedTree);

	void CleanupExtendedColumnViews();

	TArray<FText> ExtendedColumnViewNames;
	TArray<FName> ExtendedBuiltInFilterNames;
};

} // namespace UE::SequenceNavigator

#undef UE_API
