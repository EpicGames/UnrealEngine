// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Items/NavigationToolItem.h"
#include "MVVM/ViewModelPtr.h"
#include "NavigationToolDefines.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/Views/STableRow.h"

namespace UE::SequenceNavigator
{

class FNavigationToolView;
class SNavigationToolTreeView;

class SNavigationToolTreeRow
	: public SMultiColumnTableRow<FNavigationToolViewModelWeakPtr>
{
public:
	SLATE_BEGIN_ARGS(SNavigationToolTreeRow) {}
		SLATE_ATTRIBUTE(FText, HighlightText)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs
		, const TSharedRef<FNavigationToolView>& InToolView
		, const TSharedPtr<SNavigationToolTreeView>& InTreeView
		, const FNavigationToolViewModelPtr& InItem);

	//~ Begin SMultiColumnTableRow
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override;
	//~ End SMultiColumnTableRow

	TAttribute<FText> GetHighlightText() const { return HighlightText; }

	virtual FReply OnMouseButtonUp(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) override;
	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent) override;

	TSharedPtr<FNavigationToolView> GetToolView() const;

	const FTableRowStyle* GetStyle() const { return Style; }

	/** The default reply if a row did not handle AcceptDrop */
	FReply OnDefaultDrop(const FDragDropEvent& InDragDropEvent) const;

protected:
	FNavigationToolViewModelWeakPtr WeakItem;
	
	TWeakPtr<SNavigationToolTreeView> WeakTreeView;
	
	TWeakPtr<FNavigationToolView> WeakToolView;
	
	TAttribute<FText> HighlightText;
};

} // namespace UE::SequenceNavigator
