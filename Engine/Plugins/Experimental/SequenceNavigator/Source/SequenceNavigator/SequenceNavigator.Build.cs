// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SequenceNavigator : ModuleRules
{
	public SequenceNavigator(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicIncludePaths.AddRange(new string[]
		{
		});

		PrivateIncludePaths.AddRange(new string[]
		{
		});

		PublicDependencyModuleNames.AddRange(new string[]
		{
			"Core",
			"SequencerCore"
		});

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"AppFramework",
			"BlueprintGraph",
			"CoreUObject",
			"DeveloperSettings",
			"Engine",
			"InputCore",
			"LevelSequence",
			"MovieScene",
			"MovieSceneTools",
			"MovieSceneTracks",
			"Projects",
			"Sequencer",
			"Slate",
			"SlateCore",
			"SourceControl",
			"ToolMenus",
			"ToolWidgets",
			"UnrealEd",
			"WorkspaceMenuStructure"
		});

		DynamicallyLoadedModuleNames.AddRange(new string[]
		{
		});
	}
}
